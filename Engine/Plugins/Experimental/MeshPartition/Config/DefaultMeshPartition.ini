[/Script/MeshPartition.Settings]
DefaultDefinition=/MeshPartition/DataAssets/MPD_Default.MPD_Default

[CoreRedirects]
; Tessellate modifier removed, redirected to Remesh modifier
; Note: BaseEngine.ini already redirects MegaMeshEditor.MegaMeshTessellateModifier(Method) to MegaMeshEditor.TessellateModifier(Method)
+EnumRedirects=(OldName="/Script/MegaMeshEditor.TessellateModifierMethod", NewName="/Script/MeshPartitionEditor.MegaMeshRemeshModifierTessellateMethod")
+ClassRedirects=(OldName="/Script/MegaMeshEditor.TessellateModifier", NewName="/Script/MeshPartitionEditor.RemeshModifier")
+EnumRedirects=(OldName="/Script/MeshPartitionEditor.TessellateModifierMethod", NewName="/Script/MeshPartitionEditor.MegaMeshRemeshModifierTessellateMethod")
+ClassRedirects=(OldName="/Script/MeshPartitionEditor.TessellateModifier", NewName="/Script/MeshPartitionEditor.RemeshModifier")