{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Mesh Partition",
	"Description": "Large-scale mesh authoring system through spatial partitioning, non-destructive modifier editing, and platform-adaptive runtime representations.",
	"Category": "Mesh Partition",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "https://dev.epicgames.com/community/learning/knowledge-base/nK7J/unreal-engine-introduction-to-mesh-terrain",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MeshPartition",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshPartitionEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshPartitionCompute",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit"
		},
		{
			"Name": "MeshPartitionModelingToolset",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "MeshPartitionEditorUI",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "GeometryScripting",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		},
		{
			"Name": "HairStrands",
			"Enabled": true
		},
		{
			"Name": "MeshModelingToolset",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "ModelingToolsEditorMode",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "EditorDataStorageFeatures",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		},
		{
			"Name": "VEUV",
			"Enabled": true,
			"TargetAllowList": [
				"Editor"
			]
		}
	]
}
