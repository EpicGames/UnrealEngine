// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

uint2 Resolution;

Texture2D<float> Mask;

RWTexture2D<float4> RWSectionTexture;

[numthreads(8, 8, 1)]
void MainCS(uint2 DTid : SV_DispatchThreadID)
{
	if (any(DTid >= Resolution) || Mask[DTid])
	{
		return;
	}

	const int2 TexPos = int2(DTid);
	
	// From the kernel size derive the search box for candidate texels
	const int KernelSize = 3;
	const int2 SearchMinOffset = max(TexPos - KernelSize, 0);
	const int2 SearchMaxOffset = min(TexPos + KernelSize, int2(Resolution - 1));

	// candidate samples within the kernel radius can contribute
	int BestCandidateDistanceSqr = Square(KernelSize + 1);
	float4 CandidateTexel       = 0.0f;
	int    NumSamples			= 0;

	for (int CoordY = SearchMinOffset.y; CoordY <= SearchMaxOffset.y; CoordY++)
	{
		for (int CoordX = SearchMinOffset.x; CoordX <= SearchMaxOffset.x; CoordX++)
		{
			int DistanceSqr = Square(CoordX - TexPos.x) + Square(CoordY - TexPos.y);
			if (DistanceSqr > BestCandidateDistanceSqr)
			{
				continue;
			}
			
			uint2 Offset = uint2(CoordX, CoordY);
			if (!Mask[Offset])
			{
				continue;
			}

			// reset the candidate sample accumulation if closer candidate found 
			if (DistanceSqr < BestCandidateDistanceSqr)
			{
				BestCandidateDistanceSqr = DistanceSqr;
				NumSamples = 0;
				CandidateTexel = 0;
			}

			// accumulate sample to candidate texel and num samples
			CandidateTexel += RWSectionTexture[Offset];
			NumSamples++;
		}
	}

	if (NumSamples > 0)
	{
		RWSectionTexture[DTid] = CandidateTexel / float(NumSamples);
	}
}

float2 TexelSize;
int4 ResolutionPass;
Texture2D<float4> SectionMipIn;
RWTexture2D<float4> SectionMipOut;
Texture2D<float> MaskMipIn;
RWTexture2D<float> MaskMipOut;
SamplerState MipSampler;

[numthreads(8, 8, 1)]
void PullCS(uint2 DTid : SV_DispatchThreadID)
{
	if (any(DTid >= ResolutionPass.xy))
	{
		return;
	}
	
	int2 DestTexel = DTid.xy;
	int2 SourceTexel = DestTexel * 2;
	float4 Signal = 0;
	float Mask = 0;
	for (int i = 0; i < 2; ++i)
	{
		for (int j = 0; j < 2; ++j)
		{
			float SourceMask = MaskMipIn.Load(int3(SourceTexel + int2(i, j), 0));
			float4 SourceSignal = SectionMipIn.Load(int3(SourceTexel + int2(i, j), 0));
			
			Signal += SourceMask * SourceSignal;
			Mask += SourceMask;
		}
	}
	
	if (Mask > 0)
	{
		Signal = Signal / Mask;
		Mask *= 0.25;
	}
	else
	{
		Signal = 0;
		Mask = 0;
	}
	
	SectionMipOut[DestTexel] = Signal;
	MaskMipOut[DestTexel] = Mask;
}

[numthreads(8, 8, 1)]
void PushCS(uint2 DTid : SV_DispatchThreadID)
{
	if (any(DTid >= ResolutionPass.xy) || MaskMipIn[DTid])
	{
		return;
	}

	float2 coord = (DTid.xy + (0.5)) * TexelSize;
	
	float4 Source = SectionMipIn.SampleLevel(MipSampler, coord, 0);
	
	//SectionMipOut[DTid] = float4(coord, 0, 1);
	SectionMipOut[DTid] = Source;
}

