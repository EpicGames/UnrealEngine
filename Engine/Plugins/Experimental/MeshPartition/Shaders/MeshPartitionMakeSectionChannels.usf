// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Private/Common.ush"

// ----------------------------------------------------------------------------------

Buffer<int>		InMeshIndices;
Buffer<float2>	InMeshUVs;
Buffer<float>	InMeshChannels;
Buffer<int>		InMeshOutlines;
Buffer<int>		InUVElementToVertexID;

uint4  InDrawCall; // NumVertices, NumChannelAttribs, FirstVertIndex
uint   InChannelOffset; // Offset to the first channel element in the Channel buffer

// ----------------------------------------------------------------------------------
// Vertex shaders : 

void DrawUVDomainVS(
	uint VertexIndex : SV_VertexID,
	out float2 SourceUV : TEXCOORD0,
	out float4  OutWeight : TEXCOORD1,
	out float4 OutPosition : SV_POSITION)
{
	uint UVElementID;

	if (InDrawCall.w == 1)
	{
		// Outline (line-list) pass. Each outline entry is (UVElementA, UVElementB, TID),
		// stride 3, so look up the endpoint's UV element directly from the outline buffer.
		uint EdgeIndex = VertexIndex >> 1;
		uint EdgeVertexIndex = VertexIndex & 0x00000001;
		UVElementID = InMeshOutlines[EdgeIndex * 3 + EdgeVertexIndex];
	}
	else
	{
		UVElementID = InMeshIndices[VertexIndex + InDrawCall.z];
	}

	uint VertexID = InUVElementToVertexID[UVElementID];

	SourceUV = InMeshUVs[UVElementID];
	OutWeight = InMeshChannels[InChannelOffset + VertexID];

	// Transform from UV space to ndc
	float2 BaseUV = SourceUV;
	BaseUV *= 2.0;
	OutPosition.x = BaseUV.x - 1.0;
	OutPosition.y = 1.0 - BaseUV.y;
	OutPosition.zw = float2(0.0, 1.0);
}

// ----------------------------------------------------------------------------------
// Pixel shaders : 

void DrawUVDomainPS(
	float2 SourceUV : TEXCOORD0,
	float4 Weights : TEXCOORD1,
	float4 SvPosition : SV_POSITION,
	out float4 OutColor : SV_Target0,
	out float OutMask : SV_Target1)
{
	OutColor = Weights;
	OutMask  = 1.0;
}
