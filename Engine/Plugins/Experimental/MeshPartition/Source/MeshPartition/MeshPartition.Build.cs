// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshPartition : ModuleRules
{
	public MeshPartition(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bAllowUETypesInNamespaces = true;

		PublicIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DeveloperSettings",
				"DynamicMesh",
				"Engine",
				"GeometryCore",
				"MeshConversion",
				"MeshDescription",
				"PhysicsCore", // for collision mesh serialization
				"RenderCore",
				"Renderer",
				"RHI"
			}
			);
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"DataLayerEditor"
				}
			);
		}
	}
}
