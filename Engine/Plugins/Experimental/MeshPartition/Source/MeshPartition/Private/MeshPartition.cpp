// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartition.h"

#include "MeshPartitionComponent.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionSettings.h"
#include "MeshPartitionDependencyInterface.h"

namespace UE::MeshPartition
{
// Sets default values
AMeshPartition::AMeshPartition()
	: MegaMeshComponent(nullptr)
{
	PrimaryActorTick.bCanEverTick = false;

#if WITH_EDITOR
	// AMeshPartition should always be available.
	SetIsSpatiallyLoaded(false);
#endif // WITH_EDITOR

	RootComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"));
	RootComponent->SetMobility(EComponentMobility::Static);

#if WITH_EDITORONLY_DATA
	TransientSectionAttachAnchor = CreateDefaultSubobject<USceneComponent>(TEXT("TransientSectionAttachAnchor"));
	TransientSectionAttachAnchor->SetFlags(RF_Transient);
	TransientSectionAttachAnchor->SetMobility(EComponentMobility::Static);
	TransientSectionAttachAnchor->SetupAttachment(RootComponent);
#endif
}

#if WITH_EDITOR
void AMeshPartition::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (MegaMeshComponent == nullptr)
	{
		return;
	}
	
	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(AMeshPartition, MegaMeshDefinition))
	{
		MegaMeshComponent->OnDefinitionChanged(MegaMeshDefinition);	
	}
}

void AMeshPartition::PostLoad()
{
	Super::PostLoad();
	RootComponent->SetMobility(EComponentMobility::Static);
}

void AMeshPartition::PostActorCreated()
{
	Super::PostActorCreated();
	InitializeDefinition();
}

void AMeshPartition::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	InDependencies += GetActorTransform();
}

#endif // WITH_EDITOR

void AMeshPartition::PostRegisterAllComponents()
{
	Super::PostRegisterAllComponents();

	if (MegaMeshComponent != nullptr)
	{
		MegaMeshComponent->PostRegisterMegaMeshComponents();
	}
}

void AMeshPartition::PostUnregisterAllComponents()
{
	if (MegaMeshComponent != nullptr)
	{
		MegaMeshComponent->PostUnregisterMegaMeshComponents();
	}

	Super::PostUnregisterAllComponents();
}

void AMeshPartition::SetMeshPartitionComponent(UMeshPartitionComponent* InMeshPartitionComponent)
{
	MegaMeshComponent = InMeshPartitionComponent;
	
	if (MegaMeshComponent != nullptr)
	{
		if (!InMeshPartitionComponent->IsRegistered())
		{
			InMeshPartitionComponent->RegisterComponent();
		}

		// Ensure that when assigning a new megameshcomponent, it receives the correct initialization flow.
		// This is usually handled via PostRegisterAllComponents but would not be called when manually assigned.
		InMeshPartitionComponent->PostRegisterMegaMeshComponents();
	}
}

void AMeshPartition::SetMeshPartitionDefinition(UMeshPartitionDefinition* InMeshPartitionDefinition)
{
	MegaMeshDefinition = InMeshPartitionDefinition;

	if (MegaMeshComponent != nullptr)
	{
		MegaMeshComponent->OnDefinitionChanged(MegaMeshDefinition);
	}
}

FBox AMeshPartition::WorldToLocal(const FBox& InWorldBounds) const
{
	return InWorldBounds.InverseTransformBy(GetActorTransform());
}

FBox AMeshPartition::LocalToWorld(const FBox& InLocalBounds) const
{
	return InLocalBounds.TransformBy(GetActorTransform());
}

void AMeshPartition::InitializeDefinition()
{
	if (MegaMeshDefinition == nullptr)
	{
		const MeshPartition::USettings* Settings = GetDefault<MeshPartition::USettings>();

		if (Settings != nullptr)
		{
			TSoftObjectPtr<MeshPartition::UMeshPartitionDefinition> DefaultDefinition = Settings->GetDefaultDefinition();

			if (!DefaultDefinition.IsNull())
			{
				SetMeshPartitionDefinition(DefaultDefinition.LoadSynchronous());
				Modify();
			}
		}
	}
}

} // namespace UE::MeshPartition