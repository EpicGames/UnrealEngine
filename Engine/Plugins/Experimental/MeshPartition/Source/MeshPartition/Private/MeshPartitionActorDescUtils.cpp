// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionActorDescUtils.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

namespace UE::MeshPartition::Utils::ActorDesc
{
#if WITH_EDITOR
	int32 GetPropertyInt32(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName, int32 DefaultValue)
	{
		FName PropertyValue;
		PropertyPairsMap.GetProperty(PropertyName, &PropertyValue);
		if (!PropertyValue.IsNone())
		{
			int32 PropertyNumber = FCString::Atoi(*PropertyValue.ToString());
			return PropertyNumber;
		}
		return DefaultValue;
	}

	int32 GetPropertyInt32(const FWorldPartitionActorDescInstance& InActorDescInstance, const FName PropertyName, int32 DefaultValue)
	{
		FName PropertyValue;
		InActorDescInstance.GetProperty(PropertyName, &PropertyValue);
		if (!PropertyValue.IsNone())
		{
			int32 PropertyNumber = FCString::Atoi(*PropertyValue.ToString());
			return PropertyNumber;
		}
		return DefaultValue;
	}

	void SetPropertyInt32(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName, int32 NewValue)
	{
		FString PropertyValue = FString::Printf(TEXT("%d"), NewValue);
		PropertyPairsMap.AddProperty(PropertyName, FName(PropertyValue));
	}

	int32 IncrementPropertyInt32(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName)
	{
		int32 Count = GetPropertyInt32(PropertyPairsMap, PropertyName, 0);
		SetPropertyInt32(PropertyPairsMap, PropertyName, Count + 1);
		return Count + 1;
	}

	void SetPropertyGUID(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName, const FGuid& NewValue)
	{
		FString PropertyValue = NewValue.ToString();
		PropertyPairsMap.AddProperty(PropertyName, FName(PropertyValue));
	}
#endif // WITH_EDITOR
}