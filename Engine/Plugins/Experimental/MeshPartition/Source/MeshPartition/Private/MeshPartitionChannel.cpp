// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionChannel.h"
#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionDefinition.h"
#include "Components/PrimitiveComponent.h"

namespace UE::MeshPartition
{
namespace
{
void SetChannelTableOnPrimitiveComponent(UPrimitiveComponent* InPrimitiveComponent, TConstArrayView<uint8> InChannelTable)
{
	// Channel table is the channel index to texture slice indirection for the section
	// unused channels (in this section) are marked -1
	// used channels have a valid index matching a slice of the section's channel texture array

	// Packed Channel Table is assigned to all the primitive components of the Section as a custom primitive data
	// used by the Material
	
	// The packing of the data depends on the material logic to unpack the same data the 2 MUST MATCH! 
	// Unpacking logic is defined in "MeshPartitionRenderingUtils.cpp"

	// Packing logic of the channel table happens here 

	TArray<uint32> PackedChannelTable;
	PackedChannelTable.Init(-1, FChannelPacking::TableNumWords);
	
	switch (FChannelPacking::SlotNumBits)
	{
	case 8:
		{
			// 8 bits per slot => just mem copy the source data to the uniform vector
			uint64 ChannelTableSize = FMath::Min((uint64)InChannelTable.Num(), (uint64) FChannelPacking::TableNumWords * sizeof(uint32));
			FMemory::Memcpy((void*)PackedChannelTable.GetData(), InChannelTable.GetData(), ChannelTableSize);
		} 
		break;
	case 7:
	case 6:
	case 5:
	case 4:
	case 3:
	case 2:
	case 1:
		{
			uint32 SlotBitsMask = (1 << FChannelPacking::SlotNumBits) - 1;
			int32 SourceSlotId = 0;
			for (uint32 WordId = 0; (WordId < FChannelPacking::TableNumWords) && (SourceSlotId < InChannelTable.Num()); ++WordId)
			{
				// Initialize the Destination word with all bits to 1
				uint32 DestWord = 0xFFFFFFFF;

				for (uint32 SlotId = 0; (SlotId < FChannelPacking::WordNumSlots) && (SourceSlotId < InChannelTable.Num()); ++SlotId)
				{
					// Mask out from the initial 0xFF fvalue at the location of this slot in the word
					DestWord &= ~((0xFF & SlotBitsMask) << (SlotId * FChannelPacking::SlotNumBits));
					// Next write this slot value
					DestWord |= (InChannelTable[SourceSlotId] & SlotBitsMask) << (SlotId * FChannelPacking::SlotNumBits);

					// move on to next source slot
					++SourceSlotId;
				}

				// Set this word in the packed table
				PackedChannelTable[WordId] = DestWord;
			}
		}
		break;
	default:
		// default is no op, the CHANNEL TABLE is empty
		break;
	}

	InPrimitiveComponent->SetDefaultCustomPrimitiveDataFloatArray(UE::MeshPartition::PrimitiveDataIndex, MakeConstArrayView((float*)PackedChannelTable.GetData(), PackedChannelTable.Num()));
}
}

TArray<FName> FChannelMap::GetChannels() const
{
	TArray<FName> Channels;
	for (const MeshPartition::FChannelDesc& Channel : ChannelDescs)
	{
		Channels.Add(Channel.Name);
	}
	return Channels;
}

int32 FChannelMap::FindChannel(FName InChannelName) const
{
	return ChannelDescs.IndexOfByKey(InChannelName);
}

void operator += (MeshPartition::IDependencyInterface& Dependencies, const MeshPartition::FChannelMap& ChannelMap)
{
	for (const MeshPartition::FChannelDesc& Channel : ChannelMap.GetChannelDescs())
	{
		Dependencies += Channel.Name;
	}
}

bool FNameWrapper::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	if (Tag.Type == NAME_NameProperty)
	{
		Slot << Name;
		return true;
	}

	return false;
}

void FChannelPacking::SetCustomPrimitiveData(UPrimitiveComponent* InPrimitiveComponent, TConstArrayView<uint8> InChannelTable, const FVector2f& InChannelTexcoordDesc)
{
	if (InPrimitiveComponent)
	{
		SetChannelTableOnPrimitiveComponent(InPrimitiveComponent, InChannelTable);

		const int32 DataIndex = UE::MeshPartition::PrimitiveDataIndex + 4;
		InPrimitiveComponent->SetDefaultCustomPrimitiveDataVector2f(DataIndex, InChannelTexcoordDesc);
	}
}


} // namespace UE::MeshPartition
