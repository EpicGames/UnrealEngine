// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionCollisionComponent.h"

#include "DynamicMeshBuilder.h"
#include "Engine/Engine.h" // GEngine
#include "Engine/CollisionProfile.h"
#include "LocalVertexFactory.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Materials/MaterialInstanceDynamic.h" // for 'dummy' materials to hold physical materials
#include "MeshPartitionModule.h"
#include "MeshElementCollector.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "SceneInterface.h"
#include "SceneManagement.h"
#include "SceneView.h"

static TAutoConsoleVariable<bool> CVarMeshPartitionAsyncPhysics(TEXT("MeshPartition.Collision.AsyncPhysicsBuild"),
	true,
	TEXT("Allow async generation of physics data for mesh partition collision component."));

static TAutoConsoleVariable<bool> CVarMeshPartitionCollisionMeshShowPhysicalMaterial(
	TEXT("MeshPartition.Collision.ShowPhysicalMaterial"),
	false,
	TEXT("When enabled, vertex colors of the collision mesh are chosen based on the physical material"),
	ECVF_RenderThreadSafe);

namespace UE::MeshPartition
{



FMeshPartitionCollisionData::FMeshPartitionCollisionData()
{
}

UMeshPartitionCollisionComponent::UMeshPartitionCollisionComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	CollisionData = MakeShared<FMeshPartitionCollisionData>();
#endif

	bHiddenInGame = true;
	bCastDynamicShadow = false;
	bExcludeFromLightAttachmentGroup = true;

	SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);

	MarkRenderStateDirty();
}

FBoxSphereBounds UMeshPartitionCollisionComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	if (!LocalBounds.IsValid)
	{
		// If bbox is empty, set a very small bbox to avoid log spam/etc in other engine systems.
		// The check used is generally IsNearlyZero(), which defaults to KINDA_SMALL_NUMBER, so set 
		// a slightly larger box here to be above that threshold
		constexpr double Extent = (double)(KINDA_SMALL_NUMBER + SMALL_NUMBER);
		return FBoxSphereBounds(FBox(FVector(-Extent), FVector(Extent)));
	}
	return FBoxSphereBounds(LocalBounds.TransformBy(LocalToWorld));
}

void UMeshPartitionCollisionComponent::OnRegister()
{
	Super::OnRegister();
}

void UMeshPartitionCollisionComponent::OnUnregister()
{
	Super::OnUnregister();
#if WITH_EDITORONLY_DATA
	AbortPendingPhysicsCook();
#endif
}

#if WITH_EDITORONLY_DATA
void UMeshPartitionCollisionComponent::RebuildIfNeeded(bool bAllowAsyncBuild)
{
	if (bNeedsRebuild)
	{
		RebuildPhysicsData(bAllowAsyncBuild);
	}
}

void UMeshPartitionCollisionComponent::UpdateLocalBounds()
{
	LocalBounds.Init();
	if (const FTriMeshCollisionData* Mesh = GetCollisionMesh())
	{
		for (FVector3f Vertex : Mesh->Vertices)
		{
			LocalBounds += (FVector3d)Vertex;
		}
	}
}

void UMeshPartitionCollisionComponent::SetCollisionData(TSharedPtr<FMeshPartitionCollisionData> InCollisionData, FName CollisionProfileName, bool bRebuild)
{
	CollisionData = InCollisionData;
	bNeedsRebuild = true;

	UpdateLocalBounds();
	UpdateBounds();

	if (bRebuild)
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);
		Timer.Start();

		RebuildPhysicsData();

		Timer.Stop();
		UE_LOGF(LogMegaMesh, Verbose, "UMeshPartitionCollisionComponent::SetMeshCollision created physics state in %.2f seconds", Time);
	}

	if (CollisionData.IsValid())
	{
		// SetCollisionProfileName logic w/out the physics rebuild, since we handle rebuild separately (above or deferred)
		if (CollisionProfileName != BodyInstance.GetCollisionProfileName())
		{
			ECollisionEnabled::Type OldCollisionEnabled = BodyInstance.GetCollisionEnabled();
			BodyInstance.SetCollisionProfileName(CollisionProfileName);

			ECollisionEnabled::Type NewCollisionEnabled = BodyInstance.GetCollisionEnabled();

			OnComponentCollisionSettingsChanged(true);
		}
	}
}

void UMeshPartitionCollisionComponent::RebuildPhysicalMaterialData()
{
	PhysicsOnlyMaterials.Reset();
	if (!CollisionData)
	{
		return;
	}

	for (int32 Idx = 0; Idx < CollisionData->PhysicalMaterials.Num(); ++Idx)
	{
		// Note we intentionally create the material instance with a null parent because we should never actually use this as a render material
		// it is solely a workaround to pass physical materials to the physics bodyinstance
		UMaterialInstanceDynamic* MID = UMaterialInstanceDynamic::Create(nullptr, this);
		MID->PhysMaterial = CollisionData->PhysicalMaterials[Idx];
		MID->bOverridePhysMaterial = true;
		PhysicsOnlyMaterials.Add(MID);
	}

	// update the body instance physical materials
	FBodyInstance* BodyInst = GetBodyInstance();
	if (BodyInst && BodyInst->IsValidBodyInstance())
	{
		BodyInst->UpdatePhysicalMaterials();
	}
}

void UMeshPartitionCollisionComponent::RebuildPhysicsData(bool bAllowAsyncBuild)
{
	bNeedsRebuild = false;

	MarkRenderStateDirty();
	RebuildPhysicalMaterialData();

	if (AsyncBodySetup)
	{
		AsyncBodySetup->AbortPhysicsMeshAsyncCreation();
		AsyncBodySetup = nullptr;
	}

	if (!bAllowAsyncBuild || !CVarMeshPartitionAsyncPhysics.GetValueOnGameThread() ||
		!CollisionData.IsValid() || !CollisionData->Mesh.IsSet() || CollisionData->Mesh->Indices.IsEmpty())
	{
		MeshBodySetup = nullptr;
		// Generate synchronously
		RecreatePhysicsState();
		return;
	}

	AsyncBodySetup = CreateBodySetupHelper();
	UE_LOGF(LogMegaMesh, Verbose, "UMeshPartitionCollisionComponent::RebuildPhysicsData enqueuing async physics generation.  guid=%ls", *AsyncBodySetup->BodySetupGuid.ToString());

	AsyncBodySetup->CreatePhysicsMeshesAsync(FOnAsyncPhysicsCookFinished::CreateUObject(this, &UMeshPartitionCollisionComponent::FinishPhysicsAsyncCook, AsyncBodySetup.Get()));
}

void UMeshPartitionCollisionComponent::FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup)
{
	if (FinishedBodySetup != AsyncBodySetup)
	{
		UE_LOGF(LogMegaMesh, Display, "UMeshPartitionCollisionComponent::FinishPhysicsAsyncCook discarding cancelled UBodySetup generation.  guid=%ls", *FinishedBodySetup->BodySetupGuid.ToString());
		return;
	}

	// Note: bSuccess (confusingly) reports whether it had to do an async cook, not whether the cook failed
	// It will be false if the physics has no collision (so nothing to cook), but in that case we still apply the empty-collision body setup

	// Received the UBodySetup containing the cooked physics data.  Apply it.
	UE_LOGF(LogMegaMesh, Verbose, "UMeshPartitionCollisionComponent::FinishPhysicsAsyncCook applying new UBodySetup.  guid=%ls", *FinishedBodySetup->BodySetupGuid.ToString());
	MeshBodySetup = FinishedBodySetup;
	AsyncBodySetup = nullptr;

	RecreatePhysicsState();
}

void UMeshPartitionCollisionComponent::AbortPendingPhysicsCook()
{
	if (AsyncBodySetup)
	{
		UE_LOGF(LogMegaMesh, Display, "UMeshPartitionCollisionComponent::AbortPendingPhysicsCook aborting stale UBodySetup generation.  guid=%ls", *AsyncBodySetup->BodySetupGuid.ToString());
		AsyncBodySetup->AbortPhysicsMeshAsyncCreation();
		AsyncBodySetup = nullptr;
	}
}

UBodySetup* UMeshPartitionCollisionComponent::CreateBodySetupHelper()
{
	UBodySetup* NewBodySetup = NewObject<UBodySetup>(this, NAME_None, RF_NoFlags);
	NewBodySetup->BodySetupGuid = FGuid::NewGuid();

	NewBodySetup->bGenerateMirroredCollision = false;
	NewBodySetup->bDoubleSidedGeometry = true;
	NewBodySetup->CollisionTraceFlag = CTF_UseComplexAsSimple;

	NewBodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	NewBodySetup->bSupportUVsAndFaceRemap = false;
	NewBodySetup->bSupportFaceRemapOnMeshBVH = false;
	NewBodySetup->bSupportVertexRemap = false;

	return NewBodySetup;
}
#endif // #if WITH_EDITORONLY_DATA

UBodySetup* UMeshPartitionCollisionComponent::GetBodySetup()
{
#if WITH_EDITORONLY_DATA
	if (!MeshBodySetup)
	{
		MeshBodySetup = CreateBodySetupHelper();
	}
#endif

	return MeshBodySetup;
}

bool UMeshPartitionCollisionComponent::GetPhysicsTriMeshData(struct FTriMeshCollisionData* OutMeshCollisionData, bool InUseAllTriData)
{
#if WITH_EDITORONLY_DATA
	if (const FTriMeshCollisionData* Mesh = GetCollisionMesh())
	{
		if (ensure(OutMeshCollisionData))
		{
			*OutMeshCollisionData = *Mesh;
			return true;
		}
	}
#else
	UE_LOGF(LogMegaMesh, Warning, "Mesh Partition Collision Component's complex collision mesh not available at runtime.");
#endif
	return false; 
}

bool UMeshPartitionCollisionComponent::ContainsPhysicsTriMeshData(bool /*InUseAllTriData*/) const
{
#if WITH_EDITORONLY_DATA
	const FTriMeshCollisionData* Mesh = GetCollisionMesh();
	return Mesh && !Mesh->Indices.IsEmpty();
#else
	UE_LOGF(LogMegaMesh, Warning, "Mesh Partition Collision Component's complex collision mesh not available at runtime.");
	return false;
#endif
}

bool UMeshPartitionCollisionComponent::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool /*InUseAllTriData*/) const
{
#if WITH_EDITORONLY_DATA
	if (const FTriMeshCollisionData* Mesh = GetCollisionMesh())
	{
		OutTriMeshEstimates.VerticeCount = Mesh->Vertices.Num();
	}
	else
#endif
	{
		OutTriMeshEstimates.VerticeCount = 0;
	}
	return true;
}

void UMeshPartitionCollisionComponent::GetMeshId(FString& OutMeshId)
{
	// This is the opportunity to mark the physics inputs with an ID for ddc to cache the physics body.
}


void UMeshPartitionCollisionComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
#if WITH_EDITORONLY_DATA
	if (!Ar.IsFilterEditorOnly())
	{
		bool bHasCollisionData = CollisionData.IsValid();
		Ar << bHasCollisionData;
		if (bHasCollisionData)
		{
			if (Ar.IsLoading())
			{
				CollisionData = MakeShared<FMeshPartitionCollisionData>();
			}

			bool bHasMesh = CollisionData->Mesh.IsSet();
			Ar << bHasMesh;
			if (bHasMesh)
			{
				if (Ar.IsLoading())
				{
					CollisionData->Mesh.Emplace();
				}
				Ar << CollisionData->Mesh.GetValue();
			}
		}
	}
#endif
}

#if UE_ENABLE_DEBUG_DRAWING && WITH_EDITORONLY_DATA
/** Represents a UMeshPartitionCollisionComponent to the scene manager. */
class FDrawMeshPartitionCollisionSceneProxy final : public FPrimitiveSceneProxy
{
public:
	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

	FDrawMeshPartitionCollisionSceneProxy(const UMeshPartitionCollisionComponent* InComponent)
		: FPrimitiveSceneProxy(InComponent)
		, CollisionData(InComponent->GetMeshCollisionData())
		, VertexFactory(GetScene().GetFeatureLevel(), "FDrawMeshPartitionCollisionVertexFactory")
	{
		bWillEverBeLit = false;

		if (HasMeshDataToRender())
		{
			InitMaterialDebugColors(InComponent);

			InitResources(*CollisionData->Mesh);


			ENQUEUE_RENDER_COMMAND(FDrawMeshPartitionCollisionSceneProxyInitialize)(
				[this](FRHICommandListImmediate& RHICmdList)
				{
					PositionVertexBuffer.InitResource(RHICmdList);
					StaticMeshVertexBuffer.InitResource(RHICmdList);
					ColorVertexBuffer.InitResource(RHICmdList);

					FLocalVertexFactory::FDataType Data;
					PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
					StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
					StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&VertexFactory, Data);
					ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);

					VertexFactory.SetData(RHICmdList, Data);

					VertexFactory.InitResource(RHICmdList);


					PositionVertexBuffer.InitResource(RHICmdList);
					StaticMeshVertexBuffer.InitResource(RHICmdList);
					ColorVertexBuffer.InitResource(RHICmdList);
					IndexBuffer.InitResource(RHICmdList);
				});
		}
	}

	~FDrawMeshPartitionCollisionSceneProxy()
	{
		VertexFactory.ReleaseResource();
		PositionVertexBuffer.ReleaseResource();
		StaticMeshVertexBuffer.ReleaseResource();
		ColorVertexBuffer.ReleaseResource();
		IndexBuffer.ReleaseResource();
	}

	void InitMaterialDebugColors(const UMeshPartitionCollisionComponent* InComponent)
	{
		PhysicsMaterialColors.Init(FColor::Black, InComponent->PhysicsOnlyMaterials.Num());
		for (int32 MatIdx = 0; MatIdx < InComponent->PhysicsOnlyMaterials.Num(); ++MatIdx)
		{
			if (!InComponent->PhysicsOnlyMaterials[MatIdx])
			{
				continue;
			}
			if (UPhysicalMaterial* PhysMat = InComponent->PhysicsOnlyMaterials[MatIdx]->GetPhysicalMaterial())
			{
				PhysicsMaterialColors[MatIdx] = PhysMat->DebugColor.ToFColor(/*bSRGB = */false);
			}
		}
	}

	void InitResources(const FTriMeshCollisionData& CollisionMesh)
	{
		TArray<uint32>& Indices = IndexBuffer.Indices;

		bool bPerTriColors = !CollisionMesh.MaterialIndices.IsEmpty() && PhysicsMaterialColors.Num() > 1;

		Tasks::FTask BuildIndexBuffer = Tasks::Launch(TEXT("BuildIndexBuffer"),
			[this, &CollisionMesh, &Indices, bPerTriColors]()
			{
				Indices.SetNumUninitialized(CollisionMesh.Indices.Num() * 3);
				if (bPerTriColors)
				{
					ParallelFor(CollisionMesh.Indices.Num() * 3,
						[&Indices](int32 Idx)
						{
							Indices[Idx] = Idx;
						});
				}
				else
				{
					ParallelFor(CollisionMesh.Indices.Num(),
						[&Indices, &CollisionMesh](int32 TID)
						{
							Indices[TID * 3 + 0] = CollisionMesh.Indices[TID].v0;
							Indices[TID * 3 + 1] = CollisionMesh.Indices[TID].v1;
							Indices[TID * 3 + 2] = CollisionMesh.Indices[TID].v2;
						});
				}
			});

		Tasks::FTask BuildVertexBuffers = Tasks::Launch(TEXT("BuildVertexBuffers"),
			[this, &CollisionMesh, bPerTriColors]()
			{
				if (bPerTriColors)
				{
					// Separate vertices per triangle, to support per-triangle colors
					PositionVertexBuffer.Init(CollisionMesh.Indices.Num() * 3);
					StaticMeshVertexBuffer.Init(CollisionMesh.Indices.Num() * 3, 1/*num texture coords; cannot be zero*/, false/*cpu access*/);
					ColorVertexBuffer.Init(CollisionMesh.Indices.Num() * 3);

					ParallelFor(CollisionMesh.Indices.Num(),
						[this, &CollisionMesh](int32 TID)
						{
							FTriIndices Tri = CollisionMesh.Indices[TID];
							int32 Verts[3]{ Tri.v0, Tri.v1, Tri.v2 };
							uint16 MatIdx = CollisionMesh.MaterialIndices.IsEmpty() ? 0 : CollisionMesh.MaterialIndices[TID];
							FColor UseColor = PhysicsMaterialColors.IsValidIndex(MatIdx) ? PhysicsMaterialColors[MatIdx] : FColor::Black;
							for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
							{
								int32 VID = Verts[SubIdx];
								int32 OutVID = TID * 3 + SubIdx;
								PositionVertexBuffer.VertexPosition(OutVID) = CollisionMesh.Vertices[VID];
								ColorVertexBuffer.VertexColor(OutVID) = UseColor;
								// Note: Dummy values currently set for tangents/UVs, since we don't need them for collision view
								StaticMeshVertexBuffer.SetVertexTangents(OutVID, FVector3f::XAxisVector, FVector3f::YAxisVector, FVector3f::ZAxisVector);
								StaticMeshVertexBuffer.SetVertexUV(OutVID, 0, FVector2f(0.f));
							}
						});
				}
				else
				{
					FColor UseColor = PhysicsMaterialColors.IsEmpty() ? FColor::Black : PhysicsMaterialColors[0];
					PositionVertexBuffer.Init(CollisionMesh.Vertices.Num());
					StaticMeshVertexBuffer.Init(CollisionMesh.Vertices.Num(), 1/*num texture coords; cannot be zero*/, false/*cpu access*/);
					ColorVertexBuffer.Init(CollisionMesh.Vertices.Num());

					ParallelFor(CollisionMesh.Vertices.Num(),
						[this, &CollisionMesh, UseColor](int32 VID)
						{
							PositionVertexBuffer.VertexPosition(VID) = CollisionMesh.Vertices[VID];
							ColorVertexBuffer.VertexColor(VID) = UseColor;
							// Note: Dummy values currently set for tangents/UVs, since we don't need them for collision view
							StaticMeshVertexBuffer.SetVertexTangents(VID, FVector3f::XAxisVector, FVector3f::YAxisVector, FVector3f::ZAxisVector);
							StaticMeshVertexBuffer.SetVertexUV(VID, 0, FVector2f(0.f));
						});
				}
			});

		Tasks::Wait(MakeArrayView({ BuildIndexBuffer, BuildVertexBuffers }));
	}

	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
	{
		if (IndexBuffer.Indices.IsEmpty())
		{
			return;
		}

		// Note: Colors from StaticMeshSceneProxy.cpp's collision rendering code
		FColor SimpleCollisionColor = FColor(157, 149, 223, 255);
		FColor ComplexCollisionColor = FColor(0, 255, 255, 255);

		for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
		{
			if (VisibilityMap & (1 << ViewIndex))
			{
				const FSceneView* View = Views[ViewIndex];

				if (AllowDebugViewmodes())
				{
					UMaterial* MaterialToUse = GEngine->WireframeMaterial;
					// We use the simple collision color b/c we currently assume "simple as complex collision" is used for this component
					FLinearColor DrawCollisionColor = SimpleCollisionColor;
					if (CVarMeshPartitionCollisionMeshShowPhysicalMaterial.GetValueOnRenderThread() && GEngine->VertexColorViewModeMaterial_ColorOnly)
					{
						MaterialToUse = GEngine->VertexColorViewModeMaterial_ColorOnly;
						DrawCollisionColor = FLinearColor::White;
					}

					const bool bDrawWireframe = true;

					// Create colored proxy
					FColoredMaterialRenderProxy* CollisionMaterialInstance = new FColoredMaterialRenderProxy(MaterialToUse->GetRenderProxy(), DrawCollisionColor);
					Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);

					// Draw the mesh with collision materials
					{
						FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
						FPrimitiveUniformShaderParametersBuilder Builder;
						BuildUniformShaderParameters(Builder);
						DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);
						FMeshBatch& Mesh = Collector.AllocateMesh();
						FMeshBatchElement& BatchElement = Mesh.Elements[0];
						BatchElement.IndexBuffer = &IndexBuffer;
						Mesh.bWireframe = bDrawWireframe;
						Mesh.bDisableBackfaceCulling = true;
						Mesh.VertexFactory = &VertexFactory;
						Mesh.MaterialRenderProxy = CollisionMaterialInstance;

						BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

						BatchElement.FirstIndex = 0;
						BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
						BatchElement.MinVertexIndex = 0;
						BatchElement.MaxVertexIndex = PositionVertexBuffer.GetNumVertices() - 1;
						Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
						Mesh.Type = PT_TriangleList;
						Mesh.DepthPriorityGroup = SDPG_World;
						Mesh.bCanApplyViewModeOverrides = true;

						Collector.AddMesh(ViewIndex, Mesh);
					}
				}
			}
		}
	}

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		// TODO: also handle EngineShowFlags.CollisionPawn / EngineShowFlags.CollisionVisibility rendering
		bool bShowForCollision = View->Family->EngineShowFlags.Collision && HasMeshDataToRender();

		FPrimitiveViewRelevance Result;
		Result.bDrawRelevance = bShowForCollision;
		Result.bDynamicRelevance = true;
		Result.bShadowRelevance = false;
		Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
		return Result;
	}
	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }
	uint32 GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }

private:

	bool HasMeshDataToRender() const
	{
		return CollisionData.IsValid() && CollisionData->Mesh.IsSet() && !CollisionData->Mesh->Indices.IsEmpty();
	}

	TSharedPtr<const FMeshPartitionCollisionData> CollisionData;

	TArray<FColor> PhysicsMaterialColors;

	FLocalVertexFactory VertexFactory;
	// TODO: Use a different vertex factory and avoid the need for StaticMeshVertexBuffer and ColorVertexBuffer
	FStaticMeshVertexBuffer StaticMeshVertexBuffer;
	FColorVertexBuffer ColorVertexBuffer;

	FPositionVertexBuffer PositionVertexBuffer;
	FDynamicMeshIndexBuffer32 IndexBuffer;
};
#endif // UE_ENABLE_DEBUG_DRAWING && WITH_EDITORONLY_DATA


FPrimitiveSceneProxy* UMeshPartitionCollisionComponent::CreateSceneProxy()
{
#if UE_ENABLE_DEBUG_DRAWING && WITH_EDITORONLY_DATA
	// if debug drawing is enabled (e.g. in editor or debug modes), make a simple scene proxy to draw the collision mesh
	return new FDrawMeshPartitionCollisionSceneProxy(this);
#else
	return Super::CreateSceneProxy();
#endif
}



UMaterialInterface* UMeshPartitionCollisionComponent::GetMaterial(int32 Index) const
{
	return PhysicsOnlyMaterials.IsValidIndex(Index) ? PhysicsOnlyMaterials[Index] : nullptr;
}

int32 UMeshPartitionCollisionComponent::GetNumMaterials() const
{
	return PhysicsOnlyMaterials.Num();
}

void UMeshPartitionCollisionComponent::SetMaterial(int32 Index, UMaterialInterface* InMaterial)
{
	UE_LOGF(LogMegaMesh, Warning, "Mesh Partition Collision Components do not support SetMaterial");
}

void UMeshPartitionCollisionComponent::SetMaterialByName(FName MaterialSlotName, class UMaterialInterface* Material)
{
	UE_LOGF(LogMegaMesh, Warning, "Mesh Partition Collision Components do not support SetMaterialByName");
}


UMaterialInstanceDynamic* UMeshPartitionCollisionComponent::CreateAndSetMaterialInstanceDynamic(int32 ElementIndex)
{
	UE_LOGF(LogMegaMesh, Warning, "CreateAndSetMaterialInstanceDynamic on %ls: Mesh Partition Collision Components only use rendering materials to indirectly set physical materials, and should not updated via this method.", *GetPathName());

	return nullptr;
}

UMaterialInstanceDynamic* UMeshPartitionCollisionComponent::CreateAndSetMaterialInstanceDynamicFromMaterial(int32 ElementIndex, class UMaterialInterface* Parent)
{
	UE_LOGF(LogMegaMesh, Warning, "CreateAndSetMaterialInstanceDynamicFromMaterial on %ls: Mesh Partition Collision Components only use rendering materials to indirectly set physical materials, and should not updated via this method.", *GetPathName());

	return nullptr;
}

UMaterialInstanceDynamic* UMeshPartitionCollisionComponent::CreateDynamicMaterialInstance(int32 ElementIndex, class UMaterialInterface* SourceMaterial, FName OptionalName)
{
	UE_LOGF(LogMegaMesh, Warning, "CreateDynamicMaterialInstance on %ls: Mesh Partition Collision Components only use rendering materials to indirectly set physical materials, and should not updated via this method.", *GetPathName());

	return nullptr;
}



} // namespace UE::MeshPartition
