// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPartitionCompiledSection.h"

#include "MaterialCachedData.h"
#include "Algo/AnyOf.h"
#include "Components/StaticMeshComponent.h"
#include "Interfaces/ITargetPlatform.h"
#include "Engine/StaticMesh.h"
#include "MeshPartition.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionModule.h"
#include "MeshPartitionCompilerInterface.h"
#include "MeshPartitionCollisionComponent.h"
#include "Engine/Texture2DArray.h"
#include "Materials/MaterialInstanceConstant.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "MeshPartitionCompiledSectionActorDesc.h"
#include "MeshPartitionActorDescUtils.h"
#include "MeshPartitionMaterialCacheCommon.h"
#include "MeshPartitionStaticMeshComponent.h"
#include "MeshPartitionStaticMeshDescriptor.h"
#include "Engine/CollisionProfile.h"
#include "RenderUtils.h"
#include "Engine/World.h"
#include "MaterialCache/MaterialCache.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"

namespace UE::MeshPartition
{
// Actor Desc Properties for Compiled Sections
const FName MegaMeshCompiledSectionProperties::MegaMeshCompiledSectionVersion = TEXT("MegaMeshCompiledSectionVersion");
const int32 MegaMeshCompiledSectionProperties::CurrentVersion = 2;
const FName MegaMeshCompiledSectionProperties::CurrentVersionFName = TEXT("2");

ACompiledSection::ACompiledSection()
	: Parent(nullptr)
{
	PrimaryActorTick.bCanEverTick = false;

	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"));
	SceneComponent->SetMobility(EComponentMobility::Static);
	RootComponent = SceneComponent;

	FarFieldMeshComponent = CreateDefaultSubobject<MeshPartition::UMeshPartitionStaticMeshComponent>(TEXT("FarFieldMeshComponent"));
	FarFieldMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	FarFieldMeshComponent->SetupAttachment(RootComponent);
	FarFieldMeshComponent->SetMobility(EComponentMobility::Static);
	FarFieldMeshComponent->SetGenerateOverlapEvents(false);
	FarFieldMeshComponent->SetHiddenInGame(true);
	FarFieldMeshComponent->SetVisibility(false);
	FarFieldMeshComponent->bAffectIndirectLightingWhileHidden = true;
	FarFieldMeshComponent->CastShadow = false;
	FarFieldMeshComponent->bAllowCullDistanceVolume = false;
	FarFieldMeshComponent->bNeverDistanceCull = true;
	FarFieldMeshComponent->bRayTracingFarField = true;
}

void ACompiledSection::Serialize(FArchive& Ar)
{
	/* We do not want the RootComponent to keep a TObjectPtr referencing the parent's component when serialized
	 * as both actor will end-up with the same lifecycle in World Partition. */
	if (Ar.IsSaving())
	{
		if (Ar.IsPersistent() && !Ar.IsTransacting())
		{
			// Detachment must occur here while keeping the world transform.
			// This ensures that when actor descriptors/runtime streaming bounds are computed for this actor,
			// they are generated correctly.
			// If this was using KeepRelative, the root component transform would be the transform relative to the parent actor.
			// NOTE: doing this implicitly bakes the transform of the parent MeshPartition actor into the transform of this section.
			// Thus, transform changes of the parent will invalidate this CS and will require a rebuild.
			DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
		}
	}

	Super::Serialize(Ar);
}

bool ACompiledSection::NeedsLoadForTargetPlatform(const class ITargetPlatform* TargetPlatform) const
{
#if WITH_EDITOR
	return ShouldBeLoadedForPlatform(TargetPlatform);
#else
	return true;
#endif // WITH_EDITOR
}

#if WITH_EDITOR
bool ACompiledSection::ShouldBeLoadedForPlatform(const class ITargetPlatform* TargetPlatform) const
{
	const UMeshPartitionDefinition* MegaMeshDefinition = this->BuildInfo.GetMegaMeshDefinition();
	if (MegaMeshDefinition == nullptr)
	{
		UE_LOGF(LogMegaMesh, Warning, "No MegaMesh Definition attached to MegaMesh Compiled Section '%ls' -- using default behavior", *GetName());
		return (this->BuildInfo.BuildVariantName == NAME_Default);		// absent any definition, we only build and load a 'default' variant
	}

	// get the list of build variants we want for our current platform
	TArray<FName> BuildVariantNames = MegaMeshDefinition->GetCompiledSectionBuildVariantNamesForPlatform(TargetPlatform);
	const bool bSectionBelongsToTargetPlatformVariant = Algo::AnyOf(BuildVariantNames, [BuildInfoName = this->BuildInfo.BuildVariantName](const FName& VariantName)
	{
		return BuildInfoName == VariantName;
	});

	return bSectionBelongsToTargetPlatformVariant;
}
#endif // WITH_EDITOR

void ACompiledSection::PreRegisterAllComponents()
{
#if WITH_EDITOR
	if (bIsPlaceholder && GetWorld() && GetWorld()->IsPlayInEditor())
	{
		if (MeshPartition::IMeshPartitionCompilerInterface* Compiler = MeshPartition::IMeshPartitionCompilerInterface::Get())
		{
			UE_LOGF(LogMegaMesh, Verbose, "Building placeholder compiled sections for %p %ls", this, *GetActorNameOrLabel());
			Compiler->BuildPlaceholderCompiledSection(this);
		}
		else
		{
			ensureMsgf(Compiler != nullptr, TEXT("Could not access MegaMesh builder to build meshes for PIE placeholder sections"));
		}
	}
#endif // WITH_EDITOR
	Super::PreRegisterAllComponents();
}

void ACompiledSection::UpdateVirtualTextureSettings()
{
	for (TObjectPtr<UMaterialCacheVirtualTexture> MaterialCacheVirtualTexture : MaterialCacheTextures)
	{
		MaterialCacheVirtualTexture->UpdateResource();
	}
	
	// Disable fallback RVT mesh when Nanite is disabled
	if (!IsRunningCommandlet())
	{
		TArray<TObjectPtr<UStaticMesh>> StaticMeshes = GetStaticMeshes();

		const bool bNaniteEnabled = UseNanite(GMaxRHIShaderPlatform);

		if (StaticMeshes.Num() == VirtualTextureFallbackMeshComponents.Num())
		{
			for (int32 Index = 0; Index < StaticMeshes.Num(); ++Index)
			{
				UStaticMesh* StaticMesh = StaticMeshes[Index];
				UStaticMeshComponent* VirtualTextureFallbackMeshComponent = VirtualTextureFallbackMeshComponents[Index];

				const bool bShouldUsePrimaryRVT = !bNaniteEnabled || (StaticMesh && !StaticMesh->HasValidNaniteData());
				if (bShouldUsePrimaryRVT)
				{
					VirtualTextureFallbackMeshComponent->SetVisibility(false);
				}
			}
		}
	}

	for (MeshPartition::UMeshPartitionStaticMeshComponent* MeshComponent : MeshComponents)
	{
		if (MeshComponent == nullptr)
		{
			continue;
		}

		MeshComponent->MarkRenderStateDirty();
	}

	FarFieldMeshComponent->MarkRenderStateDirty();
}

void ACompiledSection::SetupChannelDataOnChildPrimitiveComponents()
{
	// Reassign the custom primitive data parameters for this section used by the material shader to fetch channel texture
	SetChannelData(ChannelTable, ChannelTexcoordDesc);
}

void ACompiledSection::SetMaterialCacheTileCount(const FIntPoint& TileCount)
{
	MaterialCacheTileCount = TileCount;
}

void ACompiledSection::RecreateMaterialCacheTextures()
{
	if (!MaterialInstance)
	{
		return;
	}
	
	if (IsMaterialCacheEnabled(GetWorld()))
	{
		UpdateMaterialCacheTextures(RootComponent, MaterialInstance, MaterialCacheTileCount, MaterialCacheTextures);
		
		// Propagate to relevant components
		for (TObjectPtr<UMeshPartitionStaticMeshComponent> MeshComponent : GetMeshComponents())
		{
			MeshComponent->MaterialCacheTextures = MaterialCacheTextures;
			MeshComponent->MarkRenderStateDirty();
		}
	}
}

void ACompiledSection::PostRegisterAllComponents()
{
	if (Parent.IsValid())
	{
		// Disable reattachment of the compiled section actors to the parent during the cook process.
		// Without this, compiled section actor relative transform will get stomped due to the detachment failing inside of Serialize
		// during the world partition cook process. The attachment parent is not loaded when serializing for cook so there is no way to retrieve
		// the correct world transform, so we end up stuck with a serialized RelativeTransform which is 0.
		if (!IsRunningCookCommandlet())
		{
			AttachToActor(Parent.Get(), FAttachmentTransformRules::KeepWorldTransform);
		}
#if WITH_EDITORONLY_DATA
		bLockLocation = true;
#endif // WITH_EDITORONLY_DATA
	}


	UpdateVirtualTextureSettings();
	
	Super::PostRegisterAllComponents();

	SetupChannelDataOnChildPrimitiveComponents();
}

void ACompiledSection::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();
	
	for (TObjectPtr<UMaterialCacheVirtualTexture> MaterialCacheVirtualTexture : MaterialCacheTextures)
	{
		MaterialCacheVirtualTexture->Unregister();
		MaterialCacheVirtualTexture->ReleaseResource();
	}
}

#if WITH_EDITOR
TUniquePtr<FWorldPartitionActorDesc> ACompiledSection::CreateClassActorDesc() const
{
	return TUniquePtr<FWorldPartitionActorDesc>(new MeshPartition::FCompiledSectionActorDesc());
}

void ACompiledSection::GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const
{
	Super::GetActorDescProperties(PropertyPairsMap);

	if (HasAnyFlags(EObjectFlags::RF_ClassDefaultObject | EObjectFlags::RF_ArchetypeObject))
	{
		return;
	}

	PropertyPairsMap.AddProperty(MegaMeshCompiledSectionProperties::MegaMeshCompiledSectionVersion, MegaMeshCompiledSectionProperties::CurrentVersionFName);
}

void ACompiledSection::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	if (bIsPlaceholder)
	{
		OutRuntimeBounds = OutEditorBounds = PlaceholderStreamingBounds;
	}
	else
	{
		Super::GetStreamingBounds(OutRuntimeBounds, OutEditorBounds);
	}
}

TArray<UActorComponent*> ACompiledSection::GetHLODRelevantComponents() const
{
	// If we have a far field component with a valid mesh, use this as our sole source of HLODs
	if (FarFieldMeshComponent && FarFieldMeshComponent->GetStaticMesh())
	{
		return { FarFieldMeshComponent };
	}

	// Otherwise use all valid mesh components
	TArray<UActorComponent*> HLODRelevantComponents;
	HLODRelevantComponents.Reserve(MeshComponents.Num());
	for (UMeshPartitionStaticMeshComponent* MeshComponent : MeshComponents)
	{
		UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh();
		if (StaticMesh && !StaticMesh->GetBoundingBox().GetExtent().Equals(FVector::ZeroVector))
		{
			HLODRelevantComponents.Add(MeshComponent);
		}
	}
	return HLODRelevantComponents;
}

#endif // WITH_EDITOR

TArray<TObjectPtr<UStaticMesh>> ACompiledSection::GetStaticMeshes() const
{
	TArray<TObjectPtr<UStaticMesh>> StaticMeshes;

	StaticMeshes.SetNum(MeshComponents.Num());

	for (int32 Index = 0; Index < MeshComponents.Num(); ++Index)
	{
		StaticMeshes[Index] = MeshComponents[Index]->GetStaticMesh();
	}

	return StaticMeshes;
}

void ACompiledSection::AddStaticMesh(UStaticMesh* InStaticMesh, const FStaticMeshDescriptor& InDescriptor)
{
	Modify(true);

	MeshPartition::UMeshPartitionStaticMeshComponent* MeshComponent = NewObject<MeshPartition::UMeshPartitionStaticMeshComponent>(this, MakeUniqueObjectName(this, MeshPartition::UMeshPartitionStaticMeshComponent::StaticClass(), TEXT("MeshComponent")));
	MeshComponent->OnComponentCreated();
	AddInstanceComponent(MeshComponent);

	MeshComponent->SetStaticMesh(InStaticMesh);
	MeshComponent->SetupAttachment(RootComponent);
	MeshComponent->SetMobility(EComponentMobility::Static);
	MeshComponent->SetCollisionProfileName(InDescriptor.CollisionProfileName);
	MeshComponent->SetCanEverAffectNavigation(InDescriptor.bCanEverAffectNavigation);
	MeshComponent->MaterialCacheTextures = MaterialCacheTextures;
	
	// TODO: Restore later
	// MeshComponent->MaterialCacheUVRegion = Descriptor.UVRegion;

	// copy channel table to this mesh component
	FChannelPacking::SetCustomPrimitiveData(MeshComponent, ChannelTable, ChannelTexcoordDesc);

	MeshComponent->RegisterComponent();
	MeshComponents.Emplace(MeshComponent);

	UStaticMeshComponent* VirtualTextureFallbackMeshComponent = NewObject<UStaticMeshComponent>(this, MakeUniqueObjectName(this, UStaticMeshComponent::StaticClass(), TEXT("VirtualTextureFallbackMeshComponent")));
	VirtualTextureFallbackMeshComponent->OnComponentCreated();
	AddInstanceComponent(VirtualTextureFallbackMeshComponent);

	VirtualTextureFallbackMeshComponent->SetStaticMesh(InStaticMesh);
	VirtualTextureFallbackMeshComponent->SetupAttachment(RootComponent);
	VirtualTextureFallbackMeshComponent->SetMobility(EComponentMobility::Static);
	VirtualTextureFallbackMeshComponent->SetForceDisableNanite(true);
	VirtualTextureFallbackMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	VirtualTextureFallbackMeshComponent->SetCanEverAffectNavigation(false);
	
	VirtualTextureFallbackMeshComponent->SetRenderInMainPass(false);
	VirtualTextureFallbackMeshComponent->SetRenderInDepthPass(false);
	VirtualTextureFallbackMeshComponent->SetVisibleInRayTracing(false);
	VirtualTextureFallbackMeshComponent->SetCastShadow(false);

	FChannelPacking::SetCustomPrimitiveData(VirtualTextureFallbackMeshComponent, ChannelTable, ChannelTexcoordDesc);

	VirtualTextureFallbackMeshComponent->RegisterComponent();

	VirtualTextureFallbackMeshComponents.Emplace(VirtualTextureFallbackMeshComponent);
}

void ACompiledSection::AddCollisionComponent(UMeshPartitionCollisionComponent* InCollisionComponent)
{
	if (InCollisionComponent == nullptr)
	{
		return;
	}

	//todo(luc.eygasier): is this really necessary? Doesn't seem used
	CollisionComponents.Emplace(InCollisionComponent);
	AddInstanceComponent(InCollisionComponent);
	InCollisionComponent->RegisterComponent();
}

UStaticMesh* ACompiledSection::GetFarFieldMesh() const
{ 
	return FarFieldMeshComponent->GetStaticMesh();
}

void ACompiledSection::SetFarFieldMesh(UStaticMesh* InStaticMesh)
{
	Modify(true);
	FarFieldMeshComponent->SetStaticMesh(InStaticMesh);
}

void ACompiledSection::SetParent(const AMeshPartition* InMegaMesh)
{
	Modify(true);
	DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
	Parent = const_cast<AMeshPartition*>(InMegaMesh);

	if (Parent.IsValid())
	{
		AttachToActor(Parent.Get(), FAttachmentTransformRules::KeepRelativeTransform);
#if WITH_EDITORONLY_DATA
		bLockLocation = true;
#endif // WITH_EDITORONLY_DATA
	}
}

AMeshPartition* ACompiledSection::GetParentMegaMesh() const
{
	return Parent.Get();
}

void ACompiledSection::SetChannelData(TConstArrayView<uint8> InChannelTable, const FVector2f& InTexcoordDesc)
{
	// Channel table is the channel index to texture slice indirection for the section
	// unused channels (in this section) are marked -1
	// used channels have a valid index matching a slice of the section's channel texture array
	ChannelTable = InChannelTable;
	ChannelTexcoordDesc = InTexcoordDesc;

	// Channel Table is assigned to all the primitive components of the Section as a custom primitive data used by the Material
	ForAllPrimitiveComponents([&](UPrimitiveComponent* PrimitiveComponent)
	{
		FChannelPacking::SetCustomPrimitiveData(PrimitiveComponent, ChannelTable, ChannelTexcoordDesc);
		return true;
	});
}

void ACompiledSection::SetChannelTexture(UTexture* InChannelTexture)
{
	ChannelTexture = InChannelTexture;
	if (MaterialInstance)
	{
#if WITH_EDITOR
		MaterialInstance->SetTextureParameterValueEditorOnly(UE::MeshPartition::ChannelTextureParameterName, InChannelTexture);
#endif
	}
}

void ACompiledSection::SetRuntimeVirtualTextures(const TArray<TObjectPtr<URuntimeVirtualTexture>>& InRVTs)
{
	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		MeshComponent->RuntimeVirtualTextures = InRVTs;
	}

	for (UStaticMeshComponent* VirtualTextureFallbackMeshComponent : VirtualTextureFallbackMeshComponents)
	{
		VirtualTextureFallbackMeshComponent->RuntimeVirtualTextures = InRVTs;
	}
}

void ACompiledSection::ForAllPrimitiveComponents(TFunctionRef<bool(UPrimitiveComponent*)> InFunc) const
{
	auto HandleComponent = [InFunc](UPrimitiveComponent* Component)
	{
		if (Component == nullptr)
		{
			return true;
		}

		return InFunc(Component);
	};

	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		if (!HandleComponent(MeshComponent))
		{
			return;
		}
	}

	if (!HandleComponent(FarFieldMeshComponent))
	{
		return;
	}

	for (UStaticMeshComponent* VirtualTextureFallbackMeshComponent : VirtualTextureFallbackMeshComponents)
	{
		if (!HandleComponent(VirtualTextureFallbackMeshComponent))
		{
			return;
		}
	}
}

bool FCompiledSectionBuildInfo::operator== (const FCompiledSectionBuildInfo& Other) const
{
	// compare all members, EXCEPT the mutable transient cache members and the dependency checksums
	return
		(BuildKey == Other.BuildKey) &&
		(BuildVariantName == Other.BuildVariantName) &&
		(MegaMeshDefinitionPath == Other.MegaMeshDefinitionPath) &&
		(BaseModifierPaths == Other.BaseModifierPaths) &&
		(ModifiersHash == Other.ModifiersHash) &&
		(PackageDependencies == Other.PackageDependencies) &&
		(PackageHash == Other.PackageHash) && 
		(ModifierSetHash == Other.ModifierSetHash) &&
		(ClassDependencies == Other.ClassDependencies) &&
		(ClassHash == Other.ClassHash) &&
		(BuildVariantHash == Other.BuildVariantHash) &&
		(MegaMeshGUID == Other.MegaMeshGUID) &&
		(MegaMeshPath == Other.MegaMeshPath) &&
		(GridCellCoord == Other.GridCellCoord);
}

bool FCompiledSectionBuildInfo::SetMegaMeshDefinition(const UMeshPartitionDefinition* Definition)
{
	if (MegaMeshDefinitionPath.TrySetPath(Definition))
	{
		MegaMeshDefinition = Definition;
		return true;
	}
	MegaMeshDefinition = nullptr;
	return false;
}

const UMeshPartitionDefinition* FCompiledSectionBuildInfo::GetMegaMeshDefinition() const
{
	if (MegaMeshDefinition == nullptr)
	{
		if (MegaMeshDefinitionPath.IsValid())
		{
			// Load the object from the asset path
			UObject* Definition = StaticLoadObject(UMeshPartitionDefinition::StaticClass(), nullptr, *MegaMeshDefinitionPath.ToString());
	
			// Check if the object was successfully loaded
			MegaMeshDefinition = (const UMeshPartitionDefinition*) Definition;
		}
		if (MegaMeshDefinition == nullptr)
		{
			MegaMeshDefinition = UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
		}
	}
	return MegaMeshDefinition;
}

void FCompiledSectionBuildInfo::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(MeshPartition::FCustomVersion::GUID);
	const int32 Version = Ar.CustomVer(MeshPartition::FCustomVersion::GUID);

	Ar << BuildKey;
	Ar << BuildVariantName;
	Ar << MegaMeshDefinitionPath;
	Ar << BaseModifierPaths;
	Ar << ModifiersHash;
	Ar << PackageDependencies;
	Ar << PackageHash;
	Ar << ClassHash;
	Ar << BuildVariantHash;
	Ar << MegaMeshGUID;
	Ar << ModifierSetHash;
	Ar << ClassDependencies;
	if (Version >= MeshPartition::FCustomVersion::AddedMeshPartitionPathToBuildInfo)
	{
		Ar << MegaMeshPath;
	}
	if (Version >= MeshPartition::FCustomVersion::AddedChecksumsToBuildInfo)
	{
#if WITH_EDITORONLY_DATA
		if (!Ar.IsFilterEditorOnly())
		{
			Ar << PackageChecksums;
			Ar << ClassChecksums;
		}
#endif // WITH_EDITORONLY_DATA
	}

	if (Version >= MeshPartition::FCustomVersion::AddedGridCellCoordToBuildInfo)
	{
		Ar << GridCellCoord.X;
		Ar << GridCellCoord.Y;
		Ar << GridCellCoord.Z;

		// FGridSettings fields are gated per custom-version here -- pre-version fields are reset
		// explicitly so a reused FCompiledSectionBuildInfo can't leak stale values from an earlier read.
		Ar << GridSettings.CellSize;
		
		if (Version >= MeshPartition::FCustomVersion::Added2DFlagToBuildInfo)
		{
			Ar << GridSettings.bIs2D;
		}
		else
		{
			GridSettings.bIs2D = false;
		}
		
		if (Version >= MeshPartition::FCustomVersion::AddedGridOriginToBuildInfo)
		{
			Ar << GridSettings.WorldOriginOffset;
		}
		else
		{
			GridSettings.WorldOriginOffset = FVector::ZeroVector;
		}
	}
}

bool FCompiledSectionBuildInfo::TargetsSameCompiledSectionAs(const FCompiledSectionBuildInfo& Other) const
{
	// If either side has an invalid grid coordinate (non-grid section), skip the coordinate check
	// so non-wp-aligned sections remain compatible with the wp aligned pipeline.
	const bool bGridCoordMatches = (!HasValidGridCellCoord() || !Other.HasValidGridCellCoord()) || (GridCellCoord == Other.GridCellCoord);

	return (BuildVariantName == Other.BuildVariantName) &&
		(MegaMeshGUID == Other.MegaMeshGUID) &&
		(MegaMeshDefinitionPath == Other.MegaMeshDefinitionPath) &&
		(BaseModifierPaths == Other.BaseModifierPaths) &&
		// If the path is not valid, it's likely just an old compiled section, default to accept
		((!MegaMeshPath.IsValid()) || (MegaMeshPath == Other.MegaMeshPath)) &&
		bGridCoordMatches;
}

#if WITH_EDITOR
bool FCompiledSectionDescriptor::BuildFromActorDescInstance(const FWorldPartitionActorDescInstance& InActorDescInstance, FCompiledSectionDescriptor& OutCompiledSectionDescriptor)
{
	using namespace Utils;
	MeshPartition::FCompiledSectionDescriptor Desc;

	const MeshPartition::FCompiledSectionActorDesc* ActorDesc = MeshPartition::FCompiledSectionActorDesc::GetFromActorDescInstance(InActorDescInstance);
	if (ActorDesc == nullptr)
	{
		// older compiled sections don't have the custom actor desc class, so we can't gather their descriptor
		return false;
	}

	// copy over BuildInfo values
	Desc.Info = ActorDesc->GetBuildInfo();

	Desc.ActorDescGuid = InActorDescInstance.GetGuid();
	check(Desc.ActorDescGuid.IsValid());

	Desc.ActorPath = InActorDescInstance.GetActorSoftPath();

	OutCompiledSectionDescriptor = Desc;
	return true;
}
#endif // WITH_EDITOR
} // namespace UE::MeshPartition