// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionCompiledSectionActorDesc.h"

#if WITH_EDITOR
#include "Landscape.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

namespace UE::MeshPartition
{
const FCompiledSectionActorDesc* FCompiledSectionActorDesc::GetFromActorDescInstance(const FWorldPartitionActorDescInstance& InActorDescInstance)
{
	// first we check version number (older versions do not use MeshPartition::FCompiledSectionActorDesc)
	int32 Version = 0;
	FName VersionName;
	if (InActorDescInstance.GetProperty(MegaMeshCompiledSectionProperties::MegaMeshCompiledSectionVersion, &VersionName))
	{
		Version = FCString::Atoi(*VersionName.ToString());
	}

	const int32 kVersionThatUsesCompiledSectionActorDesc = 2;
	if (Version < kVersionThatUsesCompiledSectionActorDesc)
	{
		return nullptr;
	}

	// we assume the version number indicates we can safely upcast to the compiled section actor desc (no checked dynamic cast is possible on structs)
	return (const FCompiledSectionActorDesc*) InActorDescInstance.GetActorDesc();
}

void FCompiledSectionActorDesc::Init(const AActor* InActor)
{
	FWorldPartitionActorDesc::Init(InActor);

	if (!bIsDefaultActorDesc)
	{
		const MeshPartition::ACompiledSection* CompiledSection = CastChecked<MeshPartition::ACompiledSection>(InActor);
		check(CompiledSection);

		this->BuildInfo = CompiledSection->GetBuildInfo();
	}
}

void FCompiledSectionActorDesc::Serialize(FArchive& Ar)
{
	FWorldPartitionActorDesc::Serialize(Ar);

	if (!bIsDefaultActorDesc)
	{
		BuildInfo.Serialize(Ar);
	}
}

bool FCompiledSectionActorDesc::Equals(const FWorldPartitionActorDesc* Other) const
{
	if (FWorldPartitionActorDesc::Equals(Other))
	{
		const FCompiledSectionActorDesc* CompiledSectionActorDesc = (FCompiledSectionActorDesc*) Other;
		return BuildInfo == CompiledSectionActorDesc->BuildInfo;
	}

	return false;
}

const FGuid& FCompiledSectionActorDesc::GetSceneOutlinerParent() const
{
	const FGuid& ParentGuid = BuildInfo.MegaMeshGUID;
	if (ParentGuid != GetGuid())
	{
		return ParentGuid;
	}
	static FGuid NoParent;
	return NoParent;
}
} // namespace UE::MeshPartition
#endif
