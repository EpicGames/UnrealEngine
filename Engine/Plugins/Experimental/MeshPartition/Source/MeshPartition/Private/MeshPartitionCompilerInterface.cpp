// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "MeshPartitionCompilerInterface.h"

namespace UE::MeshPartition
{
IMeshPartitionCompilerInterface* IMeshPartitionCompilerInterface::RegisteredCompiler = nullptr;

IMeshPartitionCompilerInterface* IMeshPartitionCompilerInterface::Get()
{
	return RegisteredCompiler;
}

void IMeshPartitionCompilerInterface::RegisterMegaMeshCompiler(IMeshPartitionCompilerInterface* Compiler)
{
	ensure(RegisteredCompiler == nullptr);
	RegisteredCompiler = Compiler;
}

void IMeshPartitionCompilerInterface::UnregisterMegaMeshCompiler(IMeshPartitionCompilerInterface* Compiler)
{
	if (ensure(RegisteredCompiler == Compiler))
	{
		RegisteredCompiler = nullptr;
	}
}

IMeshPartitionCompilerInterface::~IMeshPartitionCompilerInterface()
{
	// unregister on destruction, if it hasn't already been
	if (RegisteredCompiler == this)
	{
		UnregisterMegaMeshCompiler(this);
	}
}
} // namespace UE::MeshPartition
