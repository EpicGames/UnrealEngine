// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionComponent.h"

namespace UE::MeshPartition
{
// Sets default values for this component's properties
UMeshPartitionComponent::UMeshPartitionComponent()
{
	PrimaryComponentTick.bCanEverTick = false;
}
} // namespace UE::MeshPartition
