// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionDataLayerContainer.h"
#include "MeshPartitionDefinition.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "UObject/Package.h"
#include "EngineUtils.h"
#include "MeshPartitionModule.h"

#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/DataLayer/DataLayerUtils.h"
#include "WorldPartition/DataLayer/DataLayerInstancePrivate.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"

#if WITH_EDITOR
#include "DataLayer/DataLayerEditorSubsystem.h"
#endif // WITH_EDITOR

namespace UE::MeshPartition
{
AMeshPartitionDataLayerContainer::AMeshPartitionDataLayerContainer()
{
#if WITH_EDITOR
	bIsSpatiallyLoaded = false;
	bListedInSceneOutliner = false;
#endif // WITH_EDITOR
}

AMeshPartitionDataLayerContainer* AMeshPartitionDataLayerContainer::Get(const UWorld* InWorld)
{
	if (auto It = TActorIterator<AMeshPartitionDataLayerContainer>(InWorld); It)
	{
		return *It;
	}

	return nullptr;
}


#if WITH_EDITOR
AMeshPartitionDataLayerContainer* AMeshPartitionDataLayerContainer::GetOrCreate(UWorld* InWorld)
{
	if (AMeshPartitionDataLayerContainer* Container = Get(InWorld))
	{
		return Container;
	}

	return InWorld->SpawnActor<AMeshPartitionDataLayerContainer>();
}

bool AMeshPartitionDataLayerContainer::UpdateDataLayersFromDefinition(const UMeshPartitionDefinition* InDefinition)
{
	Modify(false);

	bool bWasModified = false;
	MeshPartition::FPerBuildVariantDataLayers& BuildVariantLayers = PerDefinitionBuildVariantDataLayers.FindOrAdd(InDefinition);

	for (const MeshPartition::FCompiledSectionBuildVariant& Variant : InDefinition->GetCompiledSectionBuildVariants())
	{
		if (Variant.Name.IsNone())
		{
			continue;
		}

		if (!BuildVariantLayers.Layers.Contains(Variant.Name))
		{
			UDataLayerAsset* DataLayer = NewVariantDataLayer(Variant.Name);
			BuildVariantLayers.Layers.Add(Variant.Name, DataLayer);
			bWasModified = true;
		}
	}

	if (bWasModified)
	{
		Modify(true);
	}
	return bWasModified;
}

bool AMeshPartitionDataLayerContainer::IsDataLayerOwnedByContainer(const UDataLayerAsset* InDataLayer) const
{
	for (const TPair<TObjectPtr<const UMeshPartitionDefinition>, MeshPartition::FPerBuildVariantDataLayers>& BuildVariantLayers : PerDefinitionBuildVariantDataLayers)
	{
		for (const TPair<FName, TObjectPtr<UDataLayerAsset>>& Layer : BuildVariantLayers.Value.Layers)
		{
			if (Layer.Value.Get() == InDataLayer)
			{
				return true;
			}
		}
	}
	return false;
}

bool AMeshPartitionDataLayerContainer::IsDataLayerRelevantForPlatform(const UDataLayerAsset* InDataLayer, ITargetPlatform* InTargetPlatform) const
{
	for (const TPair<TObjectPtr<const UMeshPartitionDefinition>, MeshPartition::FPerBuildVariantDataLayers>& BuildVariantLayers : PerDefinitionBuildVariantDataLayers)
	{
		const UMeshPartitionDefinition* Definition = BuildVariantLayers.Key;
		if (!ensure(Definition))
		{
			continue;
		}

		for (const TPair<FName, TObjectPtr<UDataLayerAsset>>& Layer : BuildVariantLayers.Value.Layers)
		{
			if (Layer.Value.Get() == InDataLayer)
			{
				const FName& VariantName = Layer.Key;
				return Definition->GetCompiledSectionBuildVariantNamesForPlatform(InTargetPlatform).Contains(VariantName);
			}
		}
	}

	// fallback to true if a data layer not managed by this container is found.
	return true;
}

TObjectPtr<UDataLayerAsset> AMeshPartitionDataLayerContainer::FindVariantDataLayer(const UMeshPartitionDefinition* InDefinition, const FName& InBuildVariant) const
{
	if (const auto* BuildVariantLayers = PerDefinitionBuildVariantDataLayers.Find(InDefinition))
	{
		return BuildVariantLayers->Layers.FindRef(InBuildVariant, {}).Get();
	}
	return {};
}

bool AMeshPartitionDataLayerContainer::InitializeDataLayerInstancesWithWorld(const UMeshPartitionDefinition* InDefinition) const
{
	UWorld* World = GetWorld();

	bool bNewDataLayerInstanceCreated = false;
	if (const MeshPartition::FPerBuildVariantDataLayers* DataLayersToInitialize = PerDefinitionBuildVariantDataLayers.Find(InDefinition))
	{
		for (const TPair<FName, TObjectPtr<UDataLayerAsset>>& VariantDataLayer : DataLayersToInitialize->Layers)
		{
			if (UDataLayerEditorSubsystem::Get()->GetDataLayerInstance(VariantDataLayer.Value) == nullptr)
			{
				FDataLayerCreationParameters Params;
				Params.WorldDataLayers = World->GetWorldDataLayers();
				Params.DataLayerAsset = VariantDataLayer.Value;
				UDataLayerInstance* DataLayer = UDataLayerEditorSubsystem::Get()->CreateDataLayerInstance(Params);
				DataLayer->SetInitialRuntimeState(EDataLayerRuntimeState::Unloaded);

				bNewDataLayerInstanceCreated = true;
			}
		}
	}
	return bNewDataLayerInstanceCreated;
}

void AMeshPartitionDataLayerContainer::RemoveDataLayerInstancesFromWorld() const
{
	UWorld* World = GetWorld();

	for (const TPair<TObjectPtr<const UMeshPartitionDefinition>, MeshPartition::FPerBuildVariantDataLayers>& DefinitionDataLayers : PerDefinitionBuildVariantDataLayers)
	{
		for (const TPair<FName, TObjectPtr<UDataLayerAsset>>& VariantDataLayer : DefinitionDataLayers.Value.Layers)
		{
			if (UDataLayerInstance* Instance = UDataLayerEditorSubsystem::Get()->GetDataLayerInstance(VariantDataLayer.Value))
			{
				UDataLayerEditorSubsystem::Get()->DeleteDataLayer(Instance);
			}
		}
	}
}

void AMeshPartitionDataLayerContainer::ClearAllDataLayers()
{
	RemoveDataLayerInstancesFromWorld();
	PerDefinitionBuildVariantDataLayers.Empty();
}

int32 AMeshPartitionDataLayerContainer::PruneUnusedDefinitions(const TSet<const UMeshPartitionDefinition*>& InUsedDefinitions)
{
	UDataLayerEditorSubsystem* DataLayerEditorSubsystem = UDataLayerEditorSubsystem::Get();
	int32 NumPruned = 0;
	
	if (DataLayerEditorSubsystem == nullptr)
	{
		return 0;
	}

	for (auto It = PerDefinitionBuildVariantDataLayers.CreateIterator(); It; ++It)
	{
		const UMeshPartitionDefinition* Definition = It.Key().Get();

		if ((Definition != nullptr) && InUsedDefinitions.Contains(Definition))
		{
			continue;
		}

		if (NumPruned == 0)
		{
			Modify(true);
		}

		// Delete data layer instances belonging to this stale entry.
		for (const TPair<FName, TObjectPtr<UDataLayerAsset>>& VariantDataLayer : It.Value().Layers)
		{
			if (UDataLayerInstance* Instance = DataLayerEditorSubsystem->GetDataLayerInstance(VariantDataLayer.Value))
			{
				DataLayerEditorSubsystem->DeleteDataLayer(Instance);
			}
		}

		It.RemoveCurrent();
		++NumPruned;
	}

	return NumPruned;
}

UDataLayerAsset* AMeshPartitionDataLayerContainer::NewVariantDataLayer(const FName& InPlatformName)
{
	UDataLayerAsset* DataLayer = NewObject<UDataLayerAsset>(this, FName(FString::Printf(TEXT("MPDL_%s"), *InPlatformName.ToString())));
	DataLayer->SetType(EDataLayerType::Runtime);
	DataLayer->SetLoadFilter(EDataLayerLoadFilter::None);
	return DataLayer;
}
#endif // WITH_EDITOR


}
