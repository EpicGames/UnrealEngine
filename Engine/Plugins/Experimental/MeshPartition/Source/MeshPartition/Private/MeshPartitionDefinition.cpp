// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionDefinition.h"
#include "MeshPartitionModule.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialInstance.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/ObjectSaveContext.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/HLOD/HLODLayer.h"

#if WITH_EDITOR
#include "Interfaces/ITargetPlatform.h"
#include "MeshPartitionDependencyInterface.h"
#endif // WITH_EDITOR

namespace UE::MeshPartition
{
const FName ChannelTextureParameterName = TEXT("MeshPartitionChannelTexture");
const int32 PrimitiveDataIndex = 0;

UMeshPartitionDefinition::UMeshPartitionDefinition()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// add a default variant to the CDO
		CompiledSectionBuildVariants.Add(MeshPartition::FCompiledSectionBuildVariant());
	}

#if WITH_EDITOR
	// Install a delegate to catch when referenced objects are modified and notify
	FCoreUObjectDelegates::OnObjectModified.AddUObject(this, &UMeshPartitionDefinition::OnObjectModified);
#endif // WITH_EDITOR
}

UMeshPartitionDefinition::~UMeshPartitionDefinition()
{
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectModified.RemoveAll(this);
	FCoreUObjectDelegates::OnObjectPreSave.RemoveAll(this);
#endif // WITH_EDITOR
}

const UMeshPartitionDefinition* UMeshPartitionDefinition::GetDefaultMegaMeshDefinition()
{
	// we use the CDO as the default definition
	return UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
}

bool UMeshPartitionDefinition::DoesPropertyNameRequireRebuild(const FName& InMemberName, const FName& InName)
{
	return 	(InName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ModifierTypePriorities)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, PreviewSectionBuildVariant)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ChannelMap)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ChannelTexelSize)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ChannelUVLayoutMethod)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ChannelVEUVSamplesPerSquareMeter)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ChannelVEUVVoxelCount)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ChannelPlaneProjectionNormalSource)) ||
			(InMemberName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, ChannelPlaneProjectionFixedNormal));
}

TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> UMeshPartitionDefinition::GetCompiledSectionBuildVariants() const
{
	if (CompiledSectionBuildVariants.Num() == 0)
	{
		const UMeshPartitionDefinition* DefaultDefinition = GetDefaultMegaMeshDefinition();
		check(DefaultDefinition->CompiledSectionBuildVariants.Num() > 0);
		return DefaultDefinition->CompiledSectionBuildVariants;
	}
	return CompiledSectionBuildVariants;
}

#if WITH_EDITOR
void UMeshPartitionDefinition::SetPreviewSectionBuildVariant(const MeshPartition::FCommonBuildVariant& InNewPreviewBuildVairant)
{
	PreviewSectionBuildVariant = InNewPreviewBuildVairant;
}

void UMeshPartitionDefinition::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies, FName InBuildVariantName) const
{
	// only gather dependencies from the build variant we are using
	TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariantArray = GetCompiledSectionBuildVariants();
	for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : BuildVariantArray)
	{
		if (BuildVariant.Name == InBuildVariantName)
		{
			BuildVariant.GatherDependencies(InDependencies);
		}
	}

	// we don't use the material contents in building sections, but we do assign the material, so the path name is a dependency
	if (Material.Get())
	{
		InDependencies += Material.Get()->GetPathName();
	}

	// PerPlatformRuntimeSettings is not a dependency when building sections
	// (platform settings just select the build variant, and sections are built using that build variant)

	InDependencies += ModifierTypePriorities;

	InDependencies += ChannelMap;
	InDependencies += ChannelTexelSize;
	InDependencies += static_cast<uint8>(ChannelUVLayoutMethod);
	InDependencies += ChannelVEUVSamplesPerSquareMeter;
	InDependencies += ChannelVEUVVoxelCount;
	InDependencies += static_cast<uint8>(ChannelPlaneProjectionNormalSource);
	InDependencies += ChannelPlaneProjectionFixedNormal;

	for (const MeshPartition::FPhysicalMaterialChannel& PhysicalChannel : PhysicalMaterialChannels)
	{
		InDependencies += PhysicalChannel.ChannelName;
		InDependencies += PhysicalChannel.MinimumCollisionRelevanceWeight;
		// we assign physical material to collision components, so the path name is a dependency
		if (PhysicalChannel.Material.Get())
		{
			InDependencies += PhysicalChannel.Material.Get()->GetPathName();
		}
	}

	// we assign physical material to collision components, so the path name is a dependency
	if (DefaultPhysicalMaterial.Get())
	{
		InDependencies += DefaultPhysicalMaterial.Get()->GetPathName();
	}

	PreviewSectionBuildVariant.GatherDependencies(InDependencies);
}

void FCommonBuildVariant::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	InDependencies += MaxSectionComplexity;

	if (TransformerPipeline != nullptr)
	{
		TransformerPipeline->GatherDependencies(InDependencies);
	}
}

void FCompiledSectionBuildVariant::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	FCommonBuildVariant::GatherDependencies(InDependencies);

	InDependencies += bSplitSectionsToMatchWorldPartitionRuntimeGrid;
	// Note: resolved grid cell size is hashed at builder level (not here)
	// because it requires world context to resolve from the WP runtime partition.
}
#endif // WITH_EDITOR

void FRuntimeSettings::PostSerialize(const FArchive& InAr)
{
	if (InAr.IsLoading())
	{
		if (BuildVariantName_DEPRECATED != NAME_None)
		{
			BuildVariantNames.AddUnique(BuildVariantName_DEPRECATED);
			BuildVariantName_DEPRECATED = NAME_None;
		}
	}
}

void UMeshPartitionDefinition::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(MeshPartition::FCustomVersion::GUID);
}

void UMeshPartitionDefinition::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITOR
	if (GetLinkerCustomVersion(MeshPartition::FCustomVersion::GUID) < MeshPartition::FCustomVersion::BuildVariantCleanup)
	{
		CompiledSectionBuildVariants = BuildVariants_DEPRECATED;
	}
#endif // WITH_EDITOR
}
	
#if WITH_EDITOR
void UMeshPartitionDefinition::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	OnDefinitionModified.Broadcast(InPropertyChangedEvent.GetMemberPropertyName(), InPropertyChangedEvent.GetPropertyName());
}

const MeshPartition::FCompiledSectionBuildVariant& UMeshPartitionDefinition::GetCompiledSectionBuildVariantByName(const FName& BuildVariantName) const
{
	TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariantArray = GetCompiledSectionBuildVariants();
	check(BuildVariantArray.Num() > 0);

	for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : BuildVariantArray)
	{
		if (BuildVariant.Name == BuildVariantName)
		{
			// found it!
			return BuildVariant;
		}
	}

	UE_LOGF(LogMegaMesh, Warning, "MegaMeshDefinition '%ls' could not find Build Variant named '%ls' -- using first BuildVariant '%ls' instead",
		*this->GetName(),
		*BuildVariantName.ToString(),
		*BuildVariantArray[0].Name.ToString());

	return BuildVariantArray[0];
}

TArray<FName> UMeshPartitionDefinition::GetCompiledSectionBuildVariantNamesForCurrentEditorPreview(const FName& InPreviewPlatformName, ITargetPlatform* PreviewPlatform) const
{
	auto GetAggregatedBuildVariantNames = [](const TArray<FName> BuildVariantNames)
	{
		return FString::JoinBy(BuildVariantNames, TEXT(", "), [](const FName& VariantName)
		{
			return VariantName.ToString();
		});
	};

	// see if there is a preview platform available
	if (PreviewPlatform)
	{
		TArray<FName> BuildVariantNames = GetCompiledSectionBuildVariantNamesForPlatform(PreviewPlatform);

		UE_LOGF(LogMegaMesh, Verbose, " - mapped PreviewPlatform %ls to platform [%ls,%ls] (BuildVariants: %ls)", *InPreviewPlatformName.ToString(), *PreviewPlatform->PlatformName(), *PreviewPlatform->IniPlatformName(), *GetAggregatedBuildVariantNames(BuildVariantNames));
		return MoveTemp(BuildVariantNames);
	}

	// otherwise, use the current platform
	FName TargetPlatformName = FPlatformProperties::PlatformName();		// i.e. "WindowsClient"
	FName IniPlatformName = FPlatformProperties::IniPlatformName();		// i.e. "Windows"
	TArray<FName> BuildVariantNames = GetCompiledSectionBuildVariantNamesForPlatform(TargetPlatformName, IniPlatformName);

	UE_LOGF(LogMegaMesh, Verbose, " - no preview platform active, using current platform: %ls (BuildVariants: %ls)", *TargetPlatformName.ToString(), *GetAggregatedBuildVariantNames(BuildVariantNames));
	return MoveTemp(BuildVariantNames);
}

TArray<FName> UMeshPartitionDefinition::GetCompiledSectionBuildVariantNamesForPlatforms(TArray<ITargetPlatform*> TargetPlatforms) const
{
	TArray<FName> Result;

	for (ITargetPlatform* TargetPlatform : TargetPlatforms)
	{
		TArray<FName> BuildVariantNames = GetCompiledSectionBuildVariantNamesForPlatform(TargetPlatform);
		for (const FName& BuildVariantName : BuildVariantNames)
		{
			Result.AddUnique(BuildVariantName);
		}
	}

	return Result;
}

TArray<FName> UMeshPartitionDefinition::GetCompiledSectionBuildVariantNamesForPlatform(const ITargetPlatform* TargetPlatform) const
{
	const FName TargetPlatformName = FName(TargetPlatform->PlatformName());	// i.e. "WindowsClient"
	const FName IniPlatformName = FName(TargetPlatform->IniPlatformName());	// i.e. "Windows"
	return GetCompiledSectionBuildVariantNamesForPlatform(TargetPlatformName, IniPlatformName);
}

TArray<FName> UMeshPartitionDefinition::GetCompiledSectionBuildVariantNamesForPlatform(const FName& InPlatformName, const FName& InIniPlatformName) const
{
	TArray<FName> Result;
	TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariantArray = GetCompiledSectionBuildVariants();
	
	check(BuildVariantArray.Num() > 0);

	// look for an exact match on the platform name, otherwise fallback to IniPlatformName
	const MeshPartition::FRuntimeSettings& PlatformRuntimeSettings =
		PerPlatformRuntimeSettings.PerPlatform.Contains(InPlatformName) ?
			PerPlatformRuntimeSettings.GetValueForPlatform(InPlatformName) :
			PerPlatformRuntimeSettings.GetValueForPlatform(InIniPlatformName);

	// we want to filter to only build variants that actually exist -- so iterate the build variants that exist, and select the ones that are in the platform build variants
	for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : BuildVariantArray)
	{
		if (PlatformRuntimeSettings.BuildVariantNames.Find(BuildVariant.Name) != INDEX_NONE)
		{
			Result.Emplace(BuildVariant.Name);
		}
	}

	if (Result.IsEmpty())
	{
		UE_LOGF(LogMegaMesh, Verbose, "Could not determine the Build Variants from MegaMeshDefinition '%ls' for platform [%ls, %ls] -- will fallback to use the first build variant instead (%ls)",
			*this->GetName(),
			*InPlatformName.ToString(),
			*InIniPlatformName.ToString(),
			*BuildVariantArray[0].Name.ToString());

		Result.Emplace(BuildVariantArray[0].Name);
	}
	
	return Result;
}

TArray<FName> UMeshPartitionDefinition::GetAllCompiledSectionBuildVariantNames() const
{
	TArray<FName> Result;

	TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariantArray = GetCompiledSectionBuildVariants();
	for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : BuildVariantArray)
	{
		Result.Emplace(BuildVariant.Name);
	}

	return Result;
}

void UMeshPartitionDefinition::OnObjectModified(UObject* InObject)
{
	UTransformerPipeline* TransformerPipeline = Cast<UTransformerPipeline>(InObject);

	if ((TransformerPipeline != nullptr) && (PreviewSectionBuildVariant.TransformerPipeline == TransformerPipeline))
	{
		OnDefinitionModified.Broadcast(GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, PreviewSectionBuildVariant), GET_MEMBER_NAME_CHECKED(FCommonBuildVariant, TransformerPipeline));
	}
}

#endif // WITH_EDITOR	

bool UMeshPartitionDefinition::ResolveValue(FName EntryName, int32& OutValue, int32 DefaultValue) const
{
	OutValue = GetChannelMap().FindChannel(EntryName);

	if (OutValue == INDEX_NONE)
	{
		OutValue = DefaultValue;
		return false;
	}

	return true;
}

void UMeshPartitionDefinition::ForEachEntry(TFunctionRef<void(FName Name, int32 Value)> Iterator) const
{
	int32 Index = 0;
	for (FName Name : GetChannelMap().GetChannels())
	{
		Iterator(Name, Index);
		Index++;
	}
}
} // namespace UE::MeshPartition
