// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionGridSettings.h"

#include "MeshPartitionDependencyInterface.h"

namespace UE::MeshPartition
{
void FGridSettings::GatherDependencies(IDependencyInterface& InOutDependencies) const
{
	InOutDependencies += CellSize;
	InOutDependencies += bIs2D;
	InOutDependencies += WorldOriginOffset.X;
	InOutDependencies += WorldOriginOffset.Y;
	InOutDependencies += WorldOriginOffset.Z;
}

FArchive& operator<<(FArchive& Ar, FGridSettings& InOutSettings)
{
	Ar << InOutSettings.CellSize;
	Ar << InOutSettings.bIs2D;
	Ar << InOutSettings.WorldOriginOffset;
	return Ar;
}
} // namespace UE::MeshPartition
