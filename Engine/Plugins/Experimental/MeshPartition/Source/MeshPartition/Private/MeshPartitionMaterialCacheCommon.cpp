// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMaterialCacheCommon.h"
#include "Materials/MaterialInterface.h"
#include "MaterialCachedData.h"
#include "MaterialCache/MaterialCache.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "Components/SceneComponent.h"
#include "Engine/World.h"

bool UE::MeshPartition::IsMaterialCacheEnabled(UWorld* World)
{
	EShaderPlatform ShaderPlatform;
	if (World)
	{
		ShaderPlatform = GetFeatureLevelShaderPlatform_Checked(World->GetFeatureLevel());
	}
	else
	{
		ShaderPlatform = GetMaxShaderPlatformChecked();
	}
	
	return IsMaterialCacheEnabled(ShaderPlatform);
}

void UE::MeshPartition::UpdateMaterialCacheTextures(
	USceneComponent* Owner,
	UMaterialInterface* Material,
	const FIntPoint& TileCount,
	TArray<TObjectPtr<UMaterialCacheVirtualTexture>>& Textures
)
{
	// Release old textures
	for (TObjectPtr<UMaterialCacheVirtualTexture> VirtualTexture : Textures)
	{
		VirtualTexture->Unregister();
		VirtualTexture->ReleaseResource();
	}
	
	Textures.Empty();
	
	// Base the texture set on the tags used in the material
	const FMaterialCachedExpressionData& ExpressionData = Material->GetCachedExpressionData();

	// Create a texture per used tag
	for (TObjectPtr<UMaterialCacheVirtualTextureTag> Tag : ExpressionData.MaterialCacheTags)
	{
		UMaterialCacheVirtualTexture* VirtualTexture = NewObject<UMaterialCacheVirtualTexture>(Owner->GetOuter());
		VirtualTexture->OwningComponent = Owner;
		VirtualTexture->Tag = Tag;
		VirtualTexture->TileCount = TileCount;
		
		// Combined shading pass, as a single material is shared across all primitives
		VirtualTexture->bSharedShading = true;
		
		VirtualTexture->UpdateResource();
		Textures.Add(VirtualTexture);
	}
}
