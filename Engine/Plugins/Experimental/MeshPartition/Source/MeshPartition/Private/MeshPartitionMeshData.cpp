// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMeshData.h"

#include <atomic>

#include "VectorUtil.h"
#include "Async/ParallelFor.h"
#include "MeshDescriptionBuilder.h"
#include "MeshPartitionModule.h"
#include "MeshAdapter/MeshVertexNormals.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "ProfilingDebugging/ScopedTimers.h"

#include "Operations/MeshClusterSimplifier.h"

namespace UE::MeshPartition
{
namespace MegaMeshMeshDataLocals
{

	// This is a workaround for some platforms not supporting C++20's std::atomic_ref
	// UE platforms should be on C++20, so hopefully we can remove this method in the future and just use std::atomic_ref directly
	// Atomic_Ref is a slight performance improvement for platforms with weak memory order such as ARM as we can avoid a full barrier with
	// relaxed ordering here.
	template<typename IntType = int>
	static inline void AtomicIncrement(IntType& Value, std::memory_order MemoryOrder)
	{
#if defined(__cpp_lib_atomic_ref)
		std::atomic_ref<IntType>(Value).fetch_add(1, MemoryOrder);
#else
		FPlatformAtomics::InterlockedIncrement(&Value);
#endif
	}
}

int FMeshData::AppendVertex(const FVector3d& InPosition)
{
	// Reuse any free vertices before trying to add a new one.
	if (FreeVertices.Num() != 0)
	{
		checkSlow(Algo::IsHeap(FreeVertices));

		int VertexID = -1;
		FreeVertices.HeapPop(VertexID, EAllowShrinking::No);
		ensure(VertexRefCount[VertexID] == INVALID_REF_COUNT);
		
		Vertices[VertexID] = InPosition;
		VertexRefCount[VertexID] = 0;
		ChannelUVs[VertexID] = FVector2f::ZeroVector;

		// Reset source UV channels for reused vertex
		for (TArray<FVector2f>& UVChannel : SourceUVChannels)
		{
			UVChannel[VertexID] = FVector2f::ZeroVector;
		}

		for (TPair<FName, TArray<float>>& Channel : WeightLayers)
		{
			Channel.Value[VertexID] = 0.;
		}

		return VertexID;
	}
	const int VertexID = Vertices.Add(InPosition);
	ChannelUVs.Add(FVector2f::ZeroVector);
	VertexRefCount.Add(0);
	Normals.Add(FVector3f(0, 0, 1.));

	for (TArray<FVector2f>& UVChannel : SourceUVChannels)
	{
		UVChannel.Add(FVector2f::ZeroVector);
	}


	for (TPair<FName, TArray<float>>& Channel : WeightLayers)
	{
		Channel.Value.Add({});
	}
	return VertexID;
}

int FMeshData::AppendTriangle(const Geometry::FIndex3i& InTriangle)
{
	for (int Index = 0; Index < 3; ++Index)
	{
		ensure(IsVertex(InTriangle[Index]));
		ensure(VertexRefCount[InTriangle[Index]] != INVALID_REF_COUNT);
		++VertexRefCount[InTriangle[Index]];
	}

	if (FreeTriangles.Num() != 0)
	{
		checkSlow(Algo::IsHeap(FreeTriangles));

		int TriangleID = - 1;
		FreeTriangles.HeapPop(TriangleID, EAllowShrinking::No);
		
		Triangles[TriangleID] = InTriangle;
		TriangleRefCount[TriangleID] = 1;
		
		return TriangleID;
	}

	const int TriangleID = Triangles.Add(InTriangle);
	BaseIDLayer.Add({});
	TriangleRefCount.Add(1);

	return TriangleID;
}

void FMeshData::RecomputeNormals(const bool bRequireDeterministicNormals)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshData::RecomputeNormals);

	Geometry::MeshNormals::ComputeVertexNormals<FMeshData, TArray<FVector3f>>(*this, Normals, bRequireDeterministicNormals);
}

void FMeshData::SummarizeUVRegion()
{
	UVRegion.Min = FVector2f(FLT_MAX, FLT_MAX);
	UVRegion.Max = FVector2f(-FLT_MAX, -FLT_MAX);
	
	for (int32_t VertexIndex : VertexIndicesItr())
	{
		UVRegion.Min = FVector2f::Min(UVRegion.Min, GetChannelUV(VertexIndex));
		UVRegion.Max = FVector2f::Max(UVRegion.Max, GetChannelUV(VertexIndex));
	}
}

FVector3d FMeshData::GetTriNormal(int InTriangleID) const
{
	FVector3d V0, V1, V2;
	GetTriVertices(InTriangleID, V0, V1, V2);
	return Geometry::VectorUtil::Normal(V0, V1, V2);
}

Geometry::FAxisAlignedBox3d FMeshData::GetBounds(bool bParallel) const
{
	TArray<Geometry::FAxisAlignedBox3d> Chunks;
	ParallelForWithTaskContext(Chunks, MaxVertexID(),
		[&](Geometry::FAxisAlignedBox3d& Chunk, int VertexID)
		{
			if (!IsVertex(VertexID))
			{
				return;
			}

			Chunk.Contain(GetVertex(VertexID));
		},
		bParallel ? EParallelForFlags::None : EParallelForFlags::ForceSingleThread
	);

	Geometry::FAxisAlignedBox3d Result;
	for (const Geometry::FAxisAlignedBox3d& Chunk : Chunks)
	{
		Result.Contain(Chunk);
	}
	return Result;
}

void FMeshData::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshData::Serialize);

	// Skip serializing for UObject reference collectors, we have none.
	if (Ar.IsObjectReferenceCollector())
	{
		return;
	}

	Ar << Vertices;
	Ar << Triangles;
	Ar << Normals;
	Ar << WeightLayers;
	Ar << ChannelUVs;
	Ar << SourceUVChannels;
	Ar << BaseIDLayer;
	Ar << FreeTriangles;
	Ar << UVRegion;

	if (Ar.IsLoading())
	{
		checkSlow(Algo::IsHeap(FreeTriangles));

		// reconstruct the vertex ref count and free vertices list:
		VertexRefCount.SetNumUninitialized(Vertices.Num());
		TriangleRefCount.SetNumUninitialized(Triangles.Num());

		
		// Assume all vertices are valid initially:
		for (int Index = 0; Index < VertexRefCount.Num(); ++Index)
		{
			VertexRefCount[Index] = 0;
		}
		
		// Build the triangle ref count list by first assuming all triangles are valid
		// then assign invalid indices to the free triangles
		for (int Index = 0; Index < TriangleRefCount.Num(); ++Index)
		{
			TriangleRefCount[Index] = 1;
		}
		
		for (int FreeTriangle : FreeTriangles)
		{
			TriangleRefCount[FreeTriangle] = INVALID_REF_COUNT;
		}
	
		ParallelFor(MaxTriangleID(), [&](int TriangleID)
		{
			if (!IsTriangle(TriangleID))
			{
				return;
			}

			const Geometry::FIndex3i& Triangle = Triangles[TriangleID];
			for (int Index = 0; Index < 3; ++Index)
			{
				MegaMeshMeshDataLocals::AtomicIncrement(VertexRefCount[Triangle[Index]], std::memory_order_relaxed);
			}
		});

		for (int VertexID = 0; VertexID < Vertices.Num(); ++VertexID)
		{
			if (!HasTriangle(VertexID))
			{
				FreeVertices.HeapPush(VertexID);
				VertexRefCount[VertexID] = INVALID_REF_COUNT;
			}
		}
	}
}

void FMeshData::MergeVertexPairs(TConstArrayView<TPair<int, int>> InMergePairs, TOptional<TConstArrayView<int>> InTriangleFilter)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshData::MergeVertexPairs);

	using FTrianglesArray = TArray<int, TInlineAllocator<8>>;

	TMap<int, FTrianglesArray> VertexToTriangle;
	auto MapVertexToTriangle = [this, &VertexToTriangle](int VertexID, int TriangleID)
	{
		FTrianglesArray& TrianglesA = VertexToTriangle.FindOrAdd(VertexID, {});
		TrianglesA.Add(TriangleID);
	};

	if (InTriangleFilter.IsSet())
	{
		for (int TriangleID : InTriangleFilter.GetValue())
		{
			const Geometry::FIndex3i Triangle = GetTriangle(TriangleID);
			MapVertexToTriangle(Triangle.A, TriangleID);
			MapVertexToTriangle(Triangle.B, TriangleID);
			MapVertexToTriangle(Triangle.C, TriangleID);
		}
	}
	else
	{
		// In the case where no triangle filter is passed in, the algorithm must
		// check every triangle to see if it contains one of the discarded vertices.
		// To do this efficiently, first build an accelleration structure to reduce 
		// the number of triangles which are added to the VertexToTriangle map.
		// This allows speed up via ParallelFor by reducing contention on the structure.
			
		TSet<int> VerticesOfInterest;
		VerticesOfInterest.Reserve(InMergePairs.Num());
		for (const TPair<int, int>& MergePair : InMergePairs)
		{
			const int DiscardVID = MergePair.Value;
			VerticesOfInterest.Add(DiscardVID);
		}

		FCriticalSection CriticalSection;
		ParallelFor(MaxTriangleID(), [&](int TriangleID)
		{
			const Geometry::FIndex3i Triangle = GetTriangle(TriangleID);
			for (int Index = 0; Index < 3; ++Index)
			{
				if (VerticesOfInterest.Contains(Triangle[Index]))
				{
					FScopeLock Lock(&CriticalSection);
					MapVertexToTriangle(Triangle[Index], TriangleID);
				}
			}
		});
	}

	// In the code below, we might collapse some triangles, which would affect the ref count of
	//  vertices aside from the discarded ones. It might be safe to delete them as we go along
	//  if the ref count hits zero, but to avoid having to consider how doing so would interact 
	//  with the rest of the function, we'll just check the ref counts after doing everything else.
	TSet<int32> VidsToCheckLater;

	for (const TPair<int, int>& MergePair : InMergePairs)
	{
		const int KeepVID = MergePair.Key;
		const int DiscardVID = MergePair.Value;

		// Previous iterations may have deleted either of the vertices, ignore invalid pairs.
		if (!IsVertex(KeepVID) || !IsVertex(DiscardVID))
		{
			continue;
		}

		if (!ensure(KeepVID != DiscardVID))
		{
			continue;
		}

		if (!ensure(VertexToTriangle.Contains(DiscardVID)))
		{
			continue;
		}

		const FTrianglesArray& TrianglesToFix = VertexToTriangle[DiscardVID];

		for (int TriangleID : TrianglesToFix)
		{
			// It's possible for a triangle to have been removed (see further below)
			if (!IsTriangle(TriangleID))
			{
				continue;
			}

			Geometry::FIndex3i& Triangle = Triangles[TriangleID];

			// If the triangle contains the kept vid as well, then there must have been an edge between KeepVID and DiscardVID.
			//  this can happen in case of degenerate edges when the mapping of merged vertices is not 1:1, where after one
			//  merge, another pair is connected by an edge.
			// There are a few ways to handle the situation: we could do this check before we touch any triangles and avoid
			//  merging entirely, we can force the mapping to be 1:1 (but this requires doing merges a section at a time, since
			//  corners need to be able to merge to multiple verts), or we can just properly handle the collapse, which is
			//  what we will do.
			if (Triangle.Contains(KeepVID))
			{
				for (int SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					VidsToCheckLater.Add(Triangle[SubIdx]);
				}
				RemoveTriangle(TriangleID, /*bRemoveUnusedVertices=*/ false);
				continue;
			}
				
			for (int Index = 0; Index < 3; ++Index)
			{
				if (Triangle[Index] == DiscardVID)
				{
					--VertexRefCount[DiscardVID];
					++VertexRefCount[KeepVID];

					Triangle[Index] = KeepVID;
					break;
				}
			}
		}

		// map the KeepVID to this new triangle, in case further merge pairs might try to discard it.
		for (int TriangleID : TrianglesToFix)
		{
			MapVertexToTriangle(KeepVID, TriangleID);
		}
		VertexToTriangle.Remove(DiscardVID);

		RemoveVertex(DiscardVID);
	} // end for merge pairs

	for (int32 Vid : VidsToCheckLater)
	{
		if (VertexRefCount.IsValidIndex(Vid) && VertexRefCount[Vid] == 0)
		{
			RemoveVertex(Vid);
		}
	}
}

void FMeshData::AppendDynamicMesh(const Geometry::FDynamicMesh3& InOtherMesh, const FTransform& InRelativeTransform, int InNewBaseID, TSet<int>* OutMergeBoundaryVertices)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshData::AppendDynamicMesh);

	TMap<int, int> VertexMap;
	VertexMap.Reserve(InOtherMesh.VertexCount());
	ReserveAdditionalVertices(InOtherMesh.VertexCount());
	for (int FromVID : InOtherMesh.VertexIndicesItr())
	{
		const FVector3d Vertex = InOtherMesh.GetVertex(FromVID);
		const int ToVID = AppendVertex(InRelativeTransform.TransformPosition(Vertex));
		VertexMap.Add(FromVID, ToVID);
	}

	ReserveAdditionalTriangles(InOtherMesh.TriangleCount());
	for (int FromTID : InOtherMesh.TriangleIndicesItr())
	{
		const Geometry::FIndex3i FromTriangle = InOtherMesh.GetTriangle(FromTID);
		Geometry::FIndex3i Triangle;
		Triangle.A = VertexMap[FromTriangle.A];
		Triangle.B = VertexMap[FromTriangle.B];
		Triangle.C = VertexMap[FromTriangle.C];
		const int TriangleID = AppendTriangle(Triangle);

		if (InNewBaseID != INDEX_NONE)
		{
			SetBaseID(TriangleID, InNewBaseID);
		}
	}

	if (InOtherMesh.Attributes())
	{
		const int32 NumUVLayers = InOtherMesh.Attributes()->NumUVLayers();
		GrowSourceUVChannelsTo(NumUVLayers);

		for (int32 UVLayerIdx = 0; UVLayerIdx < FMath::Min(NumUVLayers, MAX_SOURCE_UV_CHANNELS); ++UVLayerIdx)
		{
			if (const Geometry::FDynamicMeshUVOverlay* UVLayer = InOtherMesh.Attributes()->GetUVLayer(UVLayerIdx))
			{
				for (int ElementID : UVLayer->ElementIndicesItr())
				{
					const int VID = UVLayer->GetParentVertex(ElementID);
					if (VID != INDEX_NONE)
					{
						SetVertexUV(VertexMap[VID], UVLayer->GetElement(ElementID), UVLayerIdx);
					}
				}
			}
		}

		if (const Geometry::FDynamicMeshNormalOverlay* SectionNormalLayer = InOtherMesh.Attributes()->PrimaryNormals())
		{
			for (int ElementID : SectionNormalLayer->ElementIndicesItr())
			{
				const int VID = SectionNormalLayer->GetParentVertex(ElementID);
				if (VID != INDEX_NONE)
				{
					SetVertexNormal(VertexMap[VID], SectionNormalLayer->GetElement(ElementID));
				}
			}
		}

		for (int WeightLayerIndex = 0; WeightLayerIndex < InOtherMesh.Attributes()->NumWeightLayers(); ++WeightLayerIndex)
		{
			const Geometry::FDynamicMeshWeightAttribute* WeightLayer = InOtherMesh.Attributes()->GetWeightLayer(WeightLayerIndex);
			InitializeWeightLayer(WeightLayer->GetName());

			for (int FromVID : InOtherMesh.VertexIndicesItr())
			{
				float WeightLayerValue = 0.;
				WeightLayer->GetValue(FromVID, &WeightLayerValue);
				SetWeightLayerValue(WeightLayer->GetName(), VertexMap[FromVID], WeightLayerValue);
			}
		}
	}

	if (OutMergeBoundaryVertices)
	{
		for (int EdgeID : InOtherMesh.BoundaryEdgeIndicesItr())
		{
			Geometry::FDynamicMesh3::FEdge Edge = InOtherMesh.GetEdge(EdgeID);
			for (int Index = 0; Index < 2; ++Index)
			{
				OutMergeBoundaryVertices->Add(VertexMap[Edge.Vert[Index]]);
			}
		}
	}
}

void FMeshData::WeldCoincidentVertices(TConstArrayView<TPair<int, int>> InVertexSourcePairs)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshData::WeldCoincidentVertices);

	TSet<TPair<int, int>> MergePairs;
	FCriticalSection CriticalSection;
	ParallelFor(InVertexSourcePairs.Num(), [&](int Index)
	{
		const int VertexID = InVertexSourcePairs[Index].Key;
		const int SourceID = InVertexSourcePairs[Index].Value;
		ensure(IsVertex(VertexID));
		const FVector3d VertexPos = GetVertex(VertexID);

		for (const TPair<int, int>& OtherVertex : InVertexSourcePairs)
		{
			const int OtherVertexID = OtherVertex.Key;
			const int OtherSourceID = OtherVertex.Value;

			if (VertexID == OtherVertexID || SourceID == OtherSourceID)
			{
				continue;
			}

			const FVector3d OtherVertexPos = GetVertex(OtherVertexID);

			if (VertexPos.Equals(OtherVertexPos, 1.e-4f))
			{
				FScopeLock Lock(&CriticalSection);
				if (VertexID > OtherVertexID)
				{
					MergePairs.Add({ OtherVertexID, VertexID });
				}
				else
				{
					MergePairs.Add({ VertexID, OtherVertexID });
				}
			}
		}
	});

	MergeVertexPairs(MergePairs.Array(), {});
}

void FMeshData::ConvertToDynamicMesh(Geometry::FDynamicMesh3& OutDynamicMesh) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshData::ConvertToDynamicMesh);

	checkSlow(CheckValidity());

	OutDynamicMesh.Clear();

	OutDynamicMesh.EnableAttributes();
	OutDynamicMesh.Attributes()->SetNumNormalLayers(1);
	OutDynamicMesh.Attributes()->SetNumWeightLayers(WeightLayers.Num());

	const int32 NumSourceUVChannels = GetNumSourceUVChannels();
	const int32 NumUVChannels = GetNumUVChannels();
	OutDynamicMesh.Attributes()->SetNumUVLayers(GetNumUVChannels());

	int CurrentWeightLayerIndex = 0;
	TMap<FName, Geometry::FDynamicMeshWeightAttribute*> ResultWeightLayers;
	for (const TPair<FName, TArray<float>>& WeightLayer : WeightLayers)
	{
		Geometry::FDynamicMeshWeightAttribute* ResultWeightLayer = OutDynamicMesh.Attributes()->GetWeightLayer(CurrentWeightLayerIndex++);
		ResultWeightLayer->SetName(WeightLayer.Key);
		ResultWeightLayers.Add(WeightLayer.Key, ResultWeightLayer);
	}

	TMap<int, int> VertexMap;

	Geometry::FDynamicMeshNormalOverlay* NormalOverlay = OutDynamicMesh.Attributes()->PrimaryNormals();

	TArray<Geometry::FDynamicMeshUVOverlay*> UVOverlays;
	UVOverlays.Reserve(NumUVChannels);
	for (int32 ChannelIdx = 0; ChannelIdx < NumUVChannels; ++ChannelIdx)
	{
		UVOverlays.Add(OutDynamicMesh.Attributes()->GetUVLayer(ChannelIdx));
	}

	// Helper to copy attributes over to a newly-inserted vertex.
	// Used for initial insertion pass and non-manifold edge fix-ups below
	auto CopyPerVertexAttributes = [this, &UVOverlays, NormalOverlay, NumSourceUVChannels, &ResultWeightLayers]
		(int32 SourceVID, int32 TargetVID, bool bUnsafe)
	{
		const FVector2f ChannelUV = GetChannelUV(SourceVID);
		UVOverlays[0]->InsertElement(TargetVID, &ChannelUV.X, bUnsafe);

		for (int32 ChannelIdx = 0; ChannelIdx < NumSourceUVChannels; ++ChannelIdx)
		{
			const FVector2f UV = GetVertexUV(SourceVID, ChannelIdx);
			UVOverlays[ChannelIdx + SOURCE_UV_OFFSET]->InsertElement(TargetVID, &UV.X, bUnsafe);
		}

		const FVector3f Normal = GetVertexNormal(SourceVID);
		NormalOverlay->InsertElement(TargetVID, &Normal.X, bUnsafe);

		for (const TPair<FName, TArray<float>>& WeightLayer : WeightLayers)
		{
			ResultWeightLayers[WeightLayer.Key]->SetNewValue(TargetVID, &WeightLayer.Value[SourceVID]);
		}
	};

	OutDynamicMesh.BeginUnsafeVerticesInsert();
	NormalOverlay->BeginUnsafeElementsInsert();
	for (Geometry::FDynamicMeshUVOverlay* UVOverlay : UVOverlays)
	{
		UVOverlay->BeginUnsafeElementsInsert();
	}
	for (int VertexIndex : VertexIndicesItr())
	{
		Geometry::FVertexInfo VertexInfo;
		VertexInfo.Position = Vertices[VertexIndex];

		OutDynamicMesh.InsertVertex(VertexIndex, VertexInfo, true);
		VertexMap.Add(VertexIndex, VertexIndex);

		CopyPerVertexAttributes(VertexIndex, VertexIndex, /* bUnsafe = */ true);
	}
	OutDynamicMesh.EndUnsafeVerticesInsert();
	NormalOverlay->EndUnsafeElementsInsert();
	for (Geometry::FDynamicMeshUVOverlay* UVOverlay : UVOverlays)
	{
		UVOverlay->EndUnsafeElementsInsert();
	}

	OutDynamicMesh.BeginUnsafeTrianglesInsert();
	int32 NumUninsertableTriangles = 0;
	int32 NumDuplicatedVertices = 0;
	for (int TriangleIndex : TriangleIndicesItr())
	{
		const Geometry::FIndex3i Triangle = Triangles[TriangleIndex];

		Geometry::FIndex3i ResultTriangle(VertexMap[Triangle.A], VertexMap[Triangle.B], VertexMap[Triangle.C]);

		Geometry::EMeshResult InsertResult = OutDynamicMesh.InsertTriangle(TriangleIndex, ResultTriangle, 0, /* bUnsafe = */ true);

		if (InsertResult == Geometry::EMeshResult::Failed_WouldCreateNonmanifoldEdge)
		{
			// Duplicate vertices at the non-boundary edge(s) so we can still insert the triangle
			// (matching MeshDescription->DynamicMesh conversion)
			bool bDuplicate[3] = { false, false, false };
			for (int32 PrevCorner = 2, Corner = 0; Corner < 3; PrevCorner = Corner++)
			{
				const int32 EID = OutDynamicMesh.FindEdge(ResultTriangle[PrevCorner], ResultTriangle[Corner]);
				if (EID != Geometry::FDynamicMesh3::InvalidID && OutDynamicMesh.IsBoundaryEdge(EID) == false)
				{
					bDuplicate[PrevCorner] = true;
					bDuplicate[Corner] = true;
				}
			}
			
			for (int32 Corner = 0; Corner < 3; ++Corner)
			{
				if (!bDuplicate[Corner])
				{
					continue;
				}
				const int32 SourceVID = Triangle[Corner];

				const int32 NewVID = OutDynamicMesh.AppendVertex(Vertices[SourceVID]);
				ResultTriangle[Corner] = NewVID;
				++NumDuplicatedVertices;

				// Note we've ended the unsafe-inserts on overlay elements above, so we safe-insert these
				CopyPerVertexAttributes(SourceVID, NewVID, /* bUnsafe = */ false);
			}

			InsertResult = OutDynamicMesh.InsertTriangle(TriangleIndex, ResultTriangle, 0, /* bUnsafe = */ true);
		}

		if (InsertResult != Geometry::EMeshResult::Ok)
		{
			// Could not be represented (invalid or duplicate triangle)
			++NumUninsertableTriangles;
			continue;
		}

		// Note: by construction above, element ID is always vertex ID, so overlay tris match mesh tris
		for (Geometry::FDynamicMeshUVOverlay* UVOverlay : UVOverlays)
		{
			UVOverlay->SetTriangle(TriangleIndex, ResultTriangle);
		}
		NormalOverlay->SetTriangle(TriangleIndex, ResultTriangle);
	}
	OutDynamicMesh.EndUnsafeTrianglesInsert();

	if (NumDuplicatedVertices > 0 || NumUninsertableTriangles > 0)
	{
		UE_LOGF(LogMegaMesh, Warning,
			"FMeshData::ConvertToDynamicMesh: duplicated %d vertex/vertices to resolve non-manifold triangles; %d of %d source triangles still could not be represented (degenerate or duplicate) and were skipped.",
			NumDuplicatedVertices, NumUninsertableTriangles, TriangleCount());
	}

	// Clear out unreferenced elements (w/ no associated triangle, and thus no associated parent vertex)
	for (Geometry::FDynamicMeshUVOverlay* UVOverlay : UVOverlays)
	{
		UVOverlay->FreeUnusedElements(nullptr);
	}
	NormalOverlay->FreeUnusedElements(nullptr);
}

void FMeshData::ConvertToMeshDescription(FMeshDescription& OutMeshDescription) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshData::ConvertToMeshDescription);

	checkSlow(CheckValidity());

	OutMeshDescription.Empty();

	FMeshDescriptionBuilder Builder;
	Builder.SetMeshDescription(&OutMeshDescription);

	Builder.SuspendMeshDescriptionIndexing();

	// Export all source UV channels
	const int32 NumSourceUVChannels = GetNumSourceUVChannels();

	const int32 TotalUVLayers = GetNumUVChannels();
	Builder.SetNumUVLayers(TotalUVLayers);

	Builder.ReserveNewVertices(VertexCount());

	Builder.ReserveNewUVs(VertexCount(), CHANNEL_UV_INDEX);
	for (int32 ChannelIdx = 0; ChannelIdx < NumSourceUVChannels; ++ChannelIdx)
	{
		Builder.ReserveNewUVs(VertexCount(), ChannelIdx + SOURCE_UV_OFFSET);
	}

	TArray<FVertexID> VertexMapping;
	VertexMapping.SetNum(MaxVertexID());

	int NextVertexID = 0;
	for (int VertexID : VertexIndicesItr())
	{
		VertexMapping[VertexID] = Builder.AppendVertex(GetVertex(VertexID));
	}

	FPolygonGroupID ZeroGroupID = Builder.AppendPolygonGroup();

	for (int TriangleID : TriangleIndicesItr())
	{
		Geometry::FIndex3i Triangle = GetTriangle(TriangleID);

		FVertexInstanceID TriVertInstances[3];
		for (int Index = 0; Index < 3; ++Index)
		{
			const FVertexID Vertex = VertexMapping[Triangle[Index]];
			TriVertInstances[Index] = Builder.AppendInstance(Vertex);
		}

		const FTriangleID NewTriangle = Builder.AppendTriangle(TriVertInstances[0], TriVertInstances[1], TriVertInstances[2], ZeroGroupID);

		{
			FUVID UVIDs[3] = {};

			for (int Index = 0; Index < 3; ++Index)
			{
				const FVector2f ChannelUV = GetChannelUV(Triangle[Index]);
				UVIDs[Index] = Builder.AppendUV(FVector2D(ChannelUV), CHANNEL_UV_INDEX);
			}

			Builder.AppendUVTriangle(NewTriangle, UVIDs[0], UVIDs[1], UVIDs[2], CHANNEL_UV_INDEX);
		}

		for (int32 ChannelIdx = 0; ChannelIdx < NumSourceUVChannels; ++ChannelIdx)
		{
			FUVID UVIDs[3] = {};

			for (int Index = 0; Index < 3; ++Index)
			{
				UVIDs[Index] = Builder.AppendUV(FVector2D(GetVertexUV(Triangle[Index], ChannelIdx)), ChannelIdx + SOURCE_UV_OFFSET);
			}

			Builder.AppendUVTriangle(NewTriangle, UVIDs[0], UVIDs[1], UVIDs[2], ChannelIdx + SOURCE_UV_OFFSET);
		}

		for (int Index = 0; Index < 3; ++Index)
		{
			const FVector Normal = FVector(GetVertexNormal(Triangle[Index]));
			Builder.SetInstanceNormal(TriVertInstances[Index], Normal);
		}
	}

	Builder.ResumeMeshDescriptionIndexing();
}


static TAutoConsoleVariable<float> CVarCustomMeshPhysicsEdgeLimit(TEXT("MegaMesh.Preview.PreviewMeshPhysicsDensityLimit"),
	0.002f,  // 0.002 = max 500 edges along the length of the longest side of the bounding box
	TEXT("Used to reduce the collision data for the preview mesh.  Max edge length as a factor of the bounding box size."));

bool FMeshData::ConvertToTriMeshCollisionData(FTriMeshCollisionData* CollisionData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMeshData::ConvertToTriMeshCollisionData);
	check(CollisionData);

	// The simplifier speeds up collision generation roughly linearly, but has some cost of its own.  Unfortunately there is no
	// convenient place to hook this to get it off the game thread, like there is for the code that will consume the FTriMeshCollisionData.
	// Leave this disabled for now.
	constexpr bool bSimplifyCustomMeshCollisionData = false;
	if (bSimplifyCustomMeshCollisionData)
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);
		Timer.Start();

		Geometry::FAxisAlignedBox3d Bounds;
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(GetBounds);
			Bounds = GetBounds(false);
		}
		
		double MaxDim = FMath::Max(FMath::Max(Bounds.Width(), Bounds.Height()), Bounds.Depth());
		double TargetEdgeLength = MaxDim * CVarCustomMeshPhysicsEdgeLimit.GetValueOnGameThread();

		const FMeshData* SrcMesh = this;

		UE::Geometry::TMeshWrapperAdapterd<const MeshPartition::FMeshData> InputAdapter(SrcMesh);
		UE::Geometry::MeshClusterSimplify::FResultMeshAdapter ResultAdapter;

		// Use FResultMeshAdapter to write directly to the FTriMeshCollisionData
		CollisionData->Vertices.Reserve(Vertices.Num() / 2);
		CollisionData->Indices.Reserve(Triangles.Num() / 2);
		ResultAdapter.AppendVertex = [CollisionData](FVector3d V) -> int32 { return CollisionData->Vertices.Add(FVector3f(V));  };
		ResultAdapter.Clear = [CollisionData]() { CollisionData->Vertices.Empty(); CollisionData->Indices.Empty(); };
		ResultAdapter.GetVertex = [CollisionData](int32 VID) -> FVector3d { return (FVector3d)CollisionData->Vertices[VID]; };
		ResultAdapter.AppendTriangle = [CollisionData](UE::Geometry::FIndex3i T)-> int32
			{
				FTriIndices TI;
				TI.v0 = T.A;
				TI.v1 = T.B;
				TI.v2 = T.C;
				return CollisionData->Indices.Add(TI);
			};
		ResultAdapter.GetTriangle = [CollisionData](int32 TID) -> UE::Geometry::FIndex3i
			{
				FTriIndices TI = CollisionData->Indices[TID];
				return { TI.v0, TI.v1, TI.v2 };
			};

		UE::Geometry::MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
		SimplifyOptions.PreserveEdges.SetSeamConstraints(UE::Geometry::MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free);
		SimplifyOptions.bTransferAttributes = false;
		SimplifyOptions.TargetEdgeLength = TargetEdgeLength;

		bool bRet = UE::Geometry::MeshClusterSimplify::Simplify(InputAdapter, ResultAdapter, SimplifyOptions);
		ensure(bRet);

		Timer.Stop();
		float ReducedVertPct = 100.0f * (float)CollisionData->Vertices.Num() / (float)Vertices.Num();
		float ReducedTriPct = 100.0f * (float)CollisionData->Indices.Num() / (float)Triangles.Num();
		UE_LOGF(LogMegaMesh, Display, "FCustomMeshData: Prepared simplified collision data with edge length %f, %.2f seconds, Initial: %d verts / %d tris, Simplified: %d (%.1f%%) verts / %d (%.1f%%) tris",
			SimplifyOptions.TargetEdgeLength, Time, Vertices.Num(), Triangles.Num(), CollisionData->Vertices.Num(), ReducedVertPct, CollisionData->Indices.Num(), ReducedTriPct);
	}
	else
	{
		// No simplify.  Copy data direct to the output struct.  Positions are reduced to float.
		CollisionData->Vertices.Reserve(Vertices.Num());
		Algo::Transform(Vertices, CollisionData->Vertices, [](const FVector3d& VertexD) { return FVector3f(VertexD); });

		CollisionData->Indices.Reserve(Triangles.Num());
		Algo::Transform(Triangles, CollisionData->Indices, [](const UE::Geometry::FIndex3i& Tri)
			{
				FTriIndices Result;
				Result.v0 = Tri.A;
				Result.v1 = Tri.B;
				Result.v2 = Tri.C;
				return Result;
			});

		UE_LOGF(LogMegaMesh, Verbose, "FCustomMeshData: Prepared collision data. %d verts / %d tris", Vertices.Num(), Triangles.Num());
	}

	CollisionData->bFastCook = true;
	CollisionData->bDisableActiveEdgePrecompute = true;

	return true;
}

FGuid FMeshData::GetVersionKey()
{
	static FGuid VersionKey(TEXT("26ce0266-fdc7-4759-aee9-6692b6b79f93"));
	return VersionKey;
}

} // namespace UE::MeshPartition