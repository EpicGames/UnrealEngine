// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModule.h"
#include "Serialization/CustomVersion.h"

#define LOCTEXT_NAMESPACE "FMeshPartitionModule"

DEFINE_LOG_CATEGORY(LogMegaMesh);

namespace UE::MeshPartition
{

void FMeshPartitionModule::StartupModule()
{
}

void FMeshPartitionModule::ShutdownModule()
{
}

const FGuid MeshPartition::FCustomVersion::GUID(0xAD5A87A3, 0xC33E459C, 0x98FA61D1, 0x4C0D39E4);
FCustomVersionRegistration GRegisterMegaMeshCustomVersion(MeshPartition::FCustomVersion::GUID, MeshPartition::FCustomVersion::LatestVersion, TEXT("MegaMeshVer"));

IMPLEMENT_MODULE(FMeshPartitionModule, MeshPartition)

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
