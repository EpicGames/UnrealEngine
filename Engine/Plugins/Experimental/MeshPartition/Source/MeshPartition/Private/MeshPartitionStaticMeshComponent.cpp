// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionStaticMeshComponent.h"

namespace UE::MeshPartition
{
UMeshPartitionStaticMeshComponent::UMeshPartitionStaticMeshComponent(const FObjectInitializer& ObjectInitializer)
{
	bLumenHeightfield = true;
}

void UMeshPartitionStaticMeshComponent::OnRegister()
{
	Super::OnRegister();
}

void UMeshPartitionStaticMeshComponent::OnUnregister()
{
	Super::OnUnregister();
}
} // namespace UE::MeshPartition