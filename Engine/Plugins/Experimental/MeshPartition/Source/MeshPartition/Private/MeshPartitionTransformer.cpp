// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionTransformer.h"

#include "GameFramework/Actor.h"
#include "MeshPartitionMeshData.h"

namespace UE::MeshPartition
{

namespace MeshPartitionTransformerLocals
{

} // namespace MeshPartitionTransformerLocals

FTransformerUnit MakeTransformerUnit(AActor* InSection, TSharedPtr<const FMeshData> InMesh, bool bInRecomputeNormals, bool bInRecomputeTangents)
{
	return FTransformerUnit{InSection, MoveTemp(InMesh), bInRecomputeNormals, bInRecomputeTangents};
}

AActor* GetSectionChecked(const FTransformerUnit& InTransformerUnit)
{
	AActor* Section = InTransformerUnit.Section.Get();

	ensureMsgf(Section != nullptr, TEXT("MeshPartition: transformer section actor was destroyed mid-build (undo/level-stream/world-browser); skipping unit."));

	return Section;
}

FTransformerContext::~FTransformerContext()
{
	bWasCancelled = true;

	if (JoinTask.IsValid() && !JoinTask.IsCompleted())
	{
		if (IsInGameThread())
		{
			WaitOnGameThread(*this);
		}
		else
		{
			JoinTask.Wait();
		}
	}
}

void WaitOnGameThread(const FTransformerContext& InTransformerContext)
{
	UE::Tasks::FTask WaitTask = UE::Tasks::Launch(TEXT("WaitOnGameThread_Task"), []() {},
												  UE::Tasks::Prerequisites(InTransformerContext.JoinTask),
												  UE::Tasks::ETaskPriority::Normal,
												  UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);

	WaitTask.Wait();
}

} // namespace UE::MeshPartition
