// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionTransformerPipeline.h"

#ifdef WITH_EDITOR
#include "MeshPartitionDependencyInterface.h"
#endif // WITH_EDITOR

namespace UE::MeshPartition
{

#if WITH_EDITOR

void UTransformerPipeline::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	for (const TInstancedStruct<UE::MeshPartition::FTransformer>& Transformer : Transformers)
	{
		if (!Transformer.IsValid())
		{
			continue;
		}

		Transformer.Get().GatherDependencies(InDependencies);
	}
}

#endif // WITH_EDITOR

} // namespace UE::MeshPartition
