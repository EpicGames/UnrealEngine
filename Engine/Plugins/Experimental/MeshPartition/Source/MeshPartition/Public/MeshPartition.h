// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/Actor.h"

#include "MeshPartition.generated.h"

#define UE_API MESHPARTITION_API

namespace UE::MeshPartition
{
class UMeshPartitionComponent;
class UMeshPartitionDefinition;
struct IDependencyInterface;

/**
* Actor class defining globally a MegaMesh.
* This actor contain handles to manipulate a MegaMesh in the editor (translate/rotate/scale).
* It is also the target of external modifiers to express which MegaMesh they are targeting.
*/
UCLASS(MinimalAPI, NotPlaceable)
class AMeshPartition : public AActor
{
	GENERATED_BODY()
	
public:	
	UE_API AMeshPartition();

	// UObject Implementation
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void PostActorCreated() override;

	/** Adds dependencies on actor-level data (transform, etc.) that affect compiled section results. */
	UE_API void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const;
#endif // WITH_EDITOR
	// End UObject Implementation
	
	// AActor Implementation
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;
	// End AActor Implementation

	UMeshPartitionComponent* GetMeshPartitionComponent() const { return MegaMeshComponent; }
	UE_API void SetMeshPartitionComponent(UMeshPartitionComponent* InMeshPartitionComponent);

	UFUNCTION(BlueprintCallable, Category = "Mesh Partition")
	UMeshPartitionDefinition* GetMeshPartitionDefinition() const { return MegaMeshDefinition;	}
	UE_API void SetMeshPartitionDefinition(UMeshPartitionDefinition* InMeshPartitionDefinition);

	// World<->local AABB conversion against this actor's live transform.
	// Convention used by the build pipeline: cell coords / build mesh vertices are in
	// MeshPartition-local space; descriptor bounds and streaming bounds are in world space.
	UE_API FBox WorldToLocal(const FBox& InWorldBounds) const;
	UE_API FBox LocalToWorld(const FBox& InLocalBounds) const;

#if WITH_EDITORONLY_DATA
	// Attach target for transient editor-only section actors (APreviewSection, AInteractiveSection).
	// Attaching temporary section actors keeps these actors' attachments off of undo-tracked state.
	USceneComponent* GetTransientSectionAttachAnchor() const { return TransientSectionAttachAnchor; }
#endif

private:
	void InitializeDefinition();
	
private:
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition", meta=(DisplayName="Mesh Partition Component"))
	TObjectPtr<UMeshPartitionComponent> MegaMeshComponent;

	UPROPERTY(EditAnywhere, Category = "MeshPartition", meta=(DisplayName="Mesh Partition Definition"))
	TObjectPtr<UMeshPartitionDefinition> MegaMeshDefinition;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient)
	TObjectPtr<USceneComponent> TransientSectionAttachAnchor;
#endif
};
} // namespace UE::MeshPartition

#undef UE_API
