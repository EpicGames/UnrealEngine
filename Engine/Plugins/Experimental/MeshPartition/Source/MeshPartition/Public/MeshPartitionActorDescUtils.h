// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModule.h"

class FPropertyPairsMap;
class FWorldPartitionActorDescInstance;

namespace UE::MeshPartition::Utils::ActorDesc
{
#if WITH_EDITOR
	// functions to work with typed properties
	MESHPARTITION_API int32 GetPropertyInt32(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName, int32 DefaultValue);
	MESHPARTITION_API int32 GetPropertyInt32(const FWorldPartitionActorDescInstance& InActorDescInstance, const FName PropertyName, int32 DefaultValue);
	MESHPARTITION_API void SetPropertyInt32(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName, int32 NewValue);

	MESHPARTITION_API int32 IncrementPropertyInt32(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName);

	MESHPARTITION_API void SetPropertyGUID(FPropertyPairsMap& PropertyPairsMap, const FName PropertyName, const FGuid& NewValue);
#endif // WITH_EDITOR
}
