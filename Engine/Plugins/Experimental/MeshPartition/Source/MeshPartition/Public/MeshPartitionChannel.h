// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionChannel.generated.h"

#define UE_API MESHPARTITION_API

namespace UE::MeshPartition
{
struct IDependencyInterface;
class UMeshPartitionDefinition;

/**
* A Channel represents one layer of information supported by the MegaMesh surface.
* It is identified by its Name that must be unique in the context of the MegaMesh.
* 
* All the Channels are declared in the MegaMesh definition through the ChannelMap.
* 
* The ChannelDesc contains the name and extra informations providing ux elements, hints for optimization
*
* The ChannelMap is the collection of Channels.
* It provides the api to access, author and manage the channels in a MegaMesh.
*/

USTRUCT(MinimalAPI)
struct FChannelDesc
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "MeshPartition")
	FName Name = NAME_Default;

	bool operator ==(const FName& InOtherName) const { return Name == InOtherName; }
};


USTRUCT(MinimalAPI)
struct FChannelMap
{
	GENERATED_BODY()

public:
	int32 GetNumChannels() const { return ChannelDescs.Num(); }
	TArrayView<const MeshPartition::FChannelDesc> GetChannelDescs() const { return ChannelDescs; }
	UE_API TArray<FName> GetChannels() const;
	UE_API int32 FindChannel(FName InChannelName) const;

private:
	UPROPERTY(EditAnywhere, Category = "Material", meta = (TitleProperty = "Name"))
	TArray<MeshPartition::FChannelDesc> ChannelDescs;
};

void operator += (MeshPartition::IDependencyInterface& Dependencies, const MeshPartition::FChannelMap& ChannelMap);

/**
 * Interface to constants describing the packing of the channel slots in the channel table embedded in
 * the section primitives and accessed by the material shader
 */
struct FChannelPacking
{
	// Slot is represented by N bits.
	static const uint8 SlotNumBits = 5;  

	// Invalid slot value is all N bits set to 1.
	// SlotInvalid is also the maximum number of representable slot values with N bits
	static const uint8 SlotInvalid = ((uint32)(1 << SlotNumBits) - 1);

	// Number of channel slots of N bits in a 32bits word of the table
	static const uint8 WordNumSlots = 32 / SlotNumBits;

	// Here are the different N Values that we could pick:
	// with N = 3bits =>   7 possible representable values + 1 invalid value (  7), 10 slots per word
	// with N = 4bits =>  15 possible representable values + 1 invalid value ( 15),  8 slots per word
	// with N = 5bits =>  31 possible representable values + 1 invalid value ( 31),  6 slots per word
	// with N = 6bits =>  63 possible representable values + 1 invalid value ( 63),  5 slots per word
	// with N = 7bits => 127 possible representable values + 1 invalid value (127),  4 slots per word
	// with N = 8bits => 255 possible representable values + 1 invalid value (255),  4 slots per word

	// Table is a vec of P words (a word is 32bits).
	static const uint8 TableNumWords = 4;

	// Number of channel slots available in the table  
	static const uint8 TableMaxNumSlots = TableNumWords * WordNumSlots;

	// Maximum number of channels supported with the packing configuration
	static const uint8 MaxNumberPackedChannels = (TableMaxNumSlots >= SlotInvalid ? SlotInvalid : TableMaxNumSlots);

	// In the following table we are indicating the maximum number of packed channels
	// depending on
	//   - the slot N bits representation (in the header row)
	//   - the various possible P number of words (in the header column)
	// +-------------------------+--------+--------+--------+--------+
	// |                       N |    8   |    6   |    5   |    4   |
	// |       [Max index value] |  [255] | [ 63]  | [ 31]  | [ 15]  |
	// +-------------------------+--------+--------+--------+--------+
	// | P => Table size [bytes] |        |        |        |        |
	// |      1   =>    4        |    4   |    5   |    6   |    8 ~ |
	// |      2   =>    8        |    8   |   10   |   12   |   15 * |
	// |      3   =>   12        |   12   |   15   |   18 ~ |        |
	// |      4   =>   16        |   16   |   20   |   24 ~ |        |
	// |      5   =>   20        |   20   |   25   |   30 ~ |        |
	// |      6   =>   24        |   24   |   30   |   31 * |        |
	// |      7   =>   28        |   28   |   35   |        |        |
	// |      8   =>   32        |   32   |   40 ~ |        |        |
	// |      9   =>   36        |   36   |   45 ~ |        |        |
	// |     10   =>   40        |   40   |   50 ~ |        |        |
	// |     11   =>   44        |   44   |   55 ~ |        |        |
	// |     12   =>   48        |   48   |   60 ~ |        |        |
	// |     13   =>   52        |   52   |   63 * |        |        |
	// |         ...             |  ...   |        |        |        |
	// |     17   =>   68        |   68 ~ |        |        |        |
	// |     18   =>   72        |   72 ~ |        |        |        |
	// |         ...             |  ...   |        |        |        |
	// |     63   =>  252        |  252 ~ |        |        |        |
	// |     64   =>  256        |  255 * |        |        |        |
	// +-------------------------+--------+--------+--------+--------+
	// The * cell indicate perfect match between N and P allowing to pack ALL the slot indices possible
	// The ~ cell indicate the best combinaison N and P to pack that many slots value
	//
	// Practically speaking, with P = 4 and N = 5 we can represent 24 channels and thats our best pick for now

	static UE_API void SetCustomPrimitiveData(UPrimitiveComponent* InPrimitiveComponent, TConstArrayView<uint8> InChannelTable, const FVector2f& InChannelTexcoordDesc);
};

/**
 * Helper struct that can be used to wrap an FName so that it can be used as the target of a
 *  detail customization without customizing all FNames. Has implicit conversions to/from
 *  FName so that it can be used as a drop-in replacement.
 */
USTRUCT(MinimalAPI)
struct FNameWrapper
{
	GENERATED_BODY()

public:
	FNameWrapper() = default;
	FNameWrapper(const FName& NameIn) : Name(NameIn) {}

	void SetName(const FName& NameIn) { Name = NameIn; }
	const FName& GetName() const { return Name; }

	//~ Make it simple to drop in the struct as a replacement for FName
	operator const FName& () const { return Name; }
	operator FName& () { return Name; }
	FNameWrapper& operator=(const FName& NameIn)
	{
		SetName(NameIn);
		return *this;
	}
	bool IsNone() const { return Name.IsNone(); }
	bool operator==(const FNameWrapper& Other) { return Name == Other.Name; }

	//~ Deserializes the value from a raw FName if that is what we replaced with this struct. Requires
	//~  a TStructOpsTypeTraits specialization for the derived type.
	UE_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);
private:
	UPROPERTY(EditAnywhere, Category = "Name")
	FName Name;
};

/** 
 * Version of FNameWrapper to use for channel names. Gets used with a detail customization that uses
 *  GetOptions metadata function to constrain options when a toggle is engaged.
 */
USTRUCT(MinimalAPI)
struct FChannelName : public FNameWrapper
{
	GENERATED_BODY()
public:
	FChannelName() = default;
	FChannelName(const FName& NameIn) : FNameWrapper(NameIn) {}
	using FNameWrapper::operator=;
};
} // namespace UE::MeshPartition

// Needed to get SerializeFromMismatchedTag to be called to convert from serialized FName properties
template<>
struct TStructOpsTypeTraits<UE::MeshPartition::FChannelName> : public TStructOpsTypeTraitsBase2<UE::MeshPartition::FChannelName>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true
	};
};

#undef UE_API
