// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Components/PrimitiveComponent.h"
#include "Interfaces/Interface_CollisionDataProvider.h"
#include "PhysicsEngine/BodySetup.h"

#include "MeshPartitionCollisionComponent.generated.h"

class UPhysicalMaterial;

namespace UE::MeshPartition
{
class UMeshPartitionDefinition;


// Container for collision data that can be set on a UMeshPartitionCollisionComponent
USTRUCT()
struct FMeshPartitionCollisionData
{
	GENERATED_BODY()
public:
	TOptional<FTriMeshCollisionData> Mesh;

	UPROPERTY()
	TArray<TObjectPtr<UPhysicalMaterial>> PhysicalMaterials;

	MESHPARTITION_API FMeshPartitionCollisionData();
};

UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent), ClassGroup = Collision, hidecategories = (Object, LOD, Lighting, TextureStreaming))
class UMeshPartitionCollisionComponent : public UPrimitiveComponent, public IInterface_CollisionDataProvider
{
	GENERATED_UCLASS_BODY()
public:

	MESHPARTITION_API virtual UBodySetup* GetBodySetup() override;
	MESHPARTITION_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	MESHPARTITION_API virtual void OnRegister() override;
	MESHPARTITION_API virtual void OnUnregister() override;

#if WITH_EDITORONLY_DATA
	MESHPARTITION_API void SetCollisionData(TSharedPtr<FMeshPartitionCollisionData> InCollisionData, FName CollisionProfileName, bool bRebuild = true);
	MESHPARTITION_API void RebuildIfNeeded(bool bAllowAsyncBuild);
	TSharedPtr<const FMeshPartitionCollisionData> GetMeshCollisionData() const { return CollisionData; }
#endif

	//~ Begin Interface_CollisionDataProvider Interface
	MESHPARTITION_API virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	MESHPARTITION_API virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	MESHPARTITION_API virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	MESHPARTITION_API virtual void GetMeshId(FString& OutMeshId) override;
	//~ End Interface_CollisionDataProvider Interface

private:
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
	MESHPARTITION_API void Serialize(FArchive& Ar);

#if WITH_EDITORONLY_DATA
	UBodySetup* CreateBodySetupHelper();
	void RebuildPhysicsData(bool bAllowAsyncBuild = true);
	void RebuildPhysicalMaterialData();
	void UpdateLocalBounds();
	void AbortPendingPhysicsCook();
	void FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup);

	// return the complex collision, or nullptr if not available
	FTriMeshCollisionData* GetCollisionMesh()
	{
		return CollisionData.IsValid() ? CollisionData->Mesh.GetPtrOrNull() : nullptr;
	}
	const FTriMeshCollisionData* GetCollisionMesh() const
	{
		return CollisionData.IsValid() ? CollisionData->Mesh.GetPtrOrNull() : nullptr;
	}
#endif

	// UPrimitiveComponent materials methods -- only used to drive physical materials
	// TODO: These overrides could be removed if BodyInstance had a way to get complex collision materials without going through render materials
	MESHPARTITION_API virtual int32 GetNumMaterials() const override;
	MESHPARTITION_API virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	
	// Note: Below material set/create methods are BP-exposed on UPrimitiveComponent, but don't make sense for our physics-only materials use case; do not call them
	MESHPARTITION_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	MESHPARTITION_API virtual void SetMaterialByName(FName MaterialSlotName, class UMaterialInterface* Material) override;
	MESHPARTITION_API virtual UMaterialInstanceDynamic* CreateAndSetMaterialInstanceDynamic(int32 ElementIndex) override;
	MESHPARTITION_API virtual UMaterialInstanceDynamic* CreateAndSetMaterialInstanceDynamicFromMaterial(int32 ElementIndex, class UMaterialInterface* Parent) override;
	MESHPARTITION_API virtual UMaterialInstanceDynamic* CreateDynamicMaterialInstance(int32 ElementIndex, class UMaterialInterface* SourceMaterial, FName OptionalName) override;

private:

	// Rendering materials interfaces; used solely as storage for physical materials
	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> PhysicsOnlyMaterials;

	UPROPERTY(Instanced)
	TObjectPtr<UBodySetup> MeshBodySetup;

	UPROPERTY()
	FBox LocalBounds;

#if WITH_EDITORONLY_DATA
	UPROPERTY(Transient, Instanced)
	TObjectPtr<UBodySetup> AsyncBodySetup;  //Body setup being cooked asynchronously

	TSharedPtr<FMeshPartitionCollisionData> CollisionData;
	bool bNeedsRebuild = false;
#endif

#if UE_ENABLE_DEBUG_DRAWING && WITH_EDITORONLY_DATA
	// Represents a UMeshPartitionCollisionComponent to the scene manager.
	friend class FDrawMeshPartitionCollisionSceneProxy;
#endif
	
};


} // namespace UE::MeshPartition