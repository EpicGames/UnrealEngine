// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "MeshPartitionGridSettings.h"
#include "MeshPartitionCompiledSection.generated.h"

#define UE_API MESHPARTITION_API

class UStaticMesh;
class UMaterialInstanceConstant;
class UTexture;
class FWorldPartitionActorDescInstance;
class URuntimeVirtualTexture;

namespace UE::MeshPartition
{
class AMeshPartition;
class UMeshPartitionDefinition;
class UMeshPartitionStaticMeshComponent;
class FCompiledSectionActorDesc;
class UMeshPartitionCollisionComponent;
struct FStaticMeshDescriptor;

/**
* MegaMesh property keys injected in ActorDescs.
*/
namespace MegaMeshCompiledSectionProperties
{
	MESHPARTITION_API extern const FName MegaMeshCompiledSectionVersion;	// Version number
	MESHPARTITION_API extern const int32 CurrentVersion;					// Current Version
	MESHPARTITION_API extern const FName CurrentVersionFName;			// Current Version
} // MegaMeshCompiledSectionProperties

/**
* Describes the build and settings that produced a compiled section
* Used to track invalidation of old sections, and also filtering of different build variants for platforms
*/
USTRUCT()
struct FCompiledSectionBuildInfo
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
	friend class FCompiledSectionActorDesc;
#endif // WITH_EDITORONLY_DATA

public:
	// Uniquely identifies the build job that produced this section.
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition", meta = (NoResetToDefault))
	FGuid BuildKey;

	// Name of the Build Variant that produced this section.
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition", meta = (NoResetToDefault))
	FName BuildVariantName;

	// Definition used to build this section (asset reference)
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FTopLevelAssetPath MegaMeshDefinitionPath;

	// list of ModifierPaths for each base modifier in this compiled section
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	TArray<FSoftObjectPath> BaseModifierPaths;

	// hash of ALL of the Modifiers affecting this compiled section (base modifiers and non-base modifiers) -- used to detect changes to modifier settings
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FGuid ModifiersHash;

	// package names for all asset dependencies, gathered from modifiers affecting this section
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	TArray<FName> PackageDependencies;

	// hash of ALL of the Packages affecting this compiled section (detects if any package has changed)
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FGuid PackageHash;

#if WITH_EDITORONLY_DATA
	// partial hash of each individual package dependency, useful to debug invalidation behaviors
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	TArray<uint32> PackageChecksums;
#endif // WITH_EDITORONLY_DATA

	// hash of ALL Modifier Paths affecting this compiled section -- used by PackageHash to detect when a modifier is added or removed
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FGuid ModifierSetHash;

	// class names for all modifier classes used to construct this compiled section
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	TArray<FName> ClassDependencies;

	// hash of the MegaMeshClassVersion for each class in the list above -- used to detect when modifier class implementations are changed
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FGuid ClassHash;

#if WITH_EDITORONLY_DATA
	// partial hash of each individual class dependency, useful to debug invalidation behaviors
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	TArray<uint32> ClassChecksums;
#endif // WITH_EDITORONLY_DATA

	// hash of the general MegaMeshDefinition and specific BuildVariant settings that produced this compiled section
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FGuid BuildVariantHash;

	// actor desc GUID of the AMegaMesh actor
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FGuid MegaMeshGUID;

	// path to the AMegaMesh actor this was built for
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FSoftObjectPath MegaMeshPath;

	// Sentinel for sections not produced by a grid-aligned split.
	static inline const FIntVector InvalidGridCellCoord = FIntVector(MIN_int32, MIN_int32, MIN_int32);

	// Absolute world-space grid coordinate of this section's cell.
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition", meta = (NoResetToDefault))
	FIntVector GridCellCoord = InvalidGridCellCoord;

	// Grid configuration that produced this section (CellSize=0 means non-grid section).
	// Stamped at section creation by the builder or WorldUpdater so consumers (e.g. SeparateWorldBuilder)
	// don't need to re-resolve from the pipeline.
	// Not Transient: CPF_Transient causes StaticDuplicateObject to skip the property, which breaks
	// PIE world duplication (editor -> PIE actor copy). Regular UPROPERTY survives both duplication
	// and serialization; the default {0, false} is harmless for sections loaded from disk.
	// Not part of identity or equality comparisons.
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FGridSettings GridSettings;

private:
	// transient cache of the mega mesh definition pointer (converted from the MegaMeshDefinitionPath)
	mutable const UMeshPartitionDefinition* MegaMeshDefinition;

public:
	void Serialize(FArchive& Ar); // NOTE: this is only used by the compiled section actor descriptor serialization
	bool operator== (const FCompiledSectionBuildInfo& Other) const;
	MESHPARTITION_API bool TargetsSameCompiledSectionAs(const FCompiledSectionBuildInfo& Other) const;
	MESHPARTITION_API bool SetMegaMeshDefinition(const UMeshPartitionDefinition* Definition);
	MESHPARTITION_API const UMeshPartitionDefinition* GetMegaMeshDefinition() const;
	bool HasValidGridCellCoord() const { return GridCellCoord != InvalidGridCellCoord; }
};

struct FCompiledSectionDescriptor
{
	MeshPartition::FCompiledSectionBuildInfo Info;

	// the world partition actor desc GUID for the compiled section actor
	FGuid ActorDescGuid;

	// the path to this compiled section
	FSoftObjectPath ActorPath;

	// construct from the world partition actor desc properties
	MESHPARTITION_API static bool BuildFromActorDescInstance(const FWorldPartitionActorDescInstance& InActorDescInstance, FCompiledSectionDescriptor& OutCompiledSectionDescriptor);
};

/**
* This is the equivalent of AMegaMeshPreviewSection but used in the final game client.
* Sections are built during the cook process, producing assets (StaticMesh, Collisions, MetaData), and are part of the global MegaMesh.
*/
UCLASS(MinimalAPI)
class ACompiledSection : public AActor
{
	GENERATED_BODY()
	
public:	
	UE_API ACompiledSection();

	// UObject Implementation
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual bool NeedsLoadForTargetPlatform(const class ITargetPlatform* TargetPlatform) const override;
	// End UObject Implementation

	// AActor Implementation
	UE_API virtual void PreRegisterAllComponents() override;
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;
	virtual bool IsRuntimeOnly() const override { return true; }
#if WITH_EDITOR
	UE_API virtual TUniquePtr<FWorldPartitionActorDesc> CreateClassActorDesc() const override;
	UE_API virtual void GetActorDescProperties(FPropertyPairsMap& PropertyPairsMap) const override;
	UE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
	UE_API virtual TArray<UActorComponent*> GetHLODRelevantComponents() const override;
#endif // WITH_EDITOR
	// End AActor Implementation

	TArray<TObjectPtr<UMeshPartitionStaticMeshComponent>>& GetMeshComponents() { return MeshComponents; }
	const TArray<TObjectPtr<UMeshPartitionStaticMeshComponent>>& GetMeshComponents() const { return MeshComponents; }

	TArray<TObjectPtr<UMeshPartitionCollisionComponent>>& GetCollisionComponents() { return CollisionComponents; }
	const TArray<TObjectPtr<UMeshPartitionCollisionComponent>>& GetCollisionComponents() const { return CollisionComponents; }

	UE_API TArray<TObjectPtr<UStaticMesh>> GetStaticMeshes() const;
	UE_API void AddStaticMesh(UStaticMesh* InStaticMesh, const FStaticMeshDescriptor& InDescriptor);

	UE_API void AddCollisionComponent(UMeshPartitionCollisionComponent* InCollisionComponent);

	UE_API UStaticMesh* GetFarFieldMesh() const;
	UE_API void SetFarFieldMesh(UStaticMesh* InStaticMesh);

	UE_API void SetParent(const AMeshPartition* InMegaMesh);
	UE_API AMeshPartition* GetParentMegaMesh() const;

	void SetMaterialInstance(UMaterialInstanceConstant* InMaterialInstance) { MaterialInstance = InMaterialInstance; }
	UMaterialInstanceConstant* GetMaterialInstance() const { return MaterialInstance; }
	UE_API void SetChannelData(TConstArrayView<uint8> InChannelTable, const FVector2f& InTexcoordDesc);
	UE_API void SetChannelTexture(UTexture* InChannelTexture);
	UTexture* GetChannelTexture() { return ChannelTexture; }
	const TArray<uint8>& GetChannelTable() const { return ChannelTable; }
	UE_API void UpdateVirtualTextureSettings();
	UE_API void SetupChannelDataOnChildPrimitiveComponents();
	
	UE_API void SetMaterialCacheTileCount(const FIntPoint& TileCount);
	UE_API void RecreateMaterialCacheTextures();

	/**
	* Get/Set the Build info, describing the build and settings that produced this compiled section
	*/
	const MeshPartition::FCompiledSectionBuildInfo& GetBuildInfo() const { return BuildInfo; }
	void SetBuildInfo(const MeshPartition::FCompiledSectionBuildInfo& InBuildInfo) { BuildInfo = InBuildInfo; }

#if WITH_EDITOR
	void SetIsPlaceholder(bool bInIsPlaceholder) { bIsPlaceholder = bInIsPlaceholder; }
	void SetPlaceholderStreamingBounds(const FBox& Bounds) { check(bIsPlaceholder); PlaceholderStreamingBounds = Bounds; }
	bool IsPlaceholder() const { return bIsPlaceholder; }

	UE_API bool ShouldBeLoadedForPlatform(const class ITargetPlatform* TargetPlatform) const;
#endif // WITH_EDITOR

	UE_API void SetRuntimeVirtualTextures(const TArray<TObjectPtr<URuntimeVirtualTexture>>& InRVTs);

private:
	UE_API void ForAllPrimitiveComponents(TFunctionRef<bool(UPrimitiveComponent*)> InFunc) const;

private:
	UPROPERTY(VisibleAnywhere, Category="MeshPartition")
	TArray<TObjectPtr<UMeshPartitionStaticMeshComponent>> MeshComponents;

	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	TArray<TObjectPtr<UMeshPartitionCollisionComponent>> CollisionComponents;

	/** Fallback component for virtual texture rendering, this is a stop-gap solution, will be removed later */
	UPROPERTY()
	TArray<TObjectPtr<UStaticMeshComponent>> VirtualTextureFallbackMeshComponents;

	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	TObjectPtr<UStaticMeshComponent> FarFieldMeshComponent;

	UPROPERTY()
	TSoftObjectPtr<AMeshPartition> Parent;

	UPROPERTY()
	TObjectPtr<UMaterialInstanceConstant> MaterialInstance;
	
	UPROPERTY(EditAnywhere, Category = "Rendering")
	TObjectPtr<UTexture> ChannelTexture;

	UPROPERTY(VisibleAnywhere, Category = "MeshPartition", meta = (NoResetToDefault))
	MeshPartition::FCompiledSectionBuildInfo BuildInfo;

	UPROPERTY()
	FVector2f ChannelTexcoordDesc;
	
	UPROPERTY()
	TArray<uint8> ChannelTable;
	
	UPROPERTY()
	FIntPoint MaterialCacheTileCount = FIntPoint(1, 1);

	/** All allocated material cache textures */
	UPROPERTY(Experimental)
	TArray<TObjectPtr<UMaterialCacheVirtualTexture>> MaterialCacheTextures;

#if WITH_EDITORONLY_DATA
	// Indicate that this compiled section was created during PIE to be a placeholder for a missing compiled section, that needs to be built on the fly.
	UPROPERTY()
	bool bIsPlaceholder = false;

	UPROPERTY()
	FBox PlaceholderStreamingBounds;
#endif // WITH_EDITOR
};
} // namespace UE::MeshPartition

DEFINE_ACTORDESC_TYPE(UE::MeshPartition::ACompiledSection, UE::MeshPartition::FCompiledSectionActorDesc);

#undef UE_API
