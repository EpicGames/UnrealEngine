// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/ActorPartition/PartitionActorDesc.h"
#include "MeshPartitionCompiledSection.h"

namespace UE::MeshPartition
{
/**
* ActorDesc for MegaMeshCompiledSection
*/
class FCompiledSectionActorDesc : public FWorldPartitionActorDesc
{
public:
	const MeshPartition::FCompiledSectionBuildInfo& GetBuildInfo() const { return BuildInfo; }

	// may return null if the actor desc is an older one before the custom ActorDesc for compiled sections was introduced
	MESHPARTITION_API static const FCompiledSectionActorDesc* GetFromActorDescInstance(const FWorldPartitionActorDescInstance& InActorDescInstance);

protected:
	//~ Begin FWorldPartitionActorDesc Interface.
	MESHPARTITION_API virtual void Init(const AActor* InActor) override;
	MESHPARTITION_API virtual bool Equals(const FWorldPartitionActorDesc* Other) const override;
	virtual uint32 GetSizeOf() const override { return sizeof(MeshPartition::FCompiledSectionActorDesc); }
	MESHPARTITION_API virtual void Serialize(FArchive& Ar) override;
	MESHPARTITION_API virtual const FGuid& GetSceneOutlinerParent() const override;
	//~ End FWorldPartitionActorDesc Interface.


private:
	MeshPartition::FCompiledSectionBuildInfo BuildInfo;
};
} // namespace UE::MeshPartition
#endif
