// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API MESHPARTITION_API

namespace UE::MeshPartition
{
class ACompiledSection;

class IMeshPartitionCompilerInterface
{
	public:
		static UE_API IMeshPartitionCompilerInterface* Get();
		
		/**
		* Called to request a placeholder compiled section to be built.
		* (may be built asynchronously, not immediately before return)
		*/
		virtual void BuildPlaceholderCompiledSection(MeshPartition::ACompiledSection* CompiledSection) = 0;

		/**
		 * Globally register the megamesh compiler used to build compiled sections.
		 * (the compiler is in the editor package, but the placeholder needs to invoke it in runtime code)
		*/
		static UE_API void RegisterMegaMeshCompiler(IMeshPartitionCompilerInterface* Compiler);

		/**
		 * Globally unregister the megamesh compiler used to build compiled sections.
		 */
		static UE_API void UnregisterMegaMeshCompiler(IMeshPartitionCompilerInterface* Compiler);

	protected:
		// destructor protected and non-virtual so that you can't delete an derived class via a pointer to this interface
		UE_API ~IMeshPartitionCompilerInterface();

	private:
		static UE_API IMeshPartitionCompilerInterface* RegisteredCompiler;
};
} // namespace UE::MeshPartition

#undef UE_API
