// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Components/PrimitiveComponent.h"

#include "MeshPartitionComponent.generated.h"

#define UE_API MESHPARTITION_API

namespace UE::MeshPartition
{
class UMeshPartitionDefinition;

/**
* Base class for a component of AMeshPartition actor.
* This is a component to be able to be stripped during the cook process.
* This should contain common interfaces between runtime and editor,
* but also allows to implement the differences only useful for specific modes.
*/
UCLASS(MinimalAPI, ClassGroup=(MeshPartition))
class UMeshPartitionComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:	
	UE_API UMeshPartitionComponent();

	virtual void PostRegisterMegaMeshComponents() {}
	virtual void PostUnregisterMegaMeshComponents() {}

	/**
	* Callback called when the owner's currently used UMeshPartitionDefinition changed.
	*/
	virtual void OnDefinitionChanged(UMeshPartitionDefinition* InNewDefinition) {}
};
} // namespace UE::MeshPartition

#undef UE_API
