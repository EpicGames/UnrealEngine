// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "WorldPartition/WorldPartitionActorDesc.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "MeshPartitionDefinition.h"

#include "MeshPartitionDataLayerContainer.generated.h"

namespace UE::MeshPartition
{

class UMeshPartitionDefinition;

USTRUCT()
struct FPerBuildVariantDataLayers
{
	GENERATED_BODY()

	// Build variant name to data layer.
	UPROPERTY(EditAnywhere, Category = "DataLayer")
	TMap<FName, TObjectPtr<UDataLayerAsset>> Layers;
};

UCLASS(MinimalAPI, NotPlaceable)
class AMeshPartitionDataLayerContainer : public AActor
{
	GENERATED_BODY()
public:
	AMeshPartitionDataLayerContainer();

	MESHPARTITION_API static AMeshPartitionDataLayerContainer* Get(const UWorld* InWorld);
#if WITH_EDITOR
	MESHPARTITION_API static AMeshPartitionDataLayerContainer* GetOrCreate(UWorld* InWorld);

	/**
	 * Update the internal list of data layers based on build variants in the definition.
	 * @return true if the internal data layer list changed.
	 */
	MESHPARTITION_API bool UpdateDataLayersFromDefinition(const UMeshPartitionDefinition* InDefinition);


	MESHPARTITION_API bool IsDataLayerOwnedByContainer(const UDataLayerAsset* InDataLayer) const;
	MESHPARTITION_API bool IsDataLayerRelevantForPlatform(const UDataLayerAsset* InDataLayer, ITargetPlatform* InTargetPlatform) const;

	MESHPARTITION_API TObjectPtr<UDataLayerAsset> FindVariantDataLayer(const UMeshPartitionDefinition* InDefinition, const FName& InBuildVariant) const;

	/**
	 * Creates data layer instances for all internal data layer assets. 
	 * @return true if new data layer instances were created.
	 */
	MESHPARTITION_API bool InitializeDataLayerInstancesWithWorld(const UMeshPartitionDefinition* InDefinition) const;
	MESHPARTITION_API void RemoveDataLayerInstancesFromWorld() const;

	MESHPARTITION_API void ClearAllDataLayers();

	/**
	 * Remove entries for definitions that are no longer in use, plus any entries with null/stale definition keys.
	 * Data layer instances for removed entries are deleted from the world.
	 * @return number of entries removed (0 means the container was not modified).
	 */
	MESHPARTITION_API int32 PruneUnusedDefinitions(const TSet<const UMeshPartitionDefinition*>& InUsedDefinitions);

private:
	UDataLayerAsset* NewVariantDataLayer(const FName& InVariantName);
#endif // WITH_EDITOR

	UPROPERTY(EditAnywhere, Category = "DataLayer")
	TMap<TObjectPtr<const UMeshPartitionDefinition>, MeshPartition::FPerBuildVariantDataLayers> PerDefinitionBuildVariantDataLayers;
};
}