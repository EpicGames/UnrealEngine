// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/DataAsset.h"
#include "Engine/CollisionProfile.h"
#include "Materials/MaterialEnumeration.h"
#include "MeshPartitionChannel.h"
#include "MeshPartitionTransformer.h"
#include "MeshPartitionTransformerPipeline.h"
#include "MeshPartitionUVLayoutMethod.h"
#include "MeshReductionSettings.h"
#include "StructUtils/InstancedStruct.h"
#include "UObject/PerPlatformProperties.h"

#include "MeshPartitionDefinition.generated.h"

#define UE_API MESHPARTITION_API

class UDataLayerAsset;
class UHLODLayer;
class UMaterialInterface;
class UMaterialInstance;
class UPhysicalMaterial;
class URuntimeVirtualTexture;

namespace UE::MeshPartition
{
struct IDependencyInterface;

const extern MESHPARTITION_API FName ChannelTextureParameterName;
const extern MESHPARTITION_API int32 PrimitiveDataIndex;

USTRUCT()
struct FCommonBuildVariant
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "Build Variant")
	double MaxSectionComplexity = 256.0 * 256.0 * 4.0;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = "Pipeline")
    TObjectPtr<MeshPartition::UTransformerPipeline> TransformerPipeline;
#endif

protected:
#if WITH_EDITOR
	friend class UMeshPartitionDefinition;
	void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const;
#endif // WITH_EDITOR
};

USTRUCT()
struct FCompiledSectionBuildVariant : public FCommonBuildVariant
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category="Build Variant")
	FName Name = NAME_Default;

	UPROPERTY(EditAnywhere, Category = "WorldPartition")
	bool bSplitSectionsToMatchWorldPartitionRuntimeGrid = false;

protected:
#if WITH_EDITOR
	friend class UMeshPartitionDefinition;
	void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const;
#endif // WITH_EDITOR
};

USTRUCT()
struct FRuntimeSettings
{
	GENERATED_BODY()

public:
	void PostSerialize(const FArchive& InAr);

public:
	UPROPERTY(EditAnywhere, Category = "Runtime Settings", meta=(GetOptions="GetCompiledSectionBuildVariantOptions"))
	TArray<FName> BuildVariantNames;

	UPROPERTY(meta=(GetOptions="GetCompiledSectionBuildVariantOptions", DeprecatedProperty, DeprecationMessage="Use BuildVariantNames instead"))
	FName BuildVariantName_DEPRECATED;
};

USTRUCT()
struct FPerPlatformRuntimeSettings
#if CPP
	: public TPerPlatformProperty<FPerPlatformRuntimeSettings, MeshPartition::FRuntimeSettings, NAME_StructProperty>
#endif
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = PerPlatform)
	MeshPartition::FRuntimeSettings Default;

#if WITH_EDITORONLY_DATA
	UPROPERTY(EditAnywhere, Category = PerPlatform)
	TMap<FName, MeshPartition::FRuntimeSettings> PerPlatform;
#endif
};

USTRUCT()
struct FPhysicalMaterialChannel
{
	GENERATED_BODY()

public:
	/** The channel to associate with a physical material. */
	UPROPERTY(EditAnywhere, Category = Physics, meta = (GetOptions = "GetChannelNames"))
	MeshPartition::FChannelName ChannelName;

	UPROPERTY(EditAnywhere, Category = Physics)
	TObjectPtr<UPhysicalMaterial> Material;

	UPROPERTY(EditAnywhere, Category = Physics, Meta = (ClampMin = "0", ClampMax = "1", Tooltip = "The minimum weight for a channel to be picked up as the dominant physical layer."))
	float MinimumCollisionRelevanceWeight = 0.0f;
};

/**
* Asset holding shared properties and settings for all components in a MegaMesh.
*
* MegaMeshClassVersion is used to track changes in the behavior of this class with respect to building sections.
* It is used, along with GatherDependencies, when determining whether existing sections are up to date, or need to be rebuilt.
*
* It should be bumped whenever code behavior changes in a way that affects how existing sections would be built.
* NOTE: Changing the MegaMeshClassVersion here invalidates ALL compiled sections (they all use this class).
*/
UCLASS(MinimalAPI, meta=(MegaMeshClassVersion = "1"))
class UMeshPartitionDefinition : public UDataAsset, public IMaterialEnumerationProvider
{
	GENERATED_BODY()

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDefinitionModified, const FName&, const FName&);

public:
	UE_API UMeshPartitionDefinition();
	UE_API virtual ~UMeshPartitionDefinition();

	// UObject Implementation
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
#endif // WITH_EDITOR	
	// End UObject Implementation

	UMaterialInterface* GetMaterial() const { return Material; }

	FOnDefinitionModified& GetOnDefinitionModified() { return OnDefinitionModified; }

	UFUNCTION(BlueprintCallable, Category = "Modifiers")
	const TArray<FName>& GetModifierTypePriorities() const { return ModifierTypePriorities; }

	static bool IsMaterialPropertyName(const FName& InName) { return InName == GET_MEMBER_NAME_CHECKED(UMeshPartitionDefinition, Material); }

	static UE_API const class UMeshPartitionDefinition* GetDefaultMegaMeshDefinition();
	
	static UE_API bool DoesPropertyNameRequireRebuild(const FName& InMemberName, const FName& InName);

	UE_API TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> GetCompiledSectionBuildVariants() const;

	MeshPartition::FChannelMap& GetChannelMap() { return ChannelMap; }
	const MeshPartition::FChannelMap& GetChannelMap() const { return ChannelMap; }
	
	UFUNCTION(CallInEditor)
	TArray<FName> GetChannelNames() const { return ChannelMap.GetChannels(); }

	float GetChannelTexelSize() const { return ChannelTexelSize; }
	float GetMaterialCacheTexelSize() const { return MaterialCacheTexelSize; }
	EChannelCollectionUVLayoutMethod GetChannelUVLayoutMethod() const { return ChannelUVLayoutMethod; }
	UE_EXPERIMENTAL(5.8, "VEUV layout parameters are experimental; the accessor and underlying property may change without deprecation as the algorithm evolves.")
	float GetChannelVEUVSamplesPerSquareMeter() const { return ChannelVEUVSamplesPerSquareMeter; }
	UE_EXPERIMENTAL(5.8, "VEUV layout parameters are experimental; the accessor and underlying property may change without deprecation as the algorithm evolves.")
	FIntVector GetChannelVEUVVoxelCount() const { return ChannelVEUVVoxelCount; }
	EPlaneProjectionNormalSource GetChannelPlaneProjectionNormalSource() const { return ChannelPlaneProjectionNormalSource; }
	FVector GetChannelPlaneProjectionFixedNormal() const { return ChannelPlaneProjectionFixedNormal; }

	const TArray<FPhysicalMaterialChannel>& GetPhysicalMaterialChannels() const { return PhysicalMaterialChannels; }
	UPhysicalMaterial* GetDefaultPhysicalMaterial() const { return DefaultPhysicalMaterial; }

	// Enumerarion Provider Interface
	UE_API virtual bool ResolveValue(FName EntryName, int32& OutValue, int32 DefaultValue = 0) const override;
	UE_API virtual void ForEachEntry(TFunctionRef<void(FName Name, int32 Value)> Iterator) const override;

#if WITH_EDITOR

	// Get the set of build variants to use in PIE for the specified preview platform. If no preview platform is specified, returns the build variant for the current platform.
	UE_API TArray<FName> GetCompiledSectionBuildVariantNamesForCurrentEditorPreview(const FName& InPreviewPlatformName, ITargetPlatform* PreviewPlatform) const;

	// Get the set of build variants needed for the specified target platforms
	UE_API TArray<FName> GetCompiledSectionBuildVariantNamesForPlatforms(TArray<ITargetPlatform*> TargetPlatforms) const;

	// Note that the platform name includes build type like "WindowsClient", while the IniPlatformName is just the platform "Windows".  It looks for PlatformName match first, then if not found it looks for IniPlatformName
	UE_API TArray<FName> GetCompiledSectionBuildVariantNamesForPlatform(const FName& InPlatformName, const FName& InIniPlatformName) const;
	UE_API TArray<FName> GetCompiledSectionBuildVariantNamesForPlatform(const ITargetPlatform* TargetPlatform) const;
	UE_API TArray<FName> GetAllCompiledSectionBuildVariantNames() const;

	UE_API const MeshPartition::FCompiledSectionBuildVariant& GetCompiledSectionBuildVariantByName(const FName& BuildVariantName) const;
	const MeshPartition::FCommonBuildVariant& GetPreviewSectionBuildVariant() const { return PreviewSectionBuildVariant; }
	UE_API void SetPreviewSectionBuildVariant(const MeshPartition::FCommonBuildVariant& InNewPreviewBuildVairant);

	// return a hash of all the settings that affect the compiled section results - FILTERED to only those used by the BuildVariant
	UE_API void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies, FName InBuildVariantName) const;

private:
	UFUNCTION()
	TArray<FString> GetCompiledSectionBuildVariantOptions() const
	{
		// return array of FStrings containing the Name of each BuildVariant
		TArray<FString> Options;
		Options.Add(FString("None"));
		for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : CompiledSectionBuildVariants)
		{
			Options.Add(BuildVariant.Name.ToString());
		}
		return Options;
	}
	
	UE_API void OnObjectModified(UObject* InObject);

#endif // WITH_EDITOR


private:
	FOnDefinitionModified OnDefinitionModified;

private:
	UPROPERTY(EditAnywhere, Category = "Material")
	TObjectPtr<UMaterialInterface> 	Material;

	UPROPERTY(EditAnywhere, Category = "Modifiers", meta=(DisplayName="Modifier Layer Priorities"))
	TArray<FName> ModifierTypePriorities;

	UPROPERTY(EditAnywhere, Category = "Build", meta=(TitleProperty="Name"))
	TArray<MeshPartition::FCompiledSectionBuildVariant> CompiledSectionBuildVariants;

	UPROPERTY(EditAnywhere, Category = "Build|Platforms")
	MeshPartition::FPerPlatformRuntimeSettings PerPlatformRuntimeSettings;

	UPROPERTY(EditAnywhere, Category = "Material", meta=(ShowOnlyInnerProperties))
	MeshPartition::FChannelMap ChannelMap;

	// Channel Texel Size controls the resolution of rasterization of the textures capturing the channel signal.
	// It is the width of an ideal texel in world units.
	// Default is 100 units (1 texel per meter). 
	// A higher number produces coarser channel textures.
	// A smaller number produces more expensive channel textures.
	UPROPERTY(EditAnywhere, Category = "Material", meta=(ClampMin="0.05"))
	float ChannelTexelSize = 100.0; 
	
	UPROPERTY(EditAnywhere, Category = "Material", meta=(ClampMin="0.05"))
	float MaterialCacheTexelSize = 0.25f;

	// Algorithm used to unwrap section UVs when rasterizing channel textures.
	UPROPERTY(EditAnywhere, Category = "Material", meta=(DisplayName="Channel UV Layout Method"))
	EChannelCollectionUVLayoutMethod ChannelUVLayoutMethod = EChannelCollectionUVLayoutMethod::ReferenceBoxProject;

	// Target sample count per square meter of section surface area
	UPROPERTY(EditAnywhere, Category = "Material|Experimental VEUV Controls",
		meta=(EditCondition="ChannelUVLayoutMethod == EChannelCollectionUVLayoutMethod::VolumeEncoded", EditConditionHides,
			ClampMin="0.001", DisplayName="VEUV Samples Per Square Meter"))
	float ChannelVEUVSamplesPerSquareMeter = 0.025f;

	// Voxel grid dimensions
	UPROPERTY(EditAnywhere, Category = "Material|Experimental VEUV Controls",
		meta=(EditCondition="ChannelUVLayoutMethod == EChannelCollectionUVLayoutMethod::VolumeEncoded", EditConditionHides,
			ClampMin="1", ClampMax="16", DisplayName="VEUV Voxel Count"))
	FIntVector ChannelVEUVVoxelCount = FIntVector(4, 4, 1);

	// Source of the projection-plane normal: the section's area-weighted average triangle normal, or a fixed direction below.
	UPROPERTY(EditAnywhere, Category = "Material|Plane Projection",
		meta=(EditCondition="ChannelUVLayoutMethod == EChannelCollectionUVLayoutMethod::PlaneProject", EditConditionHides,
			DisplayName="Plane Projection Normal Source"))
	EPlaneProjectionNormalSource ChannelPlaneProjectionNormalSource = EPlaneProjectionNormalSource::FixedPlane;

	// Fixed plane normal used when ChannelPlaneProjectionNormalSource is FixedPlane. Need not be unit length; normalized at use.
	UPROPERTY(EditAnywhere, Category = "Material|Plane Projection",
		meta=(EditCondition="ChannelUVLayoutMethod == EChannelCollectionUVLayoutMethod::PlaneProject && ChannelPlaneProjectionNormalSource == EPlaneProjectionNormalSource::FixedPlane", EditConditionHides,
			DisplayName="Plane Projection Fixed Normal"))
	FVector ChannelPlaneProjectionFixedNormal = FVector(0.0, 0.0, 1.0);

	UPROPERTY(EditAnywhere, Category = "Material|Physics")
	TArray<FPhysicalMaterialChannel> PhysicalMaterialChannels;

	UPROPERTY(EditAnywhere, Category = "Material|Physics")
	TObjectPtr<UPhysicalMaterial> DefaultPhysicalMaterial;

	UPROPERTY(EditAnywhere, Category = "Preview")
	MeshPartition::FCommonBuildVariant PreviewSectionBuildVariant;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	TArray<MeshPartition::FCompiledSectionBuildVariant> BuildVariants_DEPRECATED;
#endif // WITH_EDITORONLY_DATA

};
} // namespace UE::MeshPartition

template<>
struct TStructOpsTypeTraits<UE::MeshPartition::FRuntimeSettings> : public TStructOpsTypeTraitsBase2<UE::MeshPartition::FRuntimeSettings>
{
	enum
	{
		WithPostSerialize = true
	};
};

#undef UE_API
