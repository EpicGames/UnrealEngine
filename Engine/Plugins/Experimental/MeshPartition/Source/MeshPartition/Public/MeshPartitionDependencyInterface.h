// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#ifdef WITH_EDITOR
#include "UObject/Class.h"
#endif // WITH_EDITOR

class IAssetRegistry;
class UTexture;
class UTexture2D;
class UDynamicMesh;
class UStaticMesh;
class USplineComponent;
class UCurveFloat;

namespace UE::MeshPartition
{
enum class EDependencyFlags
{
	None = 0,							// don't add any sub-dependencies of the specified object or package
	RecurseOnPackageDependencies = 1	// automatically add any package dependencies of the specified object or package (and recurse on those)
};

struct IDependencyInterface
{
	virtual ~IDependencyInterface() {}
	virtual void AddPackageDependency(const UObject* InObject, MeshPartition::EDependencyFlags InDependencyFlags = MeshPartition::EDependencyFlags::None) = 0;
	virtual void AddPackageDependency(TArrayView<UObject*> InObjectArray) = 0;
	virtual void AddClassDependency(const UClass* InClass) = 0;

	// use this archive to do custom serialization for dependent data, if necessary
	virtual FArchive& GetDependentDataArchive() = 0;

	// prebuilt custom dependent data serialization functions for commonly used data
	// these take advantage of existing hashes if possible
	virtual void operator += (const UDynamicMesh& InMesh) = 0;
	virtual void operator += (const UStaticMesh& InMesh) = 0;
	virtual void operator += (const UTexture& InTexture) = 0;
	virtual void operator += (const UTexture2D& InTexture) = 0;
	virtual void operator += (const UCurveFloat& InTexture) = 0;
	virtual void operator += (const USplineComponent& InSpline) = 0;
	virtual void operator += (const UObject& InObject) = 0;

	// pointer dependencies are assumed to be pointing to data dependencies
	template<typename T>
	void operator += (const TObjectPtr<T> InObjectPtr)
	{
		if (InObjectPtr != nullptr)
		{
			*this += (const T&)*InObjectPtr.Get();
		}
	}

	template<typename T>
	void operator += (const T* InDataPtr)
	{
		if (InDataPtr != nullptr)
		{
			*this += (const T&)*InDataPtr;
		}
	}

	template<typename T>
	void operator += (T* InDataPtr)
	{
		if (InDataPtr != nullptr)
		{
			*this += (const T&)*InDataPtr;
		}
	}

	// for any other types, we just serialize it to our dependent data hasher
	template<typename T>
	void operator += (const T& Data)
	{
		// in theory this const cast is safe because we know that our hasher is read only (never writes to Data)
		GetDependentDataArchive() << *const_cast<T*>(&Data);
	}

	template<typename T>
	void operator += (T& Data)
	{
		GetDependentDataArchive() << Data;
	}

#if WITH_EDITOR
	// adds a dependency on the USTRUCT reflected properties of DataStruct.
	// NOTE: this cannot handle reference types, any pointer/ref properties in the struct will result in a compile error.
	template <typename T> void AddUStructDependencyViaReflection(const T& DataStruct)
	{
		check(!GetDependentDataArchive().IsLoading());
		UScriptStruct* ReflectedStruct = T::StaticStruct();
		FStructuredArchiveFromArchive StructedAr(GetDependentDataArchive());
		T DefaultSettings;
		ReflectedStruct->SerializeTaggedProperties(StructedAr.GetSlot(), (uint8*) &DataStruct, ReflectedStruct, (uint8*) &DefaultSettings, nullptr);
	}
#endif // WITH_EDITOR
};
} // namespace UE::MeshPartition