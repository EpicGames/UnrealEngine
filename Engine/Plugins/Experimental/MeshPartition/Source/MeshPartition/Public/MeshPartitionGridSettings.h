// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionGridSettings.generated.h"

namespace UE::MeshPartition
{
struct IDependencyInterface;

/**
 * Grid configuration: cell size, 2D-vs-3D mode, and grid origin offset. Used everywhere
 * a mesh is split on a regular grid -- WP-aligned compiled sections, complexity-driven
 * subsection splits, and PIE placeholders. Hashed in field-declaration order via
 * GatherDependencies; new attributes (padding, ...) extend this struct rather than
 * threading parallel parameters.
 *
 */
USTRUCT(BlueprintType)
struct FGridSettings
{
	GENERATED_BODY()

	// Cell size in unreal units. 0 means "no grid splitting" (single-section path).
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	uint32 CellSize = 0;

	// True when the grid collapses Z into a single column per X/Y cell (matches WP runtime LHGrid Is2D).
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	bool bIs2D = false;

	// World-space anchor of the grid (typically FFixedGridInfo::Origin). Consumers must shift
	// by the MeshPartition actor's LocalToWorld translation before snapping mesh-local bounds --
	// see GridHelpers::ComputeLocalAnchor.
	UPROPERTY(VisibleAnywhere, Category = "MeshPartition")
	FVector WorldOriginOffset = FVector::ZeroVector;

	// True when the grid is configured for grid-aligned splitting (CellSize > 0).
	// CellSize == 0 means "no grid" (single-section path, or WP grid resolution failed).
	bool IsGridSplit() const { return CellSize > 0; }

	MESHPARTITION_API void GatherDependencies(IDependencyInterface& InOutDependencies) const;
};

MESHPARTITION_API FArchive& operator<<(FArchive& Ar, FGridSettings& InOutSettings);
} // namespace UE::MeshPartition
