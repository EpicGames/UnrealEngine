// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"

namespace UE::MeshPartition
{
	MESHPARTITION_API bool IsMaterialCacheEnabled(UWorld* World);
	
	MESHPARTITION_API void UpdateMaterialCacheTextures(
		class USceneComponent* Owner,
		class UMaterialInterface* Material,
		const FIntPoint& TileCount,
		TArray<TObjectPtr<UMaterialCacheVirtualTexture>>& Textures
	);
}
