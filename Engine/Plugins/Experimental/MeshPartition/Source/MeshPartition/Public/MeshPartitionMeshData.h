// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Array.h"
#include "Math/MathFwd.h"
#include "IndexTypes.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "Spatial/MeshAABBTree3.h"
#include "MeshDescription.h"

struct FTriMeshCollisionData;

namespace UE::MeshPartition
{

class FMeshData
{
public:
	static constexpr int16 INVALID_REF_COUNT = -1;
	static constexpr int32 MAX_SOURCE_UV_CHANNELS = 7;
	static constexpr int32 CHANNEL_UV_INDEX = 0;
	static constexpr int32 NUM_CHANNEL_UVS = 1;
	static constexpr int32 SOURCE_UV_OFFSET = 1;

	void Copy(const FMeshData& InOther)
	{
		*this = InOther;
	}

	void Clear()
	{
		Vertices.Empty();
		VertexRefCount.Empty();
		TriangleRefCount.Empty();
		Triangles.Empty();
		Normals.Empty();
		SourceUVChannels.Empty();
		ChannelUVs.Empty();
		WeightLayers.Empty();
		BaseIDLayer.Empty();
		FreeVertices.Empty();
		FreeTriangles.Empty();
	}

	void Reset()
	{
		Vertices.Reset();
		VertexRefCount.Reset();
		TriangleRefCount.Empty();
		Triangles.Reset();
		Normals.Reset();
		SourceUVChannels.Reset();
		ChannelUVs.Reset();
		WeightLayers.Reset();
		BaseIDLayer.Reset();
		FreeVertices.Reset();
		FreeTriangles.Reset();
	}

	struct IndexIterator
	{
		inline bool operator==(const IndexIterator& InOther) const
		{
			return Index == InOther.Index;
		}
		inline bool operator!=(const IndexIterator& InOther) const
		{
			return Index != InOther.Index;
		}
		
		inline int operator*() const
		{
			return this->Index;
		}

		inline IndexIterator& operator++() // prefix
		{
			this->Next();
			return *this;
		}
		inline IndexIterator operator++(int) // postfix
		{
			IndexIterator copy(*this);
			this->Next();
			return copy;
		}
		
		IndexIterator(TConstArrayView<int16> InRefCounts, int InIndex, int InLength)
		: RefCounts(InRefCounts)
		, Index(InIndex)
		, Length(InLength)
		{
			if (Index != Length && RefCounts[Index] == INVALID_REF_COUNT)
			{
				Next();
			}
		}
	private:
		void Next()
		{
			do
			{
				++Index;
			}
			while (Index != Length && RefCounts[Index] == INVALID_REF_COUNT);
		}
		
		const TConstArrayView<int16> RefCounts;
		int Index = 0;
		int Length = 0;
	};
	
	struct FIndexRange
	{
		FIndexRange(TConstArrayView<int16> InRefCounts, int InLength)
		: RefCounts(InRefCounts)
		, Length(InLength)
		{}

		IndexIterator begin()
		{
			return IndexIterator(RefCounts, 0 , Length);
		}
		IndexIterator end()
		{
			return IndexIterator(RefCounts, Length, Length);
		}

		TConstArrayView<int16> RefCounts;
		int Length = 0;
	};

	FIndexRange VertexIndicesItr() const
	{
		return FIndexRange(VertexRefCount, Vertices.Num());
	}

	inline int32 MaxVertexID() const
	{
		return Vertices.Num();
	}
	
	inline FIndexRange TriangleIndicesItr() const
	{
		return FIndexRange(TriangleRefCount, Triangles.Num());
	}
	
	inline void GetTriVertices(int InTriangleID, FVector& InA, FVector& InB, FVector& InC) const
	{
		checkSlow(IsTriangle(InTriangleID));
		Geometry::FIndex3i Triangle = Triangles[InTriangleID];
		InA = Vertices[Triangle.A];
		InB = Vertices[Triangle.B];
		InC = Vertices[Triangle.C];
	}
	
	inline FVector3d GetVertex(int InVertexID) const
	{
		checkSlow(IsVertex(InVertexID));
		return Vertices[InVertexID];
	}

	inline FVector3d& GetVertex(int InVertexID)
	{
		checkSlow(IsVertex(InVertexID));
		return Vertices[InVertexID];
	}
	
	inline void SetVertex(int InVertexID, const FVector3d& InNewVertexPos)
	{
		checkSlow(IsVertex(InVertexID));
		Vertices[InVertexID] = InNewVertexPos;
	}
	
	MESHPARTITION_API int AppendVertex(const FVector3d& InPosition);
	
	inline void RemoveVertex(int InVertexID)
	{
		checkSlow(IsVertex(InVertexID));
		// Removing a vertex which is still part of triangles is an invalid operation.
		ensure(VertexRefCount[InVertexID] == 0);
		VertexRefCount[InVertexID] = INVALID_REF_COUNT;
		FreeVertices.HeapPush(InVertexID);
	}
	
	inline int VertexCount() const
	{
		return Vertices.Num() - FreeVertices.Num();
	}
	
	/** Reserves space in the internal data structures for InAdditionalNum additional vertices. */
	void ReserveAdditionalVertices(SIZE_T InAdditionalNum)
	{
		const SIZE_T CurrentNum = Vertices.Num();

		ReserveGeometric(Vertices, CurrentNum + InAdditionalNum);
		ReserveGeometric(ChannelUVs, CurrentNum + InAdditionalNum);
		ReserveGeometric(VertexRefCount, CurrentNum + InAdditionalNum);
		ReserveGeometric(Normals, CurrentNum + InAdditionalNum);

		for (TArray<FVector2f>& UVChannel : SourceUVChannels)
		{
			ReserveGeometric(UVChannel, CurrentNum + InAdditionalNum);
		}

		for (TPair<FName, TArray<float>>& Channel : WeightLayers)
		{
			ReserveGeometric(Channel.Value, CurrentNum + InAdditionalNum);
		}
	}
	
	inline bool IsVertex(int InVertexID) const
	{
		return InVertexID >= 0 && InVertexID < Vertices.Num() && VertexRefCount[InVertexID] != INVALID_REF_COUNT;
	}

	inline bool HasTriangle(int InVertexID) const
	{
		checkSlow(IsVertex(InVertexID));
		return VertexRefCount[InVertexID] > 0;
	}
	
	inline Geometry::FIndex3i GetTriangle(int InTriangleID) const
	{
		checkSlow(IsTriangle(InTriangleID));
		return Triangles[InTriangleID];
	}

	inline Geometry::FIndex3i& GetTriangle(int InTriangleID)
	{
		checkSlow(IsTriangle(InTriangleID));
		return Triangles[InTriangleID];
	}
	
	inline bool IsTriangle(int TriangleID) const
	{
		return TriangleID >= 0 && TriangleID < Triangles.Num() && TriangleRefCount[TriangleID] != INVALID_REF_COUNT;
	}

	inline int TriangleCount() const
	{
		ensure(Triangles.Num() >= FreeTriangles.Num());
		return Triangles.Num() - FreeTriangles.Num();
	}
	
	inline int MaxTriangleID() const
	{
		return Triangles.Num();
	}
	
	MESHPARTITION_API int AppendTriangle(const Geometry::FIndex3i& InTriangle);

	void RemoveTriangle(int TriangleID, bool bRemoveUnusedVertices)
	{
		const Geometry::FIndex3i& Triangle = Triangles[TriangleID];

		for (int Index = 0; Index < 3; ++Index)
		{
			if (--VertexRefCount[Triangle[Index]] == 0 && bRemoveUnusedVertices)
			{
				RemoveVertex(Triangle[Index]);
			}
		}

		FreeTriangles.HeapPush(TriangleID);
		TriangleRefCount[TriangleID] = INVALID_REF_COUNT;
	}

	/** Reserves space in the internal data structures for InAdditionalNum additional triangles. */
	void ReserveAdditionalTriangles(SIZE_T InAdditionalNum)
	{
		ReserveGeometric(Triangles, Triangles.Num() + InAdditionalNum);
		ReserveGeometric(BaseIDLayer, Triangles.Num() + InAdditionalNum);
		ReserveGeometric(TriangleRefCount, Triangles.Num() + InAdditionalNum);
	}

	inline void SetVertexUV(int InVertexID, const FVector2f& InUV, int32 InUVChannelIndex)
	{
		checkSlow(IsVertex(InVertexID));
		checkSlow(InUVChannelIndex >= 0 && InUVChannelIndex < SourceUVChannels.Num());
		SourceUVChannels[InUVChannelIndex][InVertexID] = InUV;
	}

	inline void SetVertexNormal(int InVertexID, const FVector3f& InNormal)
	{
		checkSlow(IsVertex(InVertexID));
		Normals[InVertexID] = InNormal;
	}

	TArray<FName> GetWeightLayerNames() const
	{
		TArray<FName> Result;
		Result.Reserve(WeightLayers.Num());
		for (const TPair<FName, TArray<float>>& WeightLayer : WeightLayers)
		{
			Result.Add(WeightLayer.Key);
		}
		return Result;
	}

	bool HasWeightLayer(const FName& InWeightLayerName) const
	{
		return WeightLayers.Contains(InWeightLayerName);
	}

	void InitializeWeightLayer(const FName& InWeightLayerName)
	{
		TArray<float>& WeightLayer = WeightLayers.FindOrAdd(InWeightLayerName);
		WeightLayer.SetNumZeroed(Vertices.Num());
	}

	inline TArray<float> GetWeightLayerValues(const FName& InWeightLayerName, TConstArrayView<int> InVertexIDs) const
	{
		TArray<float> Result;
		const TArray<float>* GlobalWeightChannel = WeightLayers.Find(InWeightLayerName);
		check(GlobalWeightChannel);

		for (int VertexID : InVertexIDs)
		{
			checkSlow(IsVertex(VertexID));
			Result.Add((*GlobalWeightChannel)[VertexID]);
		}

		return Result;
	}

	void SetWeightLayerValues(const FName& InWeightLayerName, TConstArrayView<int> InVertexIDs, TConstArrayView<float> InWeightLayerValues)
	{
		TArray<float>* GlobalWeightChannel = WeightLayers.Find(InWeightLayerName);
		if (!ensureMsgf(GlobalWeightChannel, TEXT("Missing expected weight channel: %s"), *InWeightLayerName.ToString()))
		{
			return;
		}

		for (int Index = 0; Index < InVertexIDs.Num(); ++Index)
		{
			const int VertexID = InVertexIDs[Index];
			checkSlow(IsVertex(VertexID));
			(*GlobalWeightChannel)[VertexID] = InWeightLayerValues[Index];
		}
	}

	void SetWeightLayerValue(const FName& InWeightLayerName, int InVertexID, float Value)
	{
		checkSlow(IsVertex(InVertexID));
		WeightLayers.FindChecked(InWeightLayerName)[InVertexID] = Value;
	}
	
	const TArray<float>& GetWeightLayerValues(const FName& InWeightLayerName) const
	{
		return WeightLayers.FindChecked(InWeightLayerName);
	}

	float GetWeightLayerValue(const FName& InWeightLayerName, int InVertexID) const
	{
		checkSlow(IsVertex(InVertexID));
		return WeightLayers.FindChecked(InWeightLayerName)[InVertexID];
	}

	int GetBaseID(int InTriangleID) const
	{
		checkSlow(IsTriangle(InTriangleID));
		return BaseIDLayer[InTriangleID];
	}

	void SetBaseID(int InTriangleID, int InBaseID)
	{
		checkSlow(IsTriangle(InTriangleID));
		BaseIDLayer[InTriangleID] = InBaseID;
	}

	MESHPARTITION_API void RecomputeNormals(const bool bRequireDeterministicNormals = true);
	void RecomputeTangents()
	{
		ensureMsgf(false, TEXT("Computing tangents for the custom mesh format is not yet supported"));
	}
	
	MESHPARTITION_API void SummarizeUVRegion();
	
	FBox2f GetUVRegion() const
	{
		return UVRegion;
	}

	FVector3f GetVertexNormal(int InVertexID) const
	{
		checkSlow(IsVertex(InVertexID));
		return Normals[InVertexID];
	}

	MESHPARTITION_API FVector3d GetTriNormal(int InTriangleID) const;

	inline FVector2f GetVertexUV(int InVertexID, int32 InUVChannelIndex) const
	{
		checkSlow(IsVertex(InVertexID));
		checkSlow(InUVChannelIndex >= 0 && InUVChannelIndex < SourceUVChannels.Num());
		return SourceUVChannels[InUVChannelIndex][InVertexID];
	}

	inline FVector2f GetChannelUV(int InVertexID) const
	{
		checkSlow(IsVertex(InVertexID));
		return ChannelUVs[InVertexID];
	}

	inline void SetChannelUV(int InVertexID, const FVector2f& InUV)
	{
		checkSlow(IsVertex(InVertexID));
		ChannelUVs[InVertexID] = InUV;
	}

	inline int32 GetNumSourceUVChannels() const
	{
		return SourceUVChannels.Num();
	}
	
	inline int32 GetNumUVChannels() const
	{
		return NUM_CHANNEL_UVS + GetNumSourceUVChannels();
	}

	void SetNumSourceUVChannels(int32 InNumChannels)
	{
		const int32 ClampedNumChannels = FMath::Clamp(InNumChannels, 1, MAX_SOURCE_UV_CHANNELS);
		SourceUVChannels.SetNum(ClampedNumChannels);

		for (TArray<FVector2f>& UVChannel : SourceUVChannels)
		{
			UVChannel.SetNumZeroed(Vertices.Num());
		}
	}

	void GrowSourceUVChannelsTo(int32 InNumChannels)
	{
		if (InNumChannels <= SourceUVChannels.Num())
		{
			return;
		}

		SetNumSourceUVChannels(InNumChannels);
	}

	inline int32 GetChangeStamp() const
	{
		// This is not actually ever used, it's just required for MeshAABBTree
		return 0;
	}

	bool CheckValidity() const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshData::CheckValidity);

		for (int VertexID : VertexIndicesItr())
		{
			if (!ensure(IsVertex(VertexID))) return false;
			if (!ensure(!FreeVertices.Contains(VertexID))) return false;
		}

		for (int TriangleID : TriangleIndicesItr())
		{
			if (!ensure(IsTriangle(TriangleID))) return false;
			if (!ensure(!FreeTriangles.Contains(TriangleID))) return false;
			Geometry::FIndex3i Triangle = GetTriangle(TriangleID);
			for (int Index = 0; Index < 3; ++Index)
			{
				if (!ensure(IsVertex(Triangle[Index]))) return false;
			}
		}
		return true;
	}

	MESHPARTITION_API Geometry::FAxisAlignedBox3d GetBounds(bool bParallel = true) const;

	friend FArchive& operator<<(FArchive& Ar, FMeshData& Mesh)
	{
		Mesh.Serialize(Ar);
		return Ar;
	}

	MESHPARTITION_API void Serialize(FArchive& Ar);

	/**
	* Merge pairs of vertices, mapping all indices in triangles from the discarded vertex to the kept vertex.
	* Any attributes on the discarded vertex are simply discarded and only the attributes on the kept vertex remain.
	*
	* This function is optimized for performance and does not support or validate that the list
	* of merge pairs does not contain duplicates, or that the same vertex does not map to two separate kept vertices.
	* These constraints are left up to the caller to ensure.
	*
	* @param InMergePairs List of pairs of merges in the format {KeepVID, DiscardVID}.
	* @param InTriangleFilter Optional parameter which reduces the subset of triangles which could possibly be modified
	*        when fixing triangle indices. This improves the performance as the VID -> TriangleID map is restricted.
	*/
	MESHPARTITION_API void MergeVertexPairs(TConstArrayView<TPair<int, int>> InMergePairs, TOptional<TConstArrayView<int>> InTriangleFilter);

	/**
	* Merges a dynamic mesh into this Mega Mesh, transfering all uvs, normals, and weight channels.
	*
	* @param InNewBaseID optional parameter. If valid, new triangles appended will have this base id assigned to them.
	* @param OutMergeBoundaryVertices Optional output parameter. If non-null, all boundary vertices from the input mesh are added to the set.
	*        This is useful since the primary purpose of this function is to merge base meshes, and callers may want to use these vertices to apply
	*        a weld step after appending. Since this structure doesn't store explicit edge information, using the input dynamic mesh to derive a set
	*        of edge boundary vertices allows the weld step to be fast as it doesn't need to recompute any of this information.
	*/
	MESHPARTITION_API void AppendDynamicMesh(const Geometry::FDynamicMesh3& InOtherMesh, const FTransform& InRelativeTransform, int InNewBaseID = INDEX_NONE, TSet<int>* OutMergeBoundaryVertices = nullptr);

	/**
	* Weld coincident vertices from within a subset of mesh vertices.
	* Only considers vertices which are within the passed set and only welds vertices from different sources.
	* @param InVertexSourcePairs pairs of {VertexID, SourceID}.
	*/
	MESHPARTITION_API void WeldCoincidentVertices(TConstArrayView<TPair<int, int>> InVertexSourcePairs);

	MESHPARTITION_API void ConvertToDynamicMesh(Geometry::FDynamicMesh3& OutDynamicMesh) const;
	MESHPARTITION_API void ConvertToMeshDescription(FMeshDescription& OutMeshDescription) const;

	MESHPARTITION_API bool ConvertToTriMeshCollisionData(FTriMeshCollisionData* CollisionData) const;

	/** Returns a static version key for this mesh format. Increment this if the serialized mesh data format changes. */
	MESHPARTITION_API static FGuid GetVersionKey();

	SIZE_T GetByteCount() const
	{
		SIZE_T WeightLayersByteCount = 0;
		for (const TPair<FName, TArray<float>>& Pair : WeightLayers)
		{
			WeightLayersByteCount += Pair.Value.GetAllocatedSize();
		}

		SIZE_T SourceUVsByteCount = 0;
		for (const TArray<FVector2f>& UVChannel : SourceUVChannels)
		{
			SourceUVsByteCount += UVChannel.GetAllocatedSize();
		}

		return Vertices.GetAllocatedSize()
			+ VertexRefCount.GetAllocatedSize()
			+ TriangleRefCount.GetAllocatedSize()
			+ Triangles.GetAllocatedSize()
			+ Normals.GetAllocatedSize()
			+ SourceUVsByteCount
			+ ChannelUVs.GetAllocatedSize()
			+ WeightLayersByteCount
			+ FreeVertices.GetAllocatedSize()
			+ FreeTriangles.GetAllocatedSize()
			+ sizeof(UVRegion);
	}

protected:
	// Reserves additional elements in an array while respecting geometric growth factors.
	// This ensures that calling Reserve incrementally while appending data does not invoke
	// a new memory allocation every single time. This has a measurable performance impact
	// when a lot of topology modifiers are inserting new geometry into the mesh.
	template <typename T>
	static void ReserveGeometric(TArray<T>& InArray, typename TArray<T>::SizeType InNewSize)
	{
		typename TArray<T>::SizeType Max = InArray.Max();
		Max = Max != 0 ? Max : 16;
		if (Max <= InNewSize)
		{
			while (Max < InNewSize)
			{
				Max *= 2;
			}
			InArray.Reserve(Max);
		}
	}

private:
	TArray<FVector3d> Vertices;

	// Ref count keeping track of number of triangles which reference each vertex
	// this is used to determine if a vertex should be removed if the last owning triangle
	// is removed.
	TArray<int16> VertexRefCount;

	TArray<Geometry::FIndex3i> Triangles;

	TArray<FVector3f> Normals;

	// Source UVs from imported mesh (passed through modifier pipeline)
	TArray<TArray<FVector2f>> SourceUVChannels;

	// Auto-generated channel rendering UVs (separate, not passed to modifiers)
	TArray<FVector2f> ChannelUVs;

	TMap<FName, TArray<float>> WeightLayers;
	
	FBox2f UVRegion = FBox2f(FVector2f::Zero(), FVector2f::One());

	// per triangle
	TArray<int> BaseIDLayer;
	// Refcount which acts as a boolean if this triangle is valid or not. INVALID_REF_COUNT if unused, 1 otherwise.
	TArray<int16> TriangleRefCount;

	TArray<int> FreeVertices;
	TArray<int> FreeTriangles;
};

using FMeshABBTree3 = Geometry::TMeshAABBTree3<FMeshData>;

// Simple mesh data representation used during the generation of the channel textures
// Used by the Preview Section in order to inspect the channel texture.
struct FSectionChannelGenerationMeshData
{
	TArray<int32>			Indices;
	TArray<FVector2f>		Texcoords;
	TArray<float>			Channels;
	TArray<FInt32Vector3>	Outlines;
	TArray<int32>			UVElementToVertexID;
	int32 					VertexCount;
};
	
struct FSectionChannelDomainData
{
	double Area3D;
	double AreaUV;
	double GutterUV;
};

} // namespace UE::MeshPartition
