// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "MeshPartitionModifierBlueprintInterfaces.generated.h"

namespace UE::MeshPartition
{
UINTERFACE(NotBlueprintable)
class UModifierBlueprintInterface : public UInterface
{
	GENERATED_BODY()
};


/**
* This interface exists in a non-editor module to allow some functions to be called in
*  blueprints that do not derive from an editor-only class.
*/
class IModifierBlueprintInterface
{
	GENERATED_BODY()

public:
	/**  */
	UFUNCTION(BlueprintCallable, Category = "MeshPartitionModifier", meta = (DisplayName = "Set Affected Mesh Partition"))
	virtual void BP_SetAffectedMegaMesh(AMeshPartition* InMegaMesh) = 0;
};


UINTERFACE(NotBlueprintable)
class USplineModifierBlueprintInterface : public UInterface
{
	GENERATED_BODY()
};

class ISplineModifierBlueprintInterface
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category = "Spline", meta = (DisplayName = "Set Spline Component"))
	virtual void BP_SetSplineComponent(USplineComponent* InSplineComponent) = 0;
};


} // namespace UE::MeshPartition