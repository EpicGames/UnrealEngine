// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMegaMesh, Log, All);

namespace UE::MeshPartition
{
class FMeshPartitionModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

struct FCustomVersion
{
	enum Type
	{
		BeforeCustomVersionWasAdded = 0,
		AddedMeshPartitionPathToBuildInfo = 1,
		// Default priority layer for mesh terrain modifiers was set to None
		DefaultPriorityLayerSetToNone = 2,
		// Default priority layer for sculpt modifier (which was missed above) set to None
		SculptPriorityLayerSetToNone = 3,
		// Split FCompiledSectionBuildVariant from FCommonBuildVariant so it can be used for defining preview builds
		BuildVariantCleanup = 4,
		// Added debug checksums to the package and class hashes in the compiled section build infos
		AddedChecksumsToBuildInfo = 5,
		// Added GridCellCoord (FIntVector) and ResolvedGridCellSize to FCompiledSectionBuildInfo for WP-grid-aligned sections
		AddedGridCellCoordToBuildInfo = 6,
		// Added a virtual method to identify modifiers as bases instead of relying on a named "Base" priority layer.
		VirtualIsBaseModifier = 7,
		// Store the component to actor relative transform in component descriptors rather than the component to world transform.
		ComponentToActorTransform = 8,
		// Added bIs2D flag to FCompiledSectionBuildInfo::GridSettings so 2D-grid-aware splits round-trip through serialization.
		Added2DFlagToBuildInfo = 9,
		// Added FVector Origin to FCompiledSectionBuildInfo::GridSettings so WP-grid Origin offsets are honored during snap.
		AddedGridOriginToBuildInfo = 10,
		// Add more versions as needed

		LatestVersionPlusOne,
		LatestVersion = LatestVersionPlusOne - 1
	};

	MESHPARTITION_API const static FGuid GUID;
};
} // namespace UE::MeshPartition
