// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/DeveloperSettings.h"
#include "MeshPartitionSettings.generated.h"

#define UE_API MESHPARTITION_API

namespace UE::MeshPartition
{

class UMeshPartitionDefinition;

UCLASS(config = MeshPartition, defaultconfig, meta = (DisplayName = "Mesh Partition"), MinimalAPI)
class USettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_API TSoftObjectPtr<MeshPartition::UMeshPartitionDefinition> GetDefaultDefinition() const { return DefaultDefinition; }

protected:
	UPROPERTY(EditAnywhere, config, Category = "Definition", meta = (ToolTip = "Default Definition that will be used when creating new mesh partition actors."))
	TSoftObjectPtr<MeshPartition::UMeshPartitionDefinition> DefaultDefinition;
};

} // namespace UE::MeshPartition

#undef UE_API