// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/StaticMeshComponent.h"
#include "MeshPartitionStaticMeshComponent.generated.h"

#define UE_API MESHPARTITION_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI)
class UMeshPartitionStaticMeshComponent : public UStaticMeshComponent
{
	GENERATED_BODY()

public:
	UE_API UMeshPartitionStaticMeshComponent(const FObjectInitializer& ObjectInitializer);

protected:
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
};
} // namespace UE::MeshPartition

#undef UE_API
