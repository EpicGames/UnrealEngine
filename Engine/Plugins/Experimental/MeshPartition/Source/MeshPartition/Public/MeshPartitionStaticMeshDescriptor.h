// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace UE::MeshPartition
{
	struct FStaticMeshDescriptor
	{
		FName CollisionProfileName = NAME_None;
		
		bool bCanEverAffectNavigation = true;
		
		FBox2f UVRegion = FBox2f(FVector2f::Zero(), FVector2f::One());
	};
}
