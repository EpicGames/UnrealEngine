// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MeshPartitionMeshData.h"
#include "Tasks/Task.h"
#include "StructUtils/InstancedStruct.h"

#include "MeshPartitionTransformer.generated.h"

#define UE_API MESHPARTITION_API

namespace UE::MeshPartition
{

class UMeshPartitionDefinition;
class UMeshPartitionEditorComponent;
class UPreviewMeshComponent;
struct IDependencyInterface;
struct FCommonBuildVariant;
struct FTransformer;

struct FTransformerUnit
{
	TWeakObjectPtr<AActor> Section;
	TSharedPtr<const MeshPartition::FMeshData> MeshData;
	bool bShouldRecomputeNormals = false;
	bool bShouldRecomputeTangents = false;
};

/** Construct a transformer unit from a section actor and its mesh data. */
UE_API FTransformerUnit MakeTransformerUnit(AActor* InSection, TSharedPtr<const FMeshData> InMesh, bool bInRecomputeNormals = true, bool bInRecomputeTangents = true);

/** Resolve the section actor from a transformer unit. */
UE_API AActor* GetSectionChecked(const FTransformerUnit& InTransformerUnit);

struct FTransformerContext
{
	UE_API ~FTransformerContext();

	TArray<TInstancedStruct<MeshPartition::FTransformer>> Transformers;
	TArray<MeshPartition::FTransformerUnit> TransformerUnits;
	UE::Tasks::FTask JoinTask;
	bool bWasCancelled = false;
};

void UE_API WaitOnGameThread(const FTransformerContext& InTransformerContext);

/**
 * Parent struct for mesh partition transformers.
 * Transformers are pipeline processors being executed sequentially to produce derived data associated with sections.
 */
USTRUCT()
struct FTransformer
{
	GENERATED_BODY()

	virtual ~FTransformer() = default;

	/**
	 * After intantiating the transformer. Initializes it with the definition in use.
	 * Useful if the transformer needs to capture members from the definition.
	 * @param InDefinition The definition in use.
	 * @param InVariant The build variant in use.
	 */
	virtual void Initialize(const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InVariant) {}

	/**
	 * Only used when building a preview section. Executes a transformer on a preview mesh component.
	 * This is a synchronous operation and should only be used to be able to render properly the preview mesh.
	 * Any expensive operation should be delayed until building proper derived data.
	 * @param InPreviewMeshComponent The preview component the transformer will be executed on.
	 * @param InDefinition The definition in use.
	 * @param InBuildVariant The build variant in use.
	 */
	virtual bool Execute(MeshPartition::UPreviewMeshComponent& InPreviewMeshComponent, const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InBuildVariant) const { return true; }

	/**
	 * Executes a transformers's payload. The execute method will be launched as a task by the editor component.
	 * @param InTransformerContext The context associated with this transformer.
	 * @return True if the execution succeeds.
	 */
	virtual bool Execute(MeshPartition::FTransformerContext& InTransformerContext) const { return false; }

	virtual void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const { return; }
};

} // namespace UE::MeshPartition

#undef UE_API
