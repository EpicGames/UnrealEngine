// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/DataAsset.h"
#include "StructUtils/InstancedStruct.h"
#include "MeshPartitionTransformer.h"

#include "MeshPartitionTransformerPipeline.generated.h"

#define UE_API MEGAMESH_API

namespace UE::MeshPartition
{

struct IDependencyInterface;

/**
 * Asset describing a transformer pipeline. Can be used in mesh partition definition variants.
 */
UCLASS(MinimalAPI)
class UTransformerPipeline : public UDataAsset
{
	GENERATED_BODY()

public:
	const TArray<TInstancedStruct<MeshPartition::FTransformer>>& GetTransformers() const { return Transformers;	}
	
#ifdef WITH_EDITOR
	void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const;
#endif // WITH_EDITOR

private:
	UPROPERTY(EditAnywhere, Category = "Pipeline", Meta = (ExcludeBaseStruct))
	TArray<TInstancedStruct<MeshPartition::FTransformer>> Transformers;
};

} // namespace UE::MeshPartition

#undef UE_API
