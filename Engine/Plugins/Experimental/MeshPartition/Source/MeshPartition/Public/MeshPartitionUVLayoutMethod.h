// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "MeshPartitionUVLayoutMethod.generated.h"

namespace UE::MeshPartition
{

UENUM()
enum class EChannelCollectionUVLayoutMethod : uint8
{
	// A fast box-projection heuristic. May produce visible artifacts.
	FastBoxProject			= 1		UMETA(Hidden),
	// A reference box-projection implementation.
	ReferenceBoxProject		= 2,
	// Volume-encoded UVs produced by the VEUV plugin.
	VolumeEncoded			= 3		UMETA(DisplayName = "Volume Encoded (VEUV)"),
	// Project all triangles onto a single plane.
	PlaneProject			= 4		UMETA(DisplayName = "Plane Project"),
};

// Source of the projection-plane normal for the PlaneProject UV layout method.
UENUM()
enum class EPlaneProjectionNormalSource : uint8
{
	// Use the area-weighted average of the section's triangle normals.
	AverageNormal,
	// Use a fixed normal supplied by the asset.
	FixedPlane,
};

} // namespace UE::MeshPartition
