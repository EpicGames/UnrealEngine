// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionChannel.h" // MeshPartition::FChannelName
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

class UCurveFloat;

#include "MeshPartitionProjectModifierTypes.generated.h"

namespace UE::MeshPartition
{
//~ The types in this file need to be in a non-editor-only module to be usable by the non-editor-only
//~  PCG module, because UHT currently does not editor-only types to be referenced via a UPROPERTY
//~  from non-editor-only modules, even if the property is inside a WITH_EDITORONLY_DATA block.

UENUM(BlueprintType)
enum class EProjectModifierFalloffMode : uint8
{
	Linear,
	Smooth,
	CustomCurve
};

UENUM(BlueprintType)
enum class EProjectModifierBlendMode : uint8
{
	/** 
	* Only allow moving vertices in the positive Z direction in projection space,
	*  i.e. in the opposite direction of the projection raycast.
	*/
	Raise,
	/**
	* Only allow moving vertices in the negative Z direction in projection space,
	*  i.e. in the same direction as the projeciton raycast.
	*/
	Lower,
	/**
	* Raise or lower the vertices as needed to get to the projected position.
	*/
	Set
};

UENUM()
enum class EProjectModifierChannelSourceMode : uint8
{
	VertexColor,
	VertexWeight
};

USTRUCT()
struct FProjectModifierFalloffSettings
{
	GENERATED_BODY()

private:
	/**
	* How effects of the modifier are decreased closer to the edges of the 2d projected bounds of the mesh.
	*/
	UPROPERTY(EditAnywhere, Category = Falloff)
	MeshPartition::EProjectModifierFalloffMode Mode = MeshPartition::EProjectModifierFalloffMode::Smooth;

	/**
	* Distance (in projected space) across which to fall off the modifier effects, applied inwards from the 2d
	*  rectangular bounds of the mesh projection, and only relevant where there was a potential value to write
	*  (e.g. a projection mesh was hit).
	*/
	UPROPERTY(EditAnywhere, Category = Falloff, meta = (ClampMin = "0", UIMax = "2000"))
	float Distance = 0;

	/**
	* When not 0, the 2d projected bounds used for falloff are considered to have rounded corners (clamped to
	*  half of the smallest dimension). This means that the bounds will be circular if the projected bounds
	*  were square and the corner radius is higher than half the width.
	*/
	UPROPERTY(EditAnywhere, Category = Falloff, meta = (ClampMin = "0", UIMax = "2000"))
	float CornerRadius = 0;

	UPROPERTY(EditAnywhere, Category = Falloff, meta = (
		EditCondition = "Mode == EProjectModifierFalloffMode::CustomCurve", EditConditionHides))
	TObjectPtr<UCurveFloat> FalloffCurve = nullptr;

	FDelegateHandle FalloffCurveListenerHandle;

	friend class UMeshProjectModifier;
	friend class UInstancedProjectionModifier;
	friend class FPCGProjectionSpawnerElement;
};

USTRUCT()
struct FProjectModifierWeightEntry
{
	GENERATED_BODY()

private:

	/** The destination attribute channel to write. */
	UPROPERTY(EditAnywhere, Category = Weight, meta = (GetOptions = "GetMegaMeshDefinitionChannels"))
	MeshPartition::FChannelName ChannelName;

	/** Determines where the values to write are acquired. */
	UPROPERTY(EditAnywhere, Category = Weight)
	MeshPartition::EProjectModifierChannelSourceMode SourceMode = MeshPartition::EProjectModifierChannelSourceMode::VertexColor;

	/** Which channel of a vertex color to use as source data (0 through 3 for RGBA) */
	UPROPERTY(EditAnywhere, Category = Weight, meta = (ClampMin = "0", ClampMax = "3",
		EditCondition = "SourceMode == EProjectModifierChannelSourceMode::VertexColor", EditConditionHides))
	int32 VertexColorIndex = 0;

	/** Vertex color channel name to use for the source data */
	UPROPERTY(EditAnywhere, Category = Weight, meta = (
		EditCondition = "SourceMode == EProjectModifierChannelSourceMode::VertexWeight", EditConditionHides))
	FName SourceWeightChannelName;

	UPROPERTY(EditAnywhere, Category = Weight)
	MeshPartition::EProjectModifierBlendMode BlendMode = MeshPartition::EProjectModifierBlendMode::Set;

	/** If set, the falloff settings for attributes will be different from the falloff settings used for positions */
	UPROPERTY(EditAnywhere, Category = Weight, Meta = (ShowOnlyInnerProperties))
	TOptional<MeshPartition::FProjectModifierFalloffSettings> FalloffOverrides;

	/**
	* Only relevant when Raise or Lower blend mode is used for positions. When true, the weights are not applied in
	*  areas where the position data was ignored due to being above/below the current values in Raise/Lower blend mode.
	*/
	UPROPERTY(EditAnywhere, Category = Weight)
	bool bApplyHeightMinMaxBlend = false;

	/**
	* When using "Apply Height Min Max Blend," this parameter sets the range of heights across which the weights
	* will blend. If this parameter is equal to softness parameter, then heights blended by the softness parameter
	* will also have their weights linearly blended.
	*/
	UPROPERTY(EditAnywhere, Category = Weight, meta = (ClampMin = "0", UIMax = "1000",
		EditCondition = "bApplyHeightMinMaxBlend", EditConditionHides))
	double HeightMinMaxBlendDistance = 0.0;

	friend class UMeshProjectModifier;
	friend class UInstancedProjectionModifier;
	friend class FPCGProjectionSpawnerElement;
};
} // namespace UE::MeshPartition