// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionRemeshModifierTypes.generated.h"

UENUM()
enum class EMegaMeshRemeshModifierBoundaryMode : uint8
{
	// The boundaries of the covered region are not modified in any way. If their edges are
	//  much longer than the target remeshed edges, this could result in sliver triangles,
	//  since the interior vertices still need to connect to long edges.
	FullyConstrained,

	// Allow the boundary edges to be split, but otherwise do not move them.
	SplitOnly,

	// Allow the boundary to move freely, including by the smoothing steps.
	Free,
};

UENUM()
enum class EMegaMeshRemeshModifierTessellateMethod : uint8
{
	/** GLSL-style tessellation, uniformly increases resolution on all triangles within Modifier bounds */
	UniformRings UMETA(DisplayName = "Uniform Rings (GLSL-style)"),

	/** GLSL-style tessellation, resolution increases gradually towards Modifier center */
	AdaptiveRings UMETA(DisplayName = "Adaptive Rings (GLSL-style)"),

	/** Adaptive Red-Green tessellation, resolution increases gradually towards Modifier center */
	AdaptiveRegular  UMETA(DisplayName = "Adaptive Regular (Red Green)"),
};
