// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshPartitionCompute : ModuleRules
{
	public MeshPartitionCompute(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		
		PublicIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"RenderCore",
			}
			);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Projects"
			}
			);
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
