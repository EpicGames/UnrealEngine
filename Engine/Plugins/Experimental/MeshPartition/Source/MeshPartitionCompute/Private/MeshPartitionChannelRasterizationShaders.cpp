// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionChannelRasterizationShaders.h"

IMPLEMENT_GLOBAL_SHADER(FMeshPartition_DrawUVDomainVS, "/Plugin/MeshPartition/MeshPartitionMakeSectionChannels.usf", "DrawUVDomainVS", SF_Vertex);
IMPLEMENT_GLOBAL_SHADER(FMeshPartition_DrawUVDomainPS, "/Plugin/MeshPartition/MeshPartitionMakeSectionChannels.usf", "DrawUVDomainPS", SF_Pixel);

IMPLEMENT_GLOBAL_SHADER(FMeshPartition_BorderFillCS, "/Plugin/MeshPartition/MeshPartitionBorderFill.usf", "MainCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMeshPartition_FillPullCS, "/Plugin/MeshPartition/MeshPartitionBorderFill.usf", "PullCS", SF_Compute);
IMPLEMENT_GLOBAL_SHADER(FMeshPartition_FillPushCS, "/Plugin/MeshPartition/MeshPartitionBorderFill.usf", "PushCS", SF_Compute);


