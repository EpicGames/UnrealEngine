// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionComputeModule.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h" // IMPLEMENT_MODULE
#include "ShaderCore.h" // AddShaderSourceDirectoryMapping

void FMegaMeshComputeModule::StartupModule()
{
	const FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("MeshPartition"))->GetBaseDir(), TEXT("Shaders"));
	AddShaderSourceDirectoryMapping(TEXT("/Plugin/MeshPartition"), PluginShaderDir);
}

void FMegaMeshComputeModule::ShutdownModule()
{
}
	
IMPLEMENT_MODULE(FMegaMeshComputeModule, MeshPartitionCompute)
