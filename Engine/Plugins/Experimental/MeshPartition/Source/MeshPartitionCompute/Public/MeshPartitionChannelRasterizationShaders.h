// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "PixelShaderUtils.h"
#include "ShaderParameterStruct.h"
#include "ShaderCompilerCore.h"

#define UE_API MESHPARTITIONCOMPUTE_API

//~ Plugins/MeshPartition/Shaders/MegaMeshMakeSectionChannels.usf shaders :
class FMeshPartition_DrawUVDomainVS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FMeshPartition_DrawUVDomainVS, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FMeshPartition_DrawUVDomainVS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint32>, InMeshIndices)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float2>, InMeshUVs)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<float>, InMeshChannels)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint32>, InMeshOutlines)
		SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint32>, InUVElementToVertexID)
		SHADER_PARAMETER(FUintVector4, InDrawCall)
		SHADER_PARAMETER(uint32, InChannelOffset)
	END_SHADER_PARAMETER_STRUCT()
};

class FMeshPartition_DrawUVDomainPS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FMeshPartition_DrawUVDomainPS, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FMeshPartition_DrawUVDomainPS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		RENDER_TARGET_BINDING_SLOTS()
	END_SHADER_PARAMETER_STRUCT()
};

class FMeshPartition_BorderFillCS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FMeshPartition_BorderFillCS, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FMeshPartition_BorderFillCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FUintVector2, Resolution)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<uint>,   Mask)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, RWSectionTexture)
	END_SHADER_PARAMETER_STRUCT()

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

BEGIN_SHADER_PARAMETER_STRUCT(FMeshPartition_DrawUVDomain_Parameters, )
	SHADER_PARAMETER_STRUCT_INCLUDE(FMeshPartition_DrawUVDomainVS::FParameters, VS)
	SHADER_PARAMETER_STRUCT_INCLUDE(FMeshPartition_DrawUVDomainPS::FParameters, PS)
END_SHADER_PARAMETER_STRUCT()


class FMeshPartition_FillPullCS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FMeshPartition_FillPullCS, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FMeshPartition_FillPullCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER(FIntVector4, ResolutionPass)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, SectionMipIn)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SectionMipOut)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, MaskMipIn)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float>, MaskMipOut)
		SHADER_PARAMETER_SAMPLER(SamplerState, MipSampler)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

class FMeshPartition_FillPushCS : public FGlobalShader
{
public:
	DECLARE_EXPORTED_GLOBAL_SHADER(FMeshPartition_FillPushCS, UE_API);
	SHADER_USE_PARAMETER_STRUCT(FMeshPartition_FillPushCS, FGlobalShader);

	BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
		SHADER_PARAMETER(FVector2f, TexelSize)
		SHADER_PARAMETER(FIntVector4, ResolutionPass)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float4>, SectionMipIn)
		SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, SectionMipOut)
		SHADER_PARAMETER_RDG_TEXTURE_SRV(Texture2D<float>, MaskMipIn)
		SHADER_PARAMETER_SAMPLER(SamplerState, MipSampler )
	END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}
};

#undef UE_API
