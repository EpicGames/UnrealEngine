// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

//~ Note: We use this separate module to keep the shaders in a Runtime and PostConfigInit module, despite them
//~  only being used in MegaMeshEditor, because when launching a standalone build from editor with
//~  PCGMegaMeshInterop module (which conditionally includes MegaMeshEditor), the editor module seems
//~  to not load in PostConfigInit even if that is specified as its loading phase...
class FMegaMeshComputeModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};
