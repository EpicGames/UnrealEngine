// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMeshBuilder.h"
#include "LocalVertexFactory.h"
#include "MaterialShared.h"
#include "PrimitiveSceneProxy.h"
#include "Rendering/ColorVertexBuffer.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "RHIFwd.h"

#if RHI_RAYTRACING
#include "RayTracingGeometry.h"
#endif

class UMaterialInterface;

namespace UE::MeshPartition
{
class FMeshData;
class UPreviewMeshComponent;

class FMegaMeshCustomPreviewSceneProxy : public FPrimitiveSceneProxy
{
public:
	FMegaMeshCustomPreviewSceneProxy(UPreviewMeshComponent* InComponent);
	virtual ~FMegaMeshCustomPreviewSceneProxy();

	void InitResources(const FMeshData& Mesh);

	virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;
	virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override;

#if RHI_RAYTRACING
	void InitRayTracingGeometry(FRHICommandListBase& RHICmdList);
	virtual bool IsRayTracingRelevant() const override;
	virtual bool HasRayTracingRepresentation() const override;
	virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override;
#endif // RHI_RAYTRACING

	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	virtual SIZE_T GetTypeHash() const override;
	virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;

	virtual uint32 GetMemoryFootprint(void) const override;
	uint32 GetAllocatedSize(void) const;

	FBufferRHIRef GetPositionVertexBufferRHI() const { return PositionVertexBuffer.VertexBufferRHI; }
	FBufferRHIRef GetTexCoordVertexBufferRHI() const { return StaticMeshVertexBuffer.TexCoordVertexBuffer.VertexBufferRHI; }
	FBufferRHIRef GetIndexBufferRHI() const { return IndexBuffer.IndexBufferRHI; }
	uint32 GetNumVertices() const { return PositionVertexBuffer.GetNumVertices(); }
	uint32 GetNumTriangles() const { return uint32(IndexBuffer.Indices.Num() / 3); }
	uint32 GetNumTexCoords() const { return StaticMeshVertexBuffer.GetNumTexCoords(); }
	bool GetUseFullPrecisionUVs() const { return StaticMeshVertexBuffer.GetUseFullPrecisionUVs(); }

private:
	FLocalVertexFactory VertexFactory;

	FMaterialRelevance MaterialRelevance;
	UMaterialInterface* Material = nullptr;

	FStaticMeshVertexBuffer StaticMeshVertexBuffer;
	FPositionVertexBuffer PositionVertexBuffer;
	FColorVertexBuffer ColorVertexBuffer;

	FDynamicMeshIndexBuffer32 IndexBuffer;

#if RHI_RAYTRACING
	FRayTracingGeometry RayTracingGeometry;
#endif // RHI_RAYTRACING
};

} // namespace UE::MeshPartition
