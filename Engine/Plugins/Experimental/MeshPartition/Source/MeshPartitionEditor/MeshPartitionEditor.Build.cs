// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshPartitionEditor : ModuleRules
{
	public MeshPartitionEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bAllowUETypesInNamespaces = true;

		PublicIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"EditorSubsystem",
				"ModelingComponents", // IMeshSculptLayersManager
				"MeshPartition",
			}
			);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ActionableMessage",
				"CoreUObject",
				"DynamicMesh",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"GeometryAlgorithms",
				"GeometryScriptingEditor",
				"ImageCore",
				"InputCore",
				"MeshPartitionCompute",
				"MeshConversion",
				"MeshConversionEngineTypes",
				"MeshDescription",
				"ModelingOperators",
				"ModelingComponentsEditorOnly",
				"ModelingToolsEditorMode",
				"NaniteUtilities",
				"PhysicsCore",
				"RenderCore",
				"Renderer",
				"RHI",
				"ToolMenus",
				"Slate",
				"SlateCore",
				"SourceControl",
				"StaticMeshDescription",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UnrealEd",
				"Projects",
				"LevelEditor",
				"SceneOutliner",
				"DerivedDataCache",
				"CQTest",
				"Foliage",
				"VEUVCore",
			}
			);
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
