// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionChannelCollection.h"

#include "MeshPartition.h"
#include "MeshPartitionRenderingUtils.h"

#include "Tasks/Task.h"
#include "Async/ParallelFor.h"

#include "MaterialCache/MaterialCache.h"
#include "MeshPartitionChannelRasterizationShaders.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionModifierTaskGraph.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorModule.h"

#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCollection.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "EngineUtils.h"
#include "Editor.h"
#include "Misc/App.h"
#include "RHI.h"
#include "GenerateMips.h"
#include "UObject/GarbageCollection.h"

#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "Parameterization/DynamicMeshUVEditor.h"

#include "Util/SizedDisjointSet.h"

#include "TextureResource.h"
#include "PixelShaderUtils.h"
#include "RenderGraphBuilder.h"
#include "RenderGraphUtils.h"
#include "UObject/ConstructorHelpers.h"

#include "UObject/Object.h"
#include "MeshPartitionCompiledSection.h"

namespace UE::MeshPartition
{

void ApplyChannels(ACompiledSection* InSection, const FSectionChannels& InChannels, const float InMaterialCacheTexelSize)
{
	InSection->SetChannelTexture(InChannels.Texture.Get());
	InSection->SetChannelData(InChannels.Table, InChannels.TexcoordMetrics);

	if (InChannels.Domain.AreaUV > 0)
	{
		InSection->SetMaterialCacheTileCount(GetMaterialCacheTileCount(InChannels.Domain.Area3D / InChannels.Domain.AreaUV, InMaterialCacheTexelSize));
		InSection->RecreateMaterialCacheTextures();
	}
}

// Shared on-change callback for the channel/preview CVars below: force every MeshPartition in the editor world to rebuild its sections.
static void RebuildAllSectionsOnCVarChange(IConsoleVariable*)
{
	if (!GEditor)
	{
		return;
	}
	UWorld* World = GEditor->GetEditorWorldContext().World();
	for (UE::MeshPartition::AMeshPartition* MeshPartition : TActorRange<UE::MeshPartition::AMeshPartition>(World))
	{
		if (UE::MeshPartition::UMeshPartitionEditorComponent* EditorComponent = Cast<UE::MeshPartition::UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent()))
		{
			EditorComponent->ForceRebuildAllSections(UE::MeshPartition::EChangeType::TransientStateChange);
		}
	}
}

static TAutoConsoleVariable<int32> CVarMeshPartition_UnwrapUVsMethod(
	TEXT("MeshPartition.Channels.UnwrapUVsMethod"),
	-1,
	TEXT("Override for the UV unwrap method. -1 (default): use the value from the MeshPartitionDefinition asset. ")
	TEXT("1: Fast Box Project. 2: Reference Box Project. 3: Volume Encoded (VEUV). 4: Plane Project."),
	FConsoleVariableDelegate::CreateStatic(&RebuildAllSectionsOnCVarChange)
);

// Read by the editor PreviewSection build path in MeshPartitionEditorComponent.cpp; declared extern there.
TAutoConsoleVariable<int32> CVarMeshPartition_PreviewSectionUVQuality(
	TEXT("MeshPartition.Preview.UVQuality"),
	0,
	TEXT("UV unwrap quality for editor PreviewSection builds. Currently specific to the Volume Encoded UV method. 0: Full; 1: Preview (low quality, fast iteration). Default: 0. Note compiled sections always use Full quality."),
	FConsoleVariableDelegate::CreateStatic(&RebuildAllSectionsOnCVarChange)
);

TAutoConsoleVariable<bool> CVarMeshPartition_VEUVEnableFailureFallback(
	TEXT("MeshPartition.Channels.VEUV.EnableFailureFallback"),
	true,
	TEXT("If true, sections where VEUV reports failure fall back to the reference dynamic-mesh-tools box projection. Disable to inspect the raw VEUV output (UVs may be NaN or inverted)."),
	FConsoleVariableDelegate::CreateStatic(&RebuildAllSectionsOnCVarChange)
);

static TAutoConsoleVariable<int32> CVarMeshPartition_BorderFillMethod(
	TEXT("MeshPartition.Channels.BorderFillMethod"),
	1,
	TEXT("Enables border filling on section page rendering. 0: No border; 1: Border fill; 2: Push pull"),
	FConsoleVariableDelegate::CreateStatic(&RebuildAllSectionsOnCVarChange)
);

static TAutoConsoleVariable<float> CVarMeshPartition_GutterTexelCount(
	TEXT("MeshPartition.Channels.GutterTexelCount"),
	4.0f,
	TEXT("Target texel-space gap between UV islands. Larger values reduce mip/filter bleed at the cost of UV area."),
	FConsoleVariableDelegate::CreateStatic(&RebuildAllSectionsOnCVarChange)
);

// Returns the uv method cvar's explicit override if set to a valid enum value, otherwise unset
static TOptional<EChannelCollectionUVLayoutMethod> GetUVLayoutMethodCVarOverride()
{
	const int32 UVMethodCVar = CVarMeshPartition_UnwrapUVsMethod.GetValueOnAnyThread();
	if (UVMethodCVar >= static_cast<int32>(EChannelCollectionUVLayoutMethod::FastBoxProject) &&
		UVMethodCVar <= static_cast<int32>(EChannelCollectionUVLayoutMethod::PlaneProject))
	{
		return static_cast<EChannelCollectionUVLayoutMethod>(UVMethodCVar);
	}
	return {};
}

FChannelCollectionUVLayoutOptions FChannelCollectionUVLayoutOptions::GetDefaults()
{
	FChannelCollectionUVLayoutOptions Options
	{
		.TexelSize = FChannelTextureRenderer::DefaultTexelSize,
		.MaxTextureResolution = FChannelTextureRenderer::DefaultMaxImageResolution,
		.GutterTexelCount = CVarMeshPartition_GutterTexelCount.GetValueOnAnyThread(),
		.UVLayoutMethod = GetUVLayoutMethodCVarOverride().Get(EChannelCollectionUVLayoutMethod::ReferenceBoxProject),
	};
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
	Options.VEUV.bAcceptOutputOnFailure = !CVarMeshPartition_VEUVEnableFailureFallback.GetValueOnAnyThread();
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS
	
	return Options;
}

FChannelCollectionUVLayoutOptions FChannelCollectionUVLayoutOptions::GetFromDefinition(const UMeshPartitionDefinition* InDefinition)
{
	FChannelCollectionUVLayoutOptions Options = GetDefaults();
	if (InDefinition)
	{
		Options.TexelSize = InDefinition->GetChannelTexelSize();

		// The UV-method cvar default is a -1 sentinel meaning "use the asset value".
		// An explicit, valid cvar value overrides the per-asset choice; anything else falls through to the asset.
		if (!GetUVLayoutMethodCVarOverride().IsSet())
		{
			Options.UVLayoutMethod = InDefinition->GetChannelUVLayoutMethod();
		}
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		Options.VEUV.SamplesPerSquareMeter = InDefinition->GetChannelVEUVSamplesPerSquareMeter();
		Options.VEUV.VoxelCount = InDefinition->GetChannelVEUVVoxelCount();
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

		Options.PlaneProjection.NormalSource = InDefinition->GetChannelPlaneProjectionNormalSource();
		Options.PlaneProjection.FixedNormal = InDefinition->GetChannelPlaneProjectionFixedNormal();
	}
	return Options;
}

UTexture* FChannelTextureRenderer::GetDefaultTexture()
{
	static TSoftObjectPtr<UTexture2DArray> BlackTexture(
		FSoftObjectPath(TEXT("/MeshPartition/Textures/Void2DArray.Void2DArray"))
	);
	return BlackTexture.LoadSynchronous();
}

Tasks::TTask<TOptional<UTexture*>> FChannelTextureRenderer::LaunchAllocateTextureTask(
	Tasks::TTask<FSection> InBuildSectionTask,
	UObject* InOwner)
{
	return Tasks::Launch(TEXT("AllocateTexture"),
		[InBuildSectionTask, InOwner]() mutable
		{
			const FSection& Section = InBuildSectionTask.GetResult();

			TOptional<UTexture*> SectionRenderTexture;

			int32 NumActiveSlices = Section.NumActivePages();
			if (NumActiveSlices)
			{
				UObject* Outer = InOwner ? InOwner : GetTransientPackage();
				if (UTexture* AllocatedTexture = AllocateSectionTextureResource(
						Section.RasterResolution.X,
						Section.RasterResolution.Y,
						NumActiveSlices,
						PF_G8,
						Outer,
						MakeUniqueObjectName(Outer, UTexture::StaticClass(), TEXT("MeshPartitionChannelTexture"))))
				{
					SectionRenderTexture = AllocatedTexture;
				}
			}

			return MoveTemp(SectionRenderTexture);
		},
		Tasks::Prerequisites(InBuildSectionTask)
	);
}

Tasks::TTask<MeshPartition::FSectionChannels> FChannelTextureRenderer::LaunchRenderTextureTask(
	Tasks::TTask<FSection> InBuildSectionTask,
	Tasks::TTask<TOptional<UTexture*>> InAllocateTextureTask,
	bool bInDownloadToAsset,
	bool bInExtractSectionMeshData
) {
	return Tasks::Launch(TEXT("RenderSectionChannels"),
		[InBuildSectionTask, InAllocateTextureTask, bInDownloadToAsset, bInExtractSectionMeshData]() mutable
		{
			FSection Section = MoveTemp(InBuildSectionTask.GetResult());
			TArray<uint8> ChannelOffsetTable = Section.ChannelOffsetTable;
			const FVector2f TexcoordMetrics = Section.TexcoordMetrics;
			const int32 NumActiveSlices = Section.NumActivePages();

			MeshPartition::FSectionChannels Result {
				.Table = MoveTemp(ChannelOffsetTable),
				.TexcoordMetrics = TexcoordMetrics,
			};

			if (bInExtractSectionMeshData)
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(FChannelTextureRenderer::ExtractSectionMeshData);

				Result.Mesh = {
					Section.UVmesh.Indices,
					Section.UVmesh.UVs,
					Section.UVmesh.Channels,
					Section.UVmesh.Outlines,
					Section.UVmesh.UVElementToVertexID,
					Section.UVmesh.VertexCount
				};
			}
				
			Result.Domain = {
				Section.DomainMapping.Area3D,
				Section.DomainMapping.AreaUV,
				Section.DomainMapping.GetGutterUV()
			};

			const TOptional<UTexture*>& SectionRenderTexture = InAllocateTextureTask.GetResult();

			TStrongObjectPtr<UTexture> ChannelTexture(nullptr);

			// Render Section Channels in Section Textures (Channel Pages)
			// Early exit if no valid active channel
			if (!SectionRenderTexture.IsSet())
			{
				ChannelTexture = TStrongObjectPtr<UTexture>(GetDefaultTexture());
			}
			else
			{
				SectionRenderTexture.GetValue()->UpdateResource();

				// Render section channels
				RenderSectionChannels(SectionRenderTexture.GetValue(), MoveTemp(Section));

				// and eventually download the texels to the asset in the case of the compiled section
				if (bInDownloadToAsset)
				{
					DownloadSectionTextureToAsset(SectionRenderTexture.GetValue());
				}

				// Finally, we update the section with the render target results.
				ChannelTexture = TStrongObjectPtr<UTexture>(SectionRenderTexture.GetValue());
			}

			if (ChannelTexture)
			{
				ChannelTexture->ClearInternalFlags(EInternalObjectFlags::Async);
				ForEachObjectWithOuter(ChannelTexture.Get(), [](UObject* Subobject)
					{
						Subobject->ClearInternalFlags(EInternalObjectFlags::Async);
					}
				);
			}

			Result.Texture = ChannelTexture;
			return Result;
		},
		Tasks::Prerequisites(InAllocateTextureTask),
		Tasks::ETaskPriority::Normal,
		Tasks::EExtendedTaskPriority::GameThreadNormalPri
	);
}

Tasks::TTask<MeshPartition::FSectionChannels> FChannelTextureRenderer::BuildTextureForSection(const FMeshData& InSectionMesh, UObject* InOwner, bool bDownloadToAsset, const MeshPartition::FChannelMap& InChannels, float InTexelResolution)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::BuildTextureForSection);

	Tasks::TTask<FSection> BuildSection = Tasks::Launch(TEXT("Generate UV Mesh"),
		[&SectionMesh = InSectionMesh, ChannelMap = InChannels, InTexelResolution]()
		{
			FSection Section;

			// Generate the UV domain for the section that unifies the bases in the section
			// This populates the UVMesh in the Section struct, ready to be uploaded to gpu and rendered 
			GenerateSectionMeshUVs(SectionMesh, Section, InTexelResolution);

			// Generate the Channels as UVMesh attributes for the section
			// Needs the channels declaration of the definition to make a stable mapping of the channel
			GenerateSectionMeshChannels(SectionMesh, Section, ChannelMap);

			return MoveTemp(Section);
		}
	);

	Tasks::TTask<TOptional<UTexture*>> AllocateTexture = LaunchAllocateTextureTask(BuildSection, InOwner);
	return LaunchRenderTextureTask(BuildSection, AllocateTexture, bDownloadToAsset);
}

Tasks::TTask<MeshPartition::FSectionChannels> FChannelTextureRenderer::BuildTextureForSectionWithCachedTopology(
	const FDynamicMesh3& InSectionMesh,
	const FChannelRenderUVMeshTopology& InCachedTopology,
	UObject* InOwner,
	bool bDownloadToAsset,
	const MeshPartition::FChannelMap& InChannels
)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::BuildTextureForSectionWithCachedTopology);

	Tasks::TTask<FSection> BuildSection = Tasks::Launch(TEXT("BuildSectionFromCache"),
		[&SectionMesh = InSectionMesh, &CachedTopology = InCachedTopology, ChannelMap = InChannels]() mutable
		{
			// Apply cached topology
			FSection Section = BuildSectionFromCachedTopology(CachedTopology);

			// Generate just the channels data from current mesh data
			GenerateSectionMeshChannels(SectionMesh, Section, ChannelMap);

			return MoveTemp(Section);
		}
	);

	Tasks::TTask<TOptional<UTexture*>> AllocateTexture = LaunchAllocateTextureTask(BuildSection, InOwner);
	return LaunchRenderTextureTask(BuildSection, AllocateTexture, bDownloadToAsset);
}

void FChannelTextureRenderer::GenerateSectionMeshChannels(const FMeshData& InSectionMesh, FChannelTextureRenderer::FSection& InSection, const MeshPartition::FChannelMap& InChannels)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::GenerateSectionMeshChannels);

	int32 NumVertices = InSection.UVmesh.VertexCount;
	int32 MaxNumChannels = InChannels.GetNumChannels();

	TArray<int32> ChannelActiveCounts;
	ChannelActiveCounts.Init(0, MaxNumChannels);

	if (InSectionMesh.GetWeightLayerNames().Num() > 0)
	{
		// Grab the local packed array of channel attributes from the weight layer values
		const TArray<FName> WeightLayerNames = InSectionMesh.GetWeightLayerNames();
		const TArray<FName> MeshPartitionChannels = InChannels.GetChannels();
		TArray<int32> SectionChannelIndex;
		TArray<TArray<float>> SourceSectionChannels;
		int NumUsedChannels = 0;
		int32 ChannelIndex = 0;
		for (const FName ChannelName : MeshPartitionChannels)
		{
			if (WeightLayerNames.Find(ChannelName) != INDEX_NONE)
			{
				TArray<float> WeightLayer = InSectionMesh.GetWeightLayerValues(ChannelName);

				// Check that the values in this layer are valid and active (not null):
				int32 ActiveCount = 0;
				for (float Weight : WeightLayer)
				{
					if (Weight > 0.0f)
					{
						++ActiveCount;
					}
				}

				// Channel is active
				if (ActiveCount && WeightLayer.Num() == NumVertices)
				{
					SourceSectionChannels.Emplace(MoveTemp(WeightLayer));
					SectionChannelIndex.Add(ChannelIndex);
					ChannelActiveCounts[ChannelIndex] = ActiveCount;
				}
			}

			ChannelIndex++;
		}

		// gather the section channels values
		// and allocate new channel entry if needed
		InSection.UVmesh.Channels.SetNum(NumVertices * SourceSectionChannels.Num());

		uint64 ChannelWeightsOffset = NumVertices * sizeof(float);
		for (int32 i = 0; i < SourceSectionChannels.Num(); ++i)
		{
			FMemory::Memcpy((InSection.UVmesh.Channels.GetData() + i * NumVertices), SourceSectionChannels[i].GetData(), NumVertices * sizeof(float));
		}
	}
	InSection.UVmesh.ChannelActiveCounts = ChannelActiveCounts;

	// Gather the actual active channels with non zero values
	// Populate the channel offset table
	int8 ActiveChannelIndex = 0;
	TArray<uint8> ActiveChannelIndices;
	TArray<uint8> ChannelTableIndices;
	for (int32 C : ChannelActiveCounts)
	{
		if (C > 0)
		{
			// This channel is active, and will be stored there
			ChannelTableIndices.Add(ActiveChannelIndices.Num());
			ActiveChannelIndices.Add(ActiveChannelIndex);
		}
		else
		{
			ChannelTableIndices.Add(-1); // empty channel
		}
		ActiveChannelIndex++;
	}
	InSection.UVmesh.ActiveChannelIndices = ActiveChannelIndices;
	InSection.ChannelOffsetTable = ChannelTableIndices;
}

void FChannelTextureRenderer::GenerateSectionMeshChannels(const Geometry::FDynamicMesh3& InSectionMesh, FChannelTextureRenderer::FSection& InSection, const MeshPartition::FChannelMap& InChannels)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::GenerateSectionMeshChannels);

	const int32 NumVertices = InSectionMesh.VertexCount();
	const int32 MaxNumChannels = InChannels.GetNumChannels();

	TArray<int32> ChannelActiveCounts;
	ChannelActiveCounts.SetNumZeroed(MaxNumChannels);

	const Geometry::FDynamicMeshAttributeSet* Attributes = InSectionMesh.Attributes();

	if (Attributes->NumWeightLayers() > 0)
	{
		// Build dense weight array matching vertex ordering
		// Create mapping from sparse VertexID to dense index
		TArray<int32> VertexIDToDenseIndex;
		VertexIDToDenseIndex.SetNum(InSectionMesh.MaxVertexID());
		{
			int32 DenseIndex = 0;
			for (int VertexID_Map : InSectionMesh.VertexIndicesItr())
			{
				VertexIDToDenseIndex[VertexID_Map] = DenseIndex++;
			}
			ensure(DenseIndex == NumVertices);
		}

		// Grab the local packed array of channel attributes from the weight layer values
		const TArray<FName> MeshPartitionChannels = InChannels.GetChannels();
		TArray<int32> SectionChannelIndex;
		TArray<TArray<float>> SourceSectionChannels;
		int NumUsedChannels = 0;
		int32 ChannelIndex = 0;
		for (const FName ChannelName : MeshPartitionChannels)
		{
			const Geometry::FDynamicMeshWeightAttribute* WeightLayerAttribute = nullptr;
			// Find the WeightLayerIndex on the dynamic mesh matching the current channel name, if it exists.
			for (int WeightLayerIndex = 0; WeightLayerIndex < Attributes->NumWeightLayers(); ++WeightLayerIndex)
			{
				auto WeightLayer = Attributes->GetWeightLayer(WeightLayerIndex);
				const FName WeightLayerName = WeightLayer->GetName();
				if (WeightLayerName == ChannelName)
				{
					WeightLayerAttribute = WeightLayer;
					break;
				}
			}

			if (WeightLayerAttribute)
			{
				TArray<float> WeightLayer;
				WeightLayer.SetNumZeroed(NumVertices);

				int32 ActiveCount = 0;
				for (int VertexID : InSectionMesh.VertexIndicesItr())
				{
					float Value = 0.0f;
					WeightLayerAttribute->GetValue(VertexID, &Value);

					int32 DenseIdx = VertexIDToDenseIndex[VertexID];
					WeightLayer[DenseIdx] = Value;

					if (Value > 0.0f)
					{
						++ActiveCount;
					}
				}

				// Channel is active if it has non-zero values
				if (ActiveCount > 0)
				{
					SourceSectionChannels.Emplace(MoveTemp(WeightLayer));
					SectionChannelIndex.Add(ChannelIndex);
					ChannelActiveCounts[ChannelIndex] = ActiveCount;
				}
			}

			ChannelIndex++;
		}

		// gather the section channels values
		// and allocate new channel entry if needed
		InSection.UVmesh.Channels.SetNum(NumVertices * SourceSectionChannels.Num());

		uint64 ChannelWeightsOffset = NumVertices * sizeof(float);
		for (int32 i = 0; i < SourceSectionChannels.Num(); ++i)
		{
			FMemory::Memcpy((InSection.UVmesh.Channels.GetData() + i * NumVertices), SourceSectionChannels[i].GetData(), NumVertices * sizeof(float));
		}
	}
	InSection.UVmesh.ChannelActiveCounts = ChannelActiveCounts;

	// Gather the actual active channels with non zero values
	// Populate the channel offset table
	int8 ActiveChannelIndex = 0;
	TArray<uint8> ActiveChannelIndices;
	TArray<uint8> ChannelTableIndices;
	for (int32 C : ChannelActiveCounts)
	{
		if (C > 0)
		{
			// This channel is active, and will be stored there
			ChannelTableIndices.Add(ActiveChannelIndices.Num());
			ActiveChannelIndices.Add(ActiveChannelIndex);
		}
		else
		{
			ChannelTableIndices.Add(-1); // empty channel
		}
		ActiveChannelIndex++;
	}
	InSection.UVmesh.ActiveChannelIndices = ActiveChannelIndices;
	InSection.ChannelOffsetTable = ChannelTableIndices;
}

UTexture* FChannelTextureRenderer::AllocateSectionTextureResource(int32 InSizeX, int32 InSizeY, int32 InArraySize, EPixelFormat InFormat, UObject* InOuter, const FName InName)
{
	UTexture2DArray* SectionTexture = nullptr;
	if (InSizeX > 0 && InSizeY > 0 && InArraySize > 0 &&
		(InSizeX % GPixelFormats[InFormat].BlockSizeX) == 0 &&
		(InSizeY % GPixelFormats[InFormat].BlockSizeY) == 0)
	{
		// Prevent GC from running while creating UObjects off the game thread.
		// UObjectBase::AddObject automatically sets EInternalObjectFlags::Async (a root flag)
		// on objects created off the game thread, and SetRootFlags requires either the game
		// thread or the GC lock to be held.
		FGCScopeGuard GCScopeGuard;
		SectionTexture = NewObject<UTexture2DArray>(InOuter, InName);

		SectionTexture->SetPlatformData(new FTexturePlatformData());
		SectionTexture->GetPlatformData()->SizeX = InSizeX;
		SectionTexture->GetPlatformData()->SizeY = InSizeY;
		SectionTexture->GetPlatformData()->SetNumSlices(InArraySize);
		SectionTexture->GetPlatformData()->PixelFormat = InFormat;
		SectionTexture->bNotOfflineProcessed = true;

		// Allocate mipmaps.
		const int32 NumMips = FMath::FloorLog2(FMath::Max(InSizeX, InSizeY)) + 1;
		for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
		{
			int32 MipSizeX = FMath::Max((int32)InSizeX >> MipIndex, GPixelFormats[InFormat].BlockSizeX);
			int32 MipSizeY = FMath::Max((int32)InSizeY >> MipIndex, GPixelFormats[InFormat].BlockSizeY);

			int32 NumBlocksX = MipSizeX / GPixelFormats[InFormat].BlockSizeX;
			int32 NumBlocksY = MipSizeY / GPixelFormats[InFormat].BlockSizeY;
			if (NumBlocksX >= 1 && NumBlocksY >= 1)
			{
				FTexture2DMipMap* Mip = new FTexture2DMipMap(MipSizeX, MipSizeY, InArraySize);
				SectionTexture->GetPlatformData()->Mips.Add(Mip);
				Mip->BulkData.Lock(LOCK_READ_WRITE);
				Mip->BulkData.Realloc((int64)GPixelFormats[InFormat].BlockBytes * NumBlocksX * NumBlocksY * InArraySize);
				Mip->BulkData.Unlock();
			}
			else
			{
				break;
			}
		}

		SectionTexture->SRGB = false;
		SectionTexture->CompressionNone = false;
		SectionTexture->CompressionSettings = TC_Alpha;
		SectionTexture->MipGenSettings = TMGS_LeaveExistingMips;
		SectionTexture->Filter = TF_Bilinear; // Use TF_Nearest to see the channel texels 
		SectionTexture->AddressX = TA_Clamp;
		SectionTexture->AddressY = TA_Clamp;
		SectionTexture->LODGroup = TEXTUREGROUP_Terrain_Weightmap;
	}
	else
	{
		UE_LOGF(LogTexture, Warning,
			"Invalid parameters specified for UTexture2DArray_Create() "
			"(SizeX=%d, SizeY=%d, ArraySize=%d, Format=%ls, BlockSizeX=%d, BlockSizeY=%d)",
			InSizeX, InSizeY, InArraySize,
			GPixelFormats[InFormat].Name,
			GPixelFormats[InFormat].BlockSizeX,
			GPixelFormats[InFormat].BlockSizeY);
	}
	return SectionTexture;
}

void FChannelTextureRenderer::RenderSectionChannels(UTexture* InOutSectionTexture, FChannelTextureRenderer::FSection InSection) 
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::RenderSectionChannels);

	// Headless paths (e.g. -nullrhi gauntlet runs, commandlets) can't do this channel rendering, so we early out here
	if (GUsingNullRHI || !FApp::CanEverRender())
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(MegaMeshRenderRenderProceduralPainting)([DestSectionTexture = InOutSectionTexture, Section = MoveTemp(InSection)](FRHICommandListImmediate& RHICmdList)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::Render_Command_MegaMeshRenderRenderProceduralPainting);
			
			int32 NumIndices = Section.UVmesh.NumIndices();
			int32 NumUVs = Section.UVmesh.UVs.Num();
			int32 ActualVertexCount = Section.UVmesh.VertexCount;
			int32 NumChannels = Section.UVmesh.NumChannels();
			int32 NumOutlines = Section.UVmesh.Outlines.Num();
			
			const bool bShouldRenderOutlines = NumOutlines > 0;

			TArray<uint8> ChannelOffsetTable = Section.ChannelOffsetTable;

			FRDGBuilder GraphBuilder(RHICmdList);

			MeshPartition::FRenderCaptureManager::BeginCapture(GraphBuilder);

			TRACE_CPUPROFILER_EVENT_MANUAL_START("UploadBuffers");

			// Section Indices
			FRDGBufferSRVRef RDGSectionMeshIndicesSRV;
			{
				// Section turned into RDG Resource
				FRDGBufferRef RDGSectionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumIndices), TEXT("SectionIndices"));
				GraphBuilder.QueueBufferUpload(RDGSectionBuffer, Section.UVmesh.Indices.GetData(), sizeof(int32) * NumIndices);

				FRDGBufferSRVDesc RDGSectionBufferDesc(RDGSectionBuffer, PF_R32_SINT);
				RDGSectionBufferDesc.NumElements = NumIndices;
				RDGSectionBufferDesc.StartOffsetBytes = 0;

				RDGSectionMeshIndicesSRV = GraphBuilder.CreateSRV(RDGSectionBufferDesc);
			}

			// Section UVs
			FRDGBufferSRVRef RDGSectionMeshUVsSRV;
			{
				// Section turned into RDG Resource
				FRDGBufferRef RDGSectionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(2 * sizeof(float), NumUVs), TEXT("SectionUVs"));
				GraphBuilder.QueueBufferUpload(RDGSectionBuffer, Section.UVmesh.UVs.GetData(), 2 * sizeof(float) * NumUVs);

				FRDGBufferSRVDesc RDGSectionBufferDesc(RDGSectionBuffer, PF_G32R32F);
				RDGSectionBufferDesc.NumElements = NumUVs;
				RDGSectionBufferDesc.StartOffsetBytes = 0;

				RDGSectionMeshUVsSRV = GraphBuilder.CreateSRV(RDGSectionBufferDesc);
			}

			// Section Channels
			FRDGBufferSRVRef RDGSectionMeshChannelsSRV;
			{
				// Section turned into RDG Resource
				FRDGBufferRef RDGSectionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(float), Section.UVmesh.Channels.Num()), TEXT("SectionChannelMasks"));
				GraphBuilder.QueueBufferUpload(RDGSectionBuffer, Section.UVmesh.Channels.GetData(), sizeof(float) * Section.UVmesh.Channels.Num());

				FRDGBufferSRVDesc RDGSectionBufferDesc(RDGSectionBuffer, PF_R32_FLOAT);
				RDGSectionBufferDesc.NumElements = Section.UVmesh.Channels.Num();
				RDGSectionBufferDesc.StartOffsetBytes = 0;

				RDGSectionMeshChannelsSRV = GraphBuilder.CreateSRV(RDGSectionBufferDesc);
			}

			// UV Element to Vertex ID Indirection Buffer
			FRDGBufferSRVRef RDGUVElementToVertexIDSRV;
			{
				FRDGBufferRef RDGIndirectionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumUVs), TEXT("UVElementToVertexIDIndirection"));
				GraphBuilder.QueueBufferUpload(RDGIndirectionBuffer, Section.UVmesh.UVElementToVertexID.GetData(), sizeof(int32) * NumUVs);

				FRDGBufferSRVDesc RDGIndirectionBufferDesc(RDGIndirectionBuffer, PF_R32_SINT);
				RDGIndirectionBufferDesc.NumElements = NumUVs;
				RDGIndirectionBufferDesc.StartOffsetBytes = 0;

				RDGUVElementToVertexIDSRV = GraphBuilder.CreateSRV(RDGIndirectionBufferDesc);
			}

			// Section Outlines
			FRDGBufferSRVRef RDGSectionMeshOutlinesSRV;
			{
				// Section turned into RDG Resource
				FRDGBufferRef RDGSectionBuffer = GraphBuilder.CreateBuffer(FRDGBufferDesc::CreateBufferDesc(sizeof(int32), NumOutlines * 3), TEXT("SectionChannelOutlines"));
				GraphBuilder.QueueBufferUpload(RDGSectionBuffer, Section.UVmesh.Outlines.GetData(), sizeof(int32) * NumOutlines * 3);

				FRDGBufferSRVDesc RDGSectionBufferDesc(RDGSectionBuffer, PF_R32_SINT);
				RDGSectionBufferDesc.NumElements = NumOutlines * 3;
				RDGSectionBufferDesc.StartOffsetBytes = 0;

				RDGSectionMeshOutlinesSRV = GraphBuilder.CreateSRV(RDGSectionBufferDesc);
			}
			TRACE_CPUPROFILER_EVENT_MANUAL_END();

			// Pass Shaders
			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			// Shader with Permutation : auto VertexShader = GlobalShaderMap->GetShader<FRasterizeToRectsVS>(PermutationVector);
			auto VertexShader = ShaderMap->GetShader<FMeshPartition_DrawUVDomainVS>();
			auto PixelShader = ShaderMap->GetShader<FMeshPartition_DrawUVDomainPS>();

			// Destination texture to RDG
			FTextureResource* DestResource = DestSectionTexture->GetResource();
			FRHITexture* DestTexture = DestResource->GetTexture2DArrayRHI();
			FRDGTextureRef RDGDestTexture = RegisterExternalTexture(GraphBuilder, DestTexture, TEXT("MeshPartition_SectionTexture"));
			GraphBuilder.UseInternalAccessMode(RDGDestTexture);

			// Render targets reused for each passes and blitted into destination
			FRDGTextureRef RDGRenderTarget = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(
				DestTexture->GetDesc().Extent,
				DestTexture->GetDesc().Format,
				FClearValueBinding::Black,
				ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV | ETextureCreateFlags::RenderTargetable,
				DestTexture->GetDesc().NumMips
			), TEXT("MeshPartition_SectionTexture_PassRenderTarget"));

			FRDGTextureRef RDGMaskTexture = GraphBuilder.CreateTexture(FRDGTextureDesc::Create2D(
				DestTexture->GetDesc().Extent,
				PF_R8,
				FClearValueBinding::Black,
				ETextureCreateFlags::ShaderResource | ETextureCreateFlags::UAV | ETextureCreateFlags::RenderTargetable,
				DestTexture->GetDesc().NumMips
			), TEXT("MeshPartition_SectionMask_PassRenderTarget"));

			for (int32 Page = 0; Page < Section.NumActivePages(); ++Page)
			{
				// Get definition channel index for this Page
				const int32 DefinitionChannelIndex = Section.UVmesh.ActiveChannelIndices[Page];

				// Pass Parameters
				FMeshPartition_DrawUVDomain_Parameters* PassParameters = GraphBuilder.AllocParameters<FMeshPartition_DrawUVDomain_Parameters>();
				PassParameters->VS.InMeshIndices = RDGSectionMeshIndicesSRV;
				PassParameters->VS.InMeshUVs = RDGSectionMeshUVsSRV;
				PassParameters->VS.InMeshChannels = RDGSectionMeshChannelsSRV;
				PassParameters->VS.InMeshOutlines = RDGSectionMeshOutlinesSRV;
				PassParameters->VS.InUVElementToVertexID = RDGUVElementToVertexIDSRV;
				PassParameters->VS.InChannelOffset = Page * ActualVertexCount;

				ClearUnusedGraphResources(VertexShader, &PassParameters->VS);
				ClearUnusedGraphResources(PixelShader, &PassParameters->PS);

				PassParameters->PS.RenderTargets[0] = FRenderTargetBinding(RDGRenderTarget, ERenderTargetLoadAction::EClear);
				PassParameters->PS.RenderTargets[1] = FRenderTargetBinding(RDGMaskTexture, ERenderTargetLoadAction::EClear);

				// Cue Pass
				GraphBuilder.AddPass(
					RDG_EVENT_NAME("MeshPartition_MakeSectionChannels"),
					PassParameters,
					ERDGPassFlags::Raster,
					[PassParameters, Page, &Section, ShaderMap, VertexShader, PixelShader, bShouldRenderOutlines](FRDGAsyncTask, FRHICommandList& RHICmdList)
					{
						// Pass PSO
						FGraphicsPipelineStateInitializer GraphicsPSOInit;
						RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

						FIntPoint ViewportSize(Section.RasterResolution.X, Section.RasterResolution.Y);
						RHICmdList.SetViewport(0.0f, 0.0f, 0.0f, (float)ViewportSize.X, (float)ViewportSize.Y, 1.0f);

						GraphicsPSOInit.BlendState = TStaticBlendState<>::GetRHI();
						GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();
						GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Always>::GetRHI();

						FVertexDeclarationElementList Elements;
						GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = PipelineStateCache::GetOrCreateVertexDeclaration(Elements);
						GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
						GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();


						if (bShouldRenderOutlines)
						{
							GraphicsPSOInit.PrimitiveType = PT_LineList;
							SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

							for (int32 PassId = 0; PassId < Section.Bases_UVBox.Num(); ++PassId)
							{
								const FInt32Vector2 BaseMeshRange = Section.UVmesh.BaseIndicesRanges[PassId];

								PassParameters->VS.InDrawCall = FUintVector4(Section.UVmesh.UVs.Num(), Section.UVmesh.NumChannels(), BaseMeshRange.X, 1);

								SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
								SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

								int32 NumPrimitives = Section.UVmesh.Outlines.Num();
								RHICmdList.DrawPrimitive(0, NumPrimitives, 1);
							}
						}

						GraphicsPSOInit.PrimitiveType = PT_TriangleList;
						SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);

						for (int32 PassId = 0; PassId < Section.Bases_UVBox.Num(); ++PassId)
						{
							const FInt32Vector2 BaseMeshRange = Section.UVmesh.BaseIndicesRanges[PassId];

							PassParameters->VS.InDrawCall = FUintVector4(Section.UVmesh.UVs.Num(), Section.UVmesh.NumChannels(), BaseMeshRange.X, 0);

							SetShaderParameters(RHICmdList, VertexShader, VertexShader.GetVertexShader(), PassParameters->VS);
							SetShaderParameters(RHICmdList, PixelShader, PixelShader.GetPixelShader(), PassParameters->PS);

							int32 NumPrimitives = BaseMeshRange.Y / 3;
							RHICmdList.DrawPrimitive(0, NumPrimitives, 1);
						}
					});

				// Border fill
				if (CVarMeshPartition_BorderFillMethod->GetInt() == 1)
				{
					FMeshPartition_BorderFillCS::FParameters* BorderParams = GraphBuilder.AllocParameters<FMeshPartition_BorderFillCS::FParameters>();
					BorderParams->Mask = GraphBuilder.CreateSRV(RDGMaskTexture);
					BorderParams->RWSectionTexture = GraphBuilder.CreateUAV(RDGRenderTarget);
					BorderParams->Resolution = FUintVector2(RDGRenderTarget->Desc.Extent.X, RDGRenderTarget->Desc.Extent.Y);

					TShaderRef<FMeshPartition_BorderFillCS> ComputeShader = ShaderMap->GetShader<FMeshPartition_BorderFillCS>();
						
					FComputeShaderUtils::AddPass(
						GraphBuilder,
						RDG_EVENT_NAME("MeshPartition_BorderFill"),
						ERDGPassFlags::Compute,
						ComputeShader,
						BorderParams,
						FIntVector::DivideAndRoundUp(
							FIntVector(DestTexture->GetSizeX(), DestTexture->GetSizeY(), 1),
							FIntVector(8, 8, 1)
						)
					);

					// Regular border fill and mip generation
					FGenerateMips::ExecuteCompute(GraphBuilder, GMaxRHIFeatureLevel, RDGRenderTarget, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI());
				}
				// Push pull
				else if (CVarMeshPartition_BorderFillMethod->GetInt() == 2)
				{
					const FRDGTextureDesc& TextureDesc = RDGRenderTarget->Desc;						
					FRDGTextureSRVDesc SRVDesc(RDGRenderTarget);
					FRDGTextureUAVDesc UAVDesc(RDGRenderTarget);
					FRDGTextureSRVDesc SRVMaskDesc(RDGMaskTexture);
					FRDGTextureUAVDesc UAVMaskDesc(RDGMaskTexture);
					SRVDesc.NumMipLevels = 1;
					SRVMaskDesc.NumMipLevels = 1;

					{
						TShaderRef<FMeshPartition_FillPullCS> PullShader = ShaderMap->GetShader<FMeshPartition_FillPullCS>();

						// Loop through each level of the mips that require creation and add a dispatch pass per level.
						for (int8 MipLevel = 1; MipLevel < TextureDesc.NumMips; ++MipLevel)
						{
							const FIntPoint DestTextureSize(
								FMath::Max(TextureDesc.Extent.X >> MipLevel, 1),
								FMath::Max(TextureDesc.Extent.Y >> MipLevel, 1));

							SRVDesc.MipLevel = (int8)(MipLevel - 1);
							SRVMaskDesc.MipLevel = (int8)(MipLevel - 1);
							UAVDesc.MipLevel = (int8)(MipLevel);
							UAVMaskDesc.MipLevel = (int8)(MipLevel);

							FMeshPartition_FillPullCS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FMeshPartition_FillPullCS::FParameters>();
							ShaderParams->TexelSize = FVector2f(1.0f / DestTextureSize.X, 1.0f / DestTextureSize.Y);
							ShaderParams->SectionMipIn = GraphBuilder.CreateSRV(SRVDesc);
							ShaderParams->SectionMipOut = GraphBuilder.CreateUAV(UAVDesc);
							ShaderParams->MaskMipIn = GraphBuilder.CreateSRV(SRVMaskDesc);
							ShaderParams->MaskMipOut = GraphBuilder.CreateUAV(UAVMaskDesc);
							ShaderParams->MipSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
							ShaderParams->ResolutionPass = FIntVector4(DestTextureSize.X, DestTextureSize.Y, MipLevel, TextureDesc.NumMips);

							FComputeShaderUtils::AddPass(
								GraphBuilder,
								RDG_EVENT_NAME("MeshPartition_FillPull DestMipLevel=%d", MipLevel),
								ERDGPassFlags::Compute,
								PullShader,
								ShaderParams,
								FComputeShaderUtils::GetGroupCount(DestTextureSize, FComputeShaderUtils::kGolden2DGroupSize)
							);
						}
					}
					{
						TShaderRef<FMeshPartition_FillPushCS> PushShader = ShaderMap->GetShader<FMeshPartition_FillPushCS>();

						// Loop through each level of the mips that require creation and add a dispatch pass per level.
						for (int8 MipLevel = TextureDesc.NumMips - 2; MipLevel >= 0; --MipLevel)
						{
							const FIntPoint DestTextureSize(
								FMath::Max(TextureDesc.Extent.X >> MipLevel, 1),
								FMath::Max(TextureDesc.Extent.Y >> MipLevel, 1));

							SRVDesc.MipLevel = (int8)(MipLevel + 1);
							UAVDesc.MipLevel = (int8)(MipLevel);
							SRVMaskDesc.MipLevel = (int8)(MipLevel);

							FMeshPartition_FillPushCS::FParameters* ShaderParams = GraphBuilder.AllocParameters<FMeshPartition_FillPushCS::FParameters>();
							ShaderParams->TexelSize = FVector2f(1.0f / DestTextureSize.X, 1.0f / DestTextureSize.Y);
							ShaderParams->SectionMipIn = GraphBuilder.CreateSRV(SRVDesc);
							ShaderParams->SectionMipOut = GraphBuilder.CreateUAV(UAVDesc);
							ShaderParams->MaskMipIn = GraphBuilder.CreateSRV(SRVMaskDesc);
							ShaderParams->MipSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
							ShaderParams->ResolutionPass = FIntVector4(DestTextureSize.X, DestTextureSize.Y, MipLevel, TextureDesc.NumMips);

							FComputeShaderUtils::AddPass(
								GraphBuilder,
								RDG_EVENT_NAME("MeshPartition_FillPush DestMipLevel=%d", MipLevel),
								ERDGPassFlags::Compute,
								PushShader,
								ShaderParams,
								FComputeShaderUtils::GetGroupCount(DestTextureSize, FComputeShaderUtils::kGolden2DGroupSize)
							);
						}
					}
				}
				else
				{
					// No border fill but just mip generation
					FGenerateMips::ExecuteCompute(GraphBuilder, GMaxRHIFeatureLevel, RDGRenderTarget, TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI());
				}

				// Make sure the texture is actually managed by RDG as a call to RegisterExternalTexture does not guarantee this if the texture was already registered and then had UseExternalAccessMode() called on it.
				GraphBuilder.UseInternalAccessMode(RDGDestTexture);

				// Copy to regular texture
				FRHICopyTextureInfo CopyInfo = {
					.Size = FIntVector(DestTexture->GetSizeX(), DestTexture->GetSizeY(), 1),
					.SourcePosition = FIntVector(0, 0, 0),
					.DestPosition = FIntVector(0, 0, 0),
					.SourceSliceIndex = 0,
					.DestSliceIndex = (uint32)Page,
					.NumSlices = 1,
					.SourceMipIndex = 0,
					.DestMipIndex = 0,
					.NumMips = RDGRenderTarget->Desc.NumMips
				};

				AddCopyTexturePass(GraphBuilder, RDGRenderTarget, RDGDestTexture, CopyInfo);
			}
			if (RDGDestTexture)
			{
				GraphBuilder.UseExternalAccessMode(RDGDestTexture, ERHIAccess::SRVMask);
			}

			MeshPartition::FRenderCaptureManager::EndCapture();

			GraphBuilder.Execute();
		});
}

void FChannelTextureRenderer::DownloadSectionTextureToAsset(UTexture* InAllocatedTexture)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::DownloadSectionTextureToAsset);
	// Source texture must be a valid Texture2DArray
	UTexture2DArray* AssetSrcTexture = Cast<UTexture2DArray>(InAllocatedTexture);
	if (!IsValid(AssetSrcTexture))
	{
		return;
	}

	ENQUEUE_RENDER_COMMAND(MegaMeshRenderRenderDownloadSectionTexture)([AssetSrcTexture](FRHICommandListImmediate& RHICmdList)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::Render_Command_DownloadSectionTextureToAsset);

		FRDGBuilder GraphBuilder(RHICmdList);
		FRHITexture* RHISrcTexture = AssetSrcTexture->GetResource()->GetTexture2DArrayRHI();
		FRDGTextureRef RDGSrcTexture = RegisterExternalTexture(GraphBuilder, RHISrcTexture, TEXT("SrcTexture"));

		AddReadbackTexturePass(GraphBuilder, RDG_EVENT_NAME("Readback"), RDGSrcTexture, [AssetSrcTexture, RHISrcTexture](FRHICommandListImmediate& RHICmdList)
		{
			int32 NumMips = RHISrcTexture->GetDesc().NumMips;
			int32 NumSlices = RHISrcTexture->GetDesc().ArraySize;

			// Fixel format of the saved data is single component grayscale
			ETextureSourceFormat TSFormat = TSF_G8;
			FReadSurfaceDataFlags ReadDataFlags;
			ReadDataFlags.SetLinearToGamma(false);

			AssetSrcTexture->Source.Init(
				RHISrcTexture->GetSizeX(),
				RHISrcTexture->GetSizeY(),
				NumSlices,
				NumMips,
				TSFormat
			);

			for (int32 MipIndex = 0; MipIndex < NumMips; ++MipIndex)
			{
				ReadDataFlags.SetMip(MipIndex);

				int32 MipSizeX = FMath::Max((int32) RHISrcTexture->GetSizeX() >> MipIndex, 1);
				int32 MipSizeY = FMath::Max((int32) RHISrcTexture->GetSizeY() >> MipIndex, 1);

				int64 AllocatedSize = AssetSrcTexture->Source.CalcMipSize(MipIndex);

				// Collect each slice pixels
				TArray<uint8> MultiSlicePixels;
				MultiSlicePixels.Reserve(AllocatedSize);
				for (int32 SliceIndex = 0; SliceIndex < NumSlices; ++SliceIndex)
				{
					TArray<FColor> Pixels;
					ReadDataFlags.SetArrayIndex(SliceIndex);
					RHICmdList.ReadSurfaceData(RHISrcTexture, FIntRect(0, 0, MipSizeX, MipSizeY), Pixels, ReadDataFlags);
					
					// Extract R component only 
					for (int32 i = 0; i < Pixels.Num(); ++i)
					{
						MultiSlicePixels.Add(Pixels[i].R);
					}
				}

				// Copy the collected pixels to the asset if the number of collected texels matches expectations
				int64 CollectedSize = MultiSlicePixels.Num() * sizeof(uint8);

				if (CollectedSize && CollectedSize == AllocatedSize)
				{
					void* BulkDataDest = AssetSrcTexture->Source.LockMip(MipIndex);

					FMemory::Memcpy(BulkDataDest, MultiSlicePixels.GetData(), AllocatedSize);

					AssetSrcTexture->Source.UnlockMip(MipIndex);
				}
				else
				{
					UE_LOGF(LogTexture, Warning, "Invalid CollectedSize for the downloaded data FChannelTextureRenderer::DownloadSectionTextureToAsset. ChannelTexture is going to be invalid");
				}
			}		
		});

		GraphBuilder.Execute();
	});
	FlushRenderingCommands();

	// Asset resource texture has been populated with the produced texels, notify to save
	AssetSrcTexture->UpdateResource();
	AssetSrcTexture->MarkPackageDirty();
}

} // namespace UE::MeshPartition