// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionChannelCollection.h"

#include <atomic>

#include "Tasks/Task.h"
#include "Async/ParallelFor.h"

#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionModifierTaskGraph.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorModule.h"


#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "Parameterization/DynamicMeshUVEditor.h"
#include "Parameterization/MeshUVPacking.h"
#include "MeshQueries.h"

#include "VEUV/VEUVOptimizer.h"

#include "Util/SizedDisjointSet.h"

#include "UObject/ConstructorHelpers.h"

#include "UObject/Object.h"
#include "VectorUtil.h"

using namespace UE::Geometry;
using namespace UE::Math;

namespace UE::MeshPartition
{
namespace MegaMeshChannelCollectionLocals
{
	namespace Packer
	{
		TArray<FVector2f> EvalRectanglesAndDomainSize(FVector2f& OutInitialDomainSize, const TArray<FBox2f>& InRectangles)
		{
			TArray<FVector2f> RectangleSizes;
			RectangleSizes.Init(FVector2f(0), InRectangles.Num());

			for (int32 i = 0; i < InRectangles.Num(); ++i)
			{
				RectangleSizes[i] = InRectangles[i].GetSize();
				OutInitialDomainSize = FVector2f::Max(OutInitialDomainSize, RectangleSizes[i]);
			}

			return RectangleSizes;
		}

		TArray<int32> SortRectangles(const TArray<FVector2f>& InRectangleSizes)
		{
			TArray<int32> RectangleIndices;
			RectangleIndices.Init(0, InRectangleSizes.Num());
			for (int32 i = 0; i < InRectangleSizes.Num(); ++i)
			{
				RectangleIndices[i] = i;
			}

			RectangleIndices.Sort([InRectangleSizes](const int32& RectA, const int32& RectB)
				{
					const FVector2f& RSA = InRectangleSizes[RectA];
					const FVector2f& RSB = InRectangleSizes[RectB];
					float A = RSA.X * RSA.Y;
					float B = RSB.X * RSB.Y;
					return A > B; // Order from largest to smallest area
				});

			return RectangleIndices;
		}

		int32 FindBestFreeRectangle(const FVector2f& InRectSize, float InMargin, TArray<FBox2f>& InOutFreeRectangles)
		{
			// Adjust rect with Margin
			FVector2f NewRectSize = InRectSize + 2.0f * InMargin;

			int32 BestFreeRectIdx = INDEX_NONE;
			float BestFit = std::numeric_limits<float>::infinity();

			for (int32 i = 0; i < InOutFreeRectangles.Num(); ++i)
			{
				FBox2f& FR = InOutFreeRectangles[i];
				FVector2f FRSize = FR.GetSize();

				if ((NewRectSize.X <= FRSize.X) && (NewRectSize.Y <= FRSize.Y))
				{
					// evaluate how fit is the tested rect within the candidate free rect
					float Fit = (NewRectSize.X * NewRectSize.Y) - (FRSize.X * FRSize.Y);
					if (Fit < BestFit)
					{
						BestFit = Fit;
						BestFreeRectIdx = i;
					}
				}
			}

			return BestFreeRectIdx;
		}

		TArray<FBox2f> SplitFreeRect(const FVector2f& InRectSize, float InMargin, const FBox2f& InFreeRect)
		{
			// Adjust rect with Margin
			FVector2f NewRectSize = InRectSize + 2.0f * InMargin;

			// Split the occupied free rect
			FVector2f RemainingSize = InFreeRect.GetSize() - NewRectSize;

			FVector2f H(InFreeRect.Min.X + NewRectSize.X, InFreeRect.Min.Y);
			FVector2f V(InFreeRect.Min.X, InFreeRect.Min.Y + NewRectSize.Y);

			FVector2f HM(H.X, InFreeRect.Max.Y);
			FVector2f VM(InFreeRect.Max.X, V.Y);

			bool XSpace = (RemainingSize.X > InMargin);
			bool YSpace = (RemainingSize.Y > InMargin);

			TArray<FBox2f> RemainingRects;
			if (XSpace && YSpace)
			{
				if (RemainingSize.X < RemainingSize.Y)
				{
					RemainingRects.Add(FBox2f(V, InFreeRect.Max));
					RemainingRects.Add(FBox2f(H, VM));
				}
				else
				{
					RemainingRects.Add(FBox2f(H, InFreeRect.Max));
					RemainingRects.Add(FBox2f(V, HM));
				}
			}
			else
			{
				if (XSpace)
				{
					RemainingRects.Add(FBox2f(H, InFreeRect.Max));
				}
				if (YSpace)
				{
					RemainingRects.Add(FBox2f(V, InFreeRect.Max));
				}
			}

			return RemainingRects;
		}

		FBox2f PlaceFreeRectAndSplit(const FVector2f& InRectSize, float InMargin, int32 BestFitRectIdx, TArray<FBox2f>& InOutFreeRectangles)
		{
			FBox2f& FreeRect = InOutFreeRectangles[BestFitRectIdx];

			// Assign new location of placed rect
			FBox2f NewRect(FreeRect.Min + InMargin, FreeRect.Min + InMargin + InRectSize);

			TArray<FBox2f> RemainingRects = SplitFreeRect(InRectSize, InMargin, FreeRect);

			InOutFreeRectangles.RemoveAt(BestFitRectIdx);
			InOutFreeRectangles.Append(RemainingRects);

			return NewRect;
		}

		bool Atlas_MaxRects(TArray<FBox2f>& OutDestRectangles, TArray<FBox2f>& OutFreeRects, const FVector2f& InDomainSize, const TArray<int32>& InSrcRectanglesIndices, const TArray<FVector2f>& InSrcRectangleSizes, float InMargin)
		{
			TArray<FBox2f> FreeRectangles = { FBox2f(FVector2f(0), InDomainSize) };

			for (int32 RectIdx : InSrcRectanglesIndices)
			{
				const FVector2f& RectSize = InSrcRectangleSizes[RectIdx];
				int32 BestFreeRectIdx = FindBestFreeRectangle(RectSize, InMargin, FreeRectangles);
				if (BestFreeRectIdx != INDEX_NONE)
				{
					OutDestRectangles[RectIdx] = PlaceFreeRectAndSplit(RectSize, InMargin, BestFreeRectIdx, FreeRectangles);
				}
				else
				{
					return false;
				}
			}

			OutFreeRects = FreeRectangles;
			return true;
		}

		TArray<FBox2f> CreateAtlas(const TArray<FBox2f>& InSrcRectangles, float InMargin, TArray<FBox2f>& OutFreeRects)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::CreateUVLayout_Packer_CreateAtlas);

			if (!ensure(InSrcRectangles.Num() >= 1))
			{
				return {};
			}

			if (InSrcRectangles.Num() == 1)
			{
				float OneMinus2Margins = 1.f - 2.f * InMargin;
				FBox2f Rect = InSrcRectangles[0];
				FVector2f RectSize = Rect.GetSize();
				Rect.Min = FVector2f(InMargin);
				if (RectSize.X > RectSize.Y)
				{
					Rect.Max = FVector2f(InMargin + OneMinus2Margins, InMargin + OneMinus2Margins * RectSize.Y / RectSize.X);

					if (Rect.Max.Y < OneMinus2Margins)
						OutFreeRects.Add(FBox2f(FVector2f(0, Rect.Max.Y + InMargin), FVector2f(1.0f)));
				}
				else
				{
					Rect.Max = FVector2f(InMargin + OneMinus2Margins * RectSize.X / RectSize.Y, InMargin + OneMinus2Margins);
					if (Rect.Max.X < OneMinus2Margins)
						OutFreeRects.Add(FBox2f(FVector2f(Rect.Max.X + InMargin, 0.0f), FVector2f(1.0f)));
				}
				return { Rect };
			}
			else
			{
				// Sort the Rectangles by diminishing area and eval a initial domain size encompassing the largest rect
				FVector2f DomainSize(0);
				TArray<FVector2f> SrcRectanglesSizes = EvalRectanglesAndDomainSize(DomainSize, InSrcRectangles);
				TArray<int32> SrcRectanglesIndices = SortRectangles(SrcRectanglesSizes);

				// Allocate result placed rectangles
				TArray<FBox2f> PlacedRectangles;
				PlacedRectangles.Init(FBox2f(EForceInit::ForceInit), InSrcRectangles.Num());

				// After first domain size Evaluation, the margin needs to grow proportionally
				float MarginSize = InMargin * DomainSize.GetAbsMax();
				float IterationDomainGrowth = 1.1f;
				while (!Atlas_MaxRects(PlacedRectangles, OutFreeRects, DomainSize, SrcRectanglesIndices, SrcRectanglesSizes, MarginSize))
				{
					DomainSize *= IterationDomainGrowth;
					MarginSize = InMargin * DomainSize.GetAbsMax();
				}

				// Fit the domain size as tight as we can around the found arrengement and run the placement algo one more time
				FVector2f  FitDomainBox(0);
				for (const FBox2f& Rect : PlacedRectangles)
				{
					FitDomainBox = FVector2f::Max(FitDomainBox, Rect.Max);
				}
				MarginSize = InMargin * FitDomainBox.GetAbsMax();
				DomainSize = FVector2f(FitDomainBox.GetMax() + 2.0f * MarginSize);
				Atlas_MaxRects(PlacedRectangles, OutFreeRects, DomainSize, SrcRectanglesIndices, SrcRectanglesSizes, MarginSize);

				// Got results
				// Rescale the placed Rectangles
				FVector2f InvDomainSize(1.0f / DomainSize.X, 1.0f / DomainSize.Y);
				for (auto& R : PlacedRectangles)
				{
					R.Min *= InvDomainSize;
					R.Max *= InvDomainSize;
				}
				for (auto& R : OutFreeRects)
				{
					R.Min *= InvDomainSize;
					R.Max *= InvDomainSize;
				}

				// Move is required here because NRVO is not guaranteed here.
				return MoveTemp(PlacedRectangles);
			}
		}
	} // End namespace Packer

	void CreateUVLayout_PerformThroughDynamicMesh(FMeshData& InSectionMesh, TConstArrayView<int32> TriangleIDs, TFunctionRef<void(Geometry::FDynamicMesh3&)> UVLayoutFunc, int32 AssignBaseIndex)
	{
		// Copy the triangles that need new UV unwrap to a dynamic mesh so we can use geometry processing algorithms
		TArray<int32> SectionVIDToMeshVID;
		SectionVIDToMeshVID.Init(INDEX_NONE, InSectionMesh.MaxVertexID());
		TArray<int32> MeshVIDToSectionVID;
		FDynamicMesh3 Mesh;
		for (int32 TID : TriangleIDs)
		{
			const FIndex3i SectionTri = InSectionMesh.GetTriangle(TID);
			FIndex3i ExtractTri;
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				const int32 SectionVID = SectionTri[SubIdx];
				int32& MeshVID = SectionVIDToMeshVID[SectionVID];
				if (MeshVID == INDEX_NONE)
				{
					MeshVID = Mesh.AppendVertex(InSectionMesh.GetVertex(SectionVID));
					// Mesh vertices must be 1:1 w/ the MeshVIDToSectionVID array
					const int32 AddedIndex = MeshVIDToSectionVID.Add(SectionVID);
					checkSlow(AddedIndex == MeshVID);
				}
				ExtractTri[SubIdx] = MeshVID;
			}
			Mesh.AppendTriangle(ExtractTri);
		}

		// Copy per-vertex data separately, so we can add it back as needed
		TArray<FVector3f> MeshNormals;
		TArray<float> MeshWeights;
		TArray<FName> WeightLayerNames = InSectionMesh.GetWeightLayerNames();
		MeshNormals.SetNumUninitialized(MeshVIDToSectionVID.Num());
		MeshWeights.SetNumUninitialized(WeightLayerNames.Num() * MeshVIDToSectionVID.Num());

		TArray<TArray<FVector2f>> AllSourceUVs;
		const int32 NumSourceChannels = InSectionMesh.GetNumSourceUVChannels();
		AllSourceUVs.SetNum(NumSourceChannels);
		for (int32 ChannelIdx = 0; ChannelIdx < NumSourceChannels; ++ChannelIdx)
		{
			AllSourceUVs[ChannelIdx].SetNumUninitialized(MeshVIDToSectionVID.Num());
		}

		for (int32 MeshVID = 0; MeshVID < MeshVIDToSectionVID.Num(); ++MeshVID)
		{
			int32 SectionVID = MeshVIDToSectionVID[MeshVID];
			MeshNormals[MeshVID] = InSectionMesh.GetVertexNormal(SectionVID);

			// Store all source UV channels
			for (int32 ChannelIdx = 0; ChannelIdx < NumSourceChannels; ++ChannelIdx)
			{
				AllSourceUVs[ChannelIdx][MeshVID] = InSectionMesh.GetVertexUV(SectionVID, ChannelIdx);
			}

			int32 MeshWeightBase = MeshVID * WeightLayerNames.Num();
			for (int32 WeightIdx = 0; WeightIdx < WeightLayerNames.Num(); ++WeightIdx)
			{
				MeshWeights[MeshWeightBase + WeightIdx] = InSectionMesh.GetWeightLayerValue(WeightLayerNames[WeightIdx], SectionVID);
			}
		}

		// Do the UV unwrap
		Mesh.EnableAttributes();
		Mesh.Attributes()->SetNumUVLayers(1);
		FDynamicMeshUVOverlay& Overlay = *Mesh.Attributes()->GetUVLayer(0);
		UVLayoutFunc(Mesh);
		// UVLayoutFunc must not change the underlying mesh topology, just the UVs
		if (!ensure(Mesh.MaxVertexID() == MeshNormals.Num() && Mesh.IsCompact()))
		{
			// If the mesh isn't 1:1 w/ the source data, no way to transfer the UVs back
			return;
		}

		// Delete all the triangles that we will unwrap, so we can re-add them below with new UV topology
		for (int32 TID : TriangleIDs)
		{
			InSectionMesh.RemoveTriangle(TID, true);
		}

		// Transfer the new UV overlay elements back to the section mesh, and apply the normals/weights from their source vertices
		TArray<int32> MeshElToNewVertex;
		MeshElToNewVertex.Init(INDEX_NONE, Overlay.MaxElementID());
		for (int32 ElID : Overlay.ElementIndicesItr())
		{
			int32 ParentVID = Overlay.GetParentVertex(ElID);
			int32 NewSectionVID = InSectionMesh.AppendVertex(Mesh.GetVertex(ParentVID));
			MeshElToNewVertex[ElID] = NewSectionVID;

			InSectionMesh.SetChannelUV(NewSectionVID, Overlay.GetElement(ElID));

			for (int32 ChannelIdx = 0; ChannelIdx < NumSourceChannels; ++ChannelIdx)
			{
				InSectionMesh.SetVertexUV(NewSectionVID, AllSourceUVs[ChannelIdx][ParentVID], ChannelIdx);
			}

			InSectionMesh.SetVertexNormal(NewSectionVID, MeshNormals[ParentVID]);
			int32 WeightIndexBase = ParentVID * WeightLayerNames.Num();
			for (int32 WeightIdx = 0; WeightIdx < WeightLayerNames.Num(); ++WeightIdx)
			{
				InSectionMesh.SetWeightLayerValue(WeightLayerNames[WeightIdx], NewSectionVID, MeshWeights[WeightIndexBase + WeightIdx]);
			}
		}
		// Add the unwrapped triangles back to the InSectionMesh
		for (int32 TID : Mesh.TriangleIndicesItr())
		{
			FIndex3i ElTri = Overlay.GetTriangle(TID);
			FIndex3i NewTri;
			if (!ensure(ElTri.A != INDEX_NONE))
			{
				// We should not have unset triangles in the overlay? Implies unwrap failed.
				continue;
			}
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				NewTri[SubIdx] = MeshElToNewVertex[ElTri[SubIdx]];
			}
			int32 NewTID = InSectionMesh.AppendTriangle(NewTri);
			InSectionMesh.SetBaseID(NewTID, AssignBaseIndex);
		}
	}

	/**
	 * Repack a UV overlay's islands without per-island rescaling, using the section domain's target resolution and gutter
	 */
	void PackUVOverlayPreservingScale(Geometry::FDynamicMeshUVOverlay* UVOverlay, const FSectionDomainMapping& InSectionDomainMapping)
	{
		const int32 EffectiveRes = FMath::Min(InSectionDomainMapping.MinImageResolution, InSectionDomainMapping.MaxImageResolution);
		const float GutterTexels = FMath::Max(1.0f, InSectionDomainMapping.GetGutterTexelCount());
		// StandardPack uses a 1-pixel gutter at its rasterization resolution (its GutterSize parameter is only for StackPack)
		// Set TargetRes such that one pixel covers GutterTexels output texels at the final atlas scale.
		// Clamp to a minimum of 2: we need at least 2x2 pixels to have any chance of usefully packing.
		const int32 PackTargetRes = FMath::Max(2, FMath::RoundToInt(EffectiveRes / GutterTexels));

		UVOverlay->SplitBowties();
		Geometry::FDynamicMeshUVPacker Packer(UVOverlay);
		Packer.TextureResolution = PackTargetRes;
		Packer.bAllowFlips = false;
		Packer.bPreserveScale = true;
		Packer.StandardPack();
	}

	void CreateUVLayout_SetTriangleUVsThroughDynamicMeshTools(FSectionDomainMapping& InSectionDomainMapping, FMeshData& InOutSectionMesh, TArray<TArray<int32>>& OutIslandTriangles)
	{
		// Before packing UVs, compute from-scratch UV layouts if needed
		{
			// Search for triangles without valid assigned bases
			TArray<int32> ToUnwrapTIDs;
			for (int32 TriangleID : InOutSectionMesh.TriangleIndicesItr())
			{
				const int32 BaseIdx = InOutSectionMesh.GetBaseID(TriangleID);
				// Unwrap all
				ToUnwrapTIDs.Add(TriangleID);
			}

			// If we found any un-assigned triangles, create new UV layout
			if (!ToUnwrapTIDs.IsEmpty())
			{
				// For unassigned bases, use last base w/ a null base path. Create it if it's not already there.
				int32 UseBaseIndex = 0;

				CreateUVLayout_PerformThroughDynamicMesh(InOutSectionMesh, ToUnwrapTIDs,
														 [InSectionDomainMapping](Geometry::FDynamicMesh3& Mesh)
					{
						{
							// Create temp array of all triangle IDs, for use by UV layout methods below
							TArray<int32> AllMeshTIDs;
							AllMeshTIDs.Reserve(Mesh.TriangleCount());
							for (int32 MeshTID : Mesh.TriangleIndicesItr())
							{
								AllMeshTIDs.Add(MeshTID);
							}

							// Quick initial UV layout method, TODO: revisit algorithm choice and scale
							Geometry::FDynamicMeshUVEditor UVEditor(&Mesh, 0, true);
							UVEditor.SetTriangleUVsFromBoxProjection(AllMeshTIDs, [](const FVector3d& V) { return V; }, Geometry::FFrame3d(), FVector3d(100, 100, 100));
							PackUVOverlayPreservingScale(Mesh.Attributes()->GetUVLayer(0), InSectionDomainMapping);
							UVEditor.ScaleUVAreaTo3DArea(AllMeshTIDs, true, .00001f);
						}
					},
					UseBaseIndex
				);

			}
		}

		TArray<TArray<int32>> IslandTriangles = { {} };
		IslandTriangles[0].Reserve(InOutSectionMesh.MaxTriangleID());
		for (int32 TriangleId : InOutSectionMesh.TriangleIndicesItr())
		{
			IslandTriangles[0].Add(TriangleId);
		}

		OutIslandTriangles = MoveTemp(IslandTriangles);
	}

	void CreateUVLayout_SetTriangleUVsFromPlaneProjection(FMeshData& InOutSectionMesh, const FPlaneProjectionLayoutOptions& InOptions, TArray<TArray<int32>>& OutIslandTriangles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::CreateUVLayout_SetTriangleUVsFromPlaneProjection);

		if (InOutSectionMesh.TriangleCount() == 0 || InOutSectionMesh.VertexCount() == 0)
		{
			OutIslandTriangles = { {} };
			return;
		}
		const int32 MaxVertexID = InOutSectionMesh.MaxVertexID();

		// Choose the projection-plane normal
		FVector3d FrameNormal;
		if (InOptions.NormalSource == EPlaneProjectionNormalSource::AverageNormal)
		{
			FVector3d Accum = FVector3d::ZeroVector;
			for (int32 TID : InOutSectionMesh.TriangleIndicesItr())
			{
				FVector3d A, B, C;
				InOutSectionMesh.GetTriVertices(TID, A, B, C);
				// Sum 2*Area*Normal for area-weighted average
				Accum += FVector3d::CrossProduct(B - A, C - A);
			}
			FrameNormal = Accum.GetSafeNormal();
		}
		else
		{
			FrameNormal = FVector3d(InOptions.FixedNormal).GetSafeNormal();
		}
		if (FrameNormal.IsNearlyZero())
		{
			FrameNormal = FVector3d::ZAxisVector;
		}

		const FFrame3d ProjectionFrame(FVector3d::ZeroVector, FrameNormal);

		// UVs will be renormalized downstream; set this scale just to keep them in a sensible range / to match box projection
		constexpr double Scale = 1.0 / 100.0;

		// Plane projection is single-valued per position, so vertex UVs map 1:1 and no UV-split is needed.
		ParallelFor(MaxVertexID, [&InOutSectionMesh, &ProjectionFrame, Scale](int32 VertexID)
		{
			if (!InOutSectionMesh.IsVertex(VertexID))
			{
				return;
			}
			const FVector3d Pos = InOutSectionMesh.GetVertex(VertexID);
			const FVector2d UV = ProjectionFrame.ToPlaneUV(Pos, 2) * Scale;
			InOutSectionMesh.SetChannelUV(VertexID, FVector2f(UV));
		});

		// Assign all triangles Base ID 0, since they all land in the same UV island
		TArray<TArray<int32>> IslandTriangles = { {} };
		IslandTriangles[0].Reserve(InOutSectionMesh.TriangleCount());
		for (int32 TID : InOutSectionMesh.TriangleIndicesItr())
		{
			InOutSectionMesh.SetBaseID(TID, 0);
			IslandTriangles[0].Add(TID);
		}
		OutIslandTriangles = MoveTemp(IslandTriangles);
	}

	void CreateUVLayout_SetTriangleUVsFromBoxProjection(FMeshData& InOutSectionMesh, TArray<TArray<int32>>& OutIslandTriangles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::CreateUVLayout_SetTriangleUVsFromBoxProjection);


		FMeshData* Mesh = &InOutSectionMesh;
		FFrame3d BoxFrame;
		FVector3d BoxDimensions(100, 100, 100);
		TMap<int32, int32> BaseIDMap;

		int32 NumTriangles = InOutSectionMesh.MaxTriangleID();
		if (!NumTriangles) return;

		int32 NumVertices = InOutSectionMesh.MaxVertexID();
		if (!NumVertices) return;

		for (int32 tid : InOutSectionMesh.TriangleIndicesItr())
		{
			int32 BaseID = InOutSectionMesh.GetBaseID(tid);
			if (!BaseIDMap.Contains(BaseID))
			{
				BaseIDMap.Add(BaseID, BaseIDMap.Num());
			}
		}

		const int Minor1s[3] = { 1, 0, 0 };
		const int Minor2s[3] = { 2, 2, 1 };
		const int Minor1Flip[3] = { -1, 1, 1 };
		const int Minor2Flip[3] = { -1, -1, 1 };

		auto GetTriNormal = [&](int32 tid) -> FVector3d
			{
				FVector3d A, B, C;
				Mesh->GetTriVertices(tid, A, B, C);
				return VectorUtil::Normal(A, B, C);
			};

		double ScaleY = (FMathd::Abs(BoxDimensions.Y) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.Y) : 1.0;
		double ScaleX = (FMathd::Abs(BoxDimensions.X) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.X) : 1.0;
		double ScaleZ = (FMathd::Abs(BoxDimensions.Z) > FMathf::ZeroTolerance) ? (1.0 / BoxDimensions.Z) : 1.0;
		FVector3d Scale(ScaleX, ScaleY, ScaleZ);

		auto GetMappedMeshBaseID = [&Mesh, &BaseIDMap](int32 tid) {
			int32 ID = Mesh->GetBaseID(tid);
			return BaseIDMap[ID];
		};

		TArray<std::atomic<int32>> IslandIdBins;
		IslandIdBins.SetNumZeroed(6 * BaseIDMap.Num());

		// compute assignments to the available planes based on face normals
		TArray<FVector3d> TriNormals;
		TArray<FIndex2i> TriangleBoxPlaneAssignments;
		TriNormals.SetNum(NumTriangles);
		TriangleBoxPlaneAssignments.SetNum(NumTriangles);
		TArray<int32> IndexMap;
		IndexMap.SetNum(Mesh->MaxTriangleID());
		ParallelFor(NumTriangles, [&](int32 i)
		{
			if (!Mesh->IsTriangle(i))
			{
				return;
			}
			int32 tid = i;
			TriNormals[i] = GetTriNormal(tid);
			FVector3d ScaledNormal = BoxFrame.ToFrameVector(TriNormals[i]);
			ScaledNormal *= Scale;
			FVector3d NAbs(FMathd::Abs(ScaledNormal.X), FMathd::Abs(ScaledNormal.Y), FMathd::Abs(ScaledNormal.Z));
			int MajorAxis = NAbs[0] > NAbs[1] ? (NAbs[0] > NAbs[2] ? 0 : 2) : (NAbs[1] > NAbs[2] ? 1 : 2);
			double MajorAxisSign = FMathd::Sign(ScaledNormal[MajorAxis]);
			int32 Bucket = (MajorAxisSign > 0) ? (MajorAxis + 3) : MajorAxis;
			int32 MappedBaseID = GetMappedMeshBaseID(tid);
			//int BucketID = MappedBaseID * 6 + Bucket;
			int BucketID = Bucket;
			TriangleBoxPlaneAssignments[i] = FIndex2i(MajorAxis, BucketID);
			IndexMap[tid] = i;

			IslandIdBins[BucketID].fetch_add(1, std::memory_order_relaxed);
		});

		// Maybe later...
		//	// Optimize face assignments. Small regions are grouped with larger neighbour regions.
		//	//if (MinIslandTriCount > 1)
		//	//{
		//	//	FMeshConnectedComponents Components(Mesh);
		//	//	Components.FindConnectedTriangles(Triangles, [&](int32 t1, int32 t2) { return TriangleBoxPlaneAssignments[IndexMap[t1]] == TriangleBoxPlaneAssignments[IndexMap[t2]]; });
		//	//	FMeshRegionGraph RegionGraph;
		//	//	RegionGraph.BuildFromComponents(*Mesh, Components, [&](int32 ComponentIdx) { int32 tid = Components[ComponentIdx].Indices[0]; return TriangleBoxPlaneAssignments[IndexMap[tid]].A; });
		//	//	// todo: similarity measure should probably take normals into account
		//	//	bool bMerged = RegionGraph.MergeSmallRegions(MinIslandTriCount - 1,
		//	//		[&](int32 A, int32 B) { return RegionGraph.GetRegionTriCount(A) > RegionGraph.GetRegionTriCount(B); });
		//	//	bool bSwapped = RegionGraph.OptimizeBorders();
		//	//	if (bMerged || bSwapped)
		//	//	{
		//	//		int32 N = RegionGraph.MaxRegionIndex();
		//	//		for (int32 k = 0; k < N; ++k)
		//	//		{
		//	//			if (RegionGraph.IsRegion(k))
		//	//			{
		//	//				int32 MajorAxis = RegionGraph.GetExternalID(k);
		//	//				const TArray<int32>& Tris = RegionGraph.GetRegionTris(k);
		//	//				for (int32 tid : Tris)
		//	//				{
		//	//					int32 i = IndexMap[tid];
		//	//					FVector3d ScaledNormal = BoxFrame.ToFrameVector(TriNormals[i]) * Scale;
		//	//					double MajorAxisSign = FMathd::Sign(ScaledNormal[MajorAxis]);
		//	//					int Bucket = (MajorAxisSign > 0) ? (MajorAxis + 3) : MajorAxis;
		//	//					TriangleBoxPlaneAssignments[i] = FIndex2i(MajorAxis, Bucket);
		//	//				}
		//	//			}
		//	//		}
		//	//	}
		//	//}

		TArray<TArray<int32>> IslandTriangles;

		TArray<int32> BaseIDAndBucketToIslandID;
		BaseIDAndBucketToIslandID.Init(INDEX_NONE, IslandIdBins.Num());
		int32 NumIslandIDs = 0;
		for (int32 i = 0; i < BaseIDAndBucketToIslandID.Num(); ++i)
		{
			int32 NumTrianglesInIsland = IslandIdBins[i].load();
			if (NumTrianglesInIsland > 0)
			{
				BaseIDAndBucketToIslandID[i] = NumIslandIDs;
				IslandTriangles.AddDefaulted();
				IslandTriangles[NumIslandIDs].Reserve(NumTrianglesInIsland);
				NumIslandIDs++;
			}
		}

		auto ProjAxis = [](const FVector3d& P, int Axis1, int Axis2, float Axis1Scale, float Axis2Scale)
			{
				return FVector2f(float(P[Axis1]) * Axis1Scale, float(P[Axis2]) * Axis2Scale);
			};

		TMap<FIndex2i, int32> BaseToOverlayVIDMap;

		struct NewTriangleElement
		{
			FIndex3i Triangle;
			int32 TID;
			int32 IslandID;
		};
		TArray<NewTriangleElement> NewUVTriangles;
		TArray<int32> NewUVBases;
		TArray<FVector2f> NewUVs;

		TArray<TArray<int32, TInlineAllocator<2>>> BaseToUVElements;
		BaseToUVElements.SetNumZeroed(NumVertices);

		for (int32 tid : Mesh->TriangleIndicesItr())
		{
			FIndex3i BaseTri = Mesh->GetTriangle(tid);
			FIndex2i TriBoxInfo = TriangleBoxPlaneAssignments[tid];
			FVector3d N = BoxFrame.ToFrameVector(TriNormals[tid]);

			int MajorAxis = TriBoxInfo.A;
			int MajorAxisSign = (N[MajorAxis] > 0.0) ? 1 : ((N[MajorAxis] < 0.0) ? -1 : 0);

			FMathd::Sign(N[MajorAxis]);
			int Minor1 = Minor1s[MajorAxis];
			int Minor2 = Minor2s[MajorAxis];

			int BucketID = TriBoxInfo.B;
			int32 IslandID = BaseIDAndBucketToIslandID[BucketID];

			FIndex3i ElemTri;
			bool bIsNewElemTriangle = false;
			for (int32 j = 0; j < 3; ++j)
			{
				int32 BaseVertex = BaseTri[j];
				FIndex2i ElementKey(BaseVertex, BucketID);
				const int32* FoundElementID = BaseToOverlayVIDMap.Find(ElementKey);
				if (FoundElementID == nullptr)
				{
					FVector3d Pos = Mesh->GetVertex(BaseVertex);
					FVector3d TransformPos = (Pos);
					FVector3d BoxPos = BoxFrame.ToFramePoint(TransformPos);
					BoxPos *= Scale;

					FVector2f UV = ProjAxis(BoxPos, Minor1, Minor2, float(MajorAxisSign * Minor1Flip[MajorAxis]), (float)Minor2Flip[MajorAxis]);

					// by default the uv element is the same as the base vertex...
					int32 NewElementID = BaseVertex;
					auto& FoundBaseUVIndices = BaseToUVElements[BaseVertex];
					if (FoundBaseUVIndices.IsEmpty())
					{
						FoundBaseUVIndices.Add(BaseVertex);
						Mesh->SetChannelUV(BaseVertex, UV);
						ElemTri[j] = BaseVertex;
					}
					// ... unless the base index has already been used for a different UV value, this needs to be a new uv element
					else
					{ 
						bIsNewElemTriangle = true;
						NewElementID = NewUVs.Num();
						NewUVs.Add(UV);
						NewUVBases.Add(BaseVertex);
						NewElementID = -(NewElementID + 1); // encode new element id as negative in new triangle to differentiate from existing indices
						ElemTri[j] = NewElementID;
						FoundBaseUVIndices.Add(NewElementID);
					}

					BaseToOverlayVIDMap.Add(ElementKey, NewElementID);
				}
				else
				{
					ElemTri[j] = *FoundElementID;
					if (ElemTri[j] < 0)
					{
						bIsNewElemTriangle = true;
					}
				}
			}

			if (bIsNewElemTriangle)
			{
				NewUVTriangles.Add({ ElemTri, tid, IslandID });
			}
			else
			{
				// overwrite base id of the triangle with island id
				Mesh->SetBaseID(tid, IslandID);
				IslandTriangles[IslandID].Add(tid);
			}
		}

		//// Above process can introduce bowties, so we split any bowties on new element IDs
		//// SplitBowtiesOnUVElements(NewUVIndices, true);

		if (!NewUVTriangles.IsEmpty())
		{
			// Need to allocate new vertices for the incoming splitted vertices
			Mesh->ReserveAdditionalVertices(NewUVs.Num());

			// Allocate the new vertices as duplicate of their base one, duplicate the vertex values EXCEPT for the channel UV
			const int32 NumSourceChannels = Mesh->GetNumSourceUVChannels();
			for (int32 i = 0; i < NewUVBases.Num(); ++i)
			{
				int32 BaseId = NewUVBases[i];

				auto BaseVertex = Mesh->GetVertex(BaseId);
				int32 NewId = Mesh->AppendVertex(BaseVertex);

				Mesh->SetVertexNormal(NewId, Mesh->GetVertexNormal(BaseId));
				for (const auto& Name : Mesh->GetWeightLayerNames())
				{
					Mesh->SetWeightLayerValue(Name, NewId, Mesh->GetWeightLayerValue(Name, BaseId));
				}

				Mesh->SetChannelUV(NewId, NewUVs[i]);

				for (int32 ChannelIdx = 0; ChannelIdx < NumSourceChannels; ++ChannelIdx)
				{
					Mesh->SetVertexUV(NewId, Mesh->GetVertexUV(BaseId, ChannelIdx), ChannelIdx);
				}

				NewUVBases[i] = NewId; // overwrite the NewUV base id with the new base id
			}

			//Now update triangles which connect new vertices
			for (auto& TriElem : NewUVTriangles)
			{	
				FIndex3i NewTriangle;
				for (int32 j = 0; j < 3; ++j)
				{
					int32 ElemId = TriElem.Triangle[j];
					if (ElemId < 0)
					{
						ElemId = -(ElemId + 1);
						ElemId = NewUVBases[ElemId];
					}
					NewTriangle[j] = ElemId;
				}

				// remove source triangle, and add the new one
				Mesh->RemoveTriangle(TriElem.TID, false);
				int32 NewTriId = Mesh->AppendTriangle(NewTriangle);

				Mesh->SetBaseID(NewTriId, TriElem.IslandID);
				IslandTriangles[TriElem.IslandID].Add(NewTriId); // new triangle will reuse the new tid
			}
		}
		// the full new uv overlay has been created, now let's transfer the information to the mesh

		// Mesh triangles have been bucketed in separate islands based on the projection plane
		// the source Base ID of the triangles was also used to sort faces
		// No vertex is reused among island because a vertex holds only one UV.

		// The algorithm above results in potential overlapping faces projected in the same bucket.
		// We must separate these overlapping faces in separate "sub-islands" that will be packed at different locations in the final uv domain.

		// IslandVertexSets is a disjoint set mapping vertices to their root vertex defining a group of connected triangles
		UE::Geometry::FSizedDisjointSet GroupedVertexSets(Mesh->MaxVertexID());

		// ASSUMING these overlapping faces are part of disconnected sub-islands,
		// we traverse the faces of the bucketed island and identify the group of connected faces creating disconnected sub-islands
		for (const TArray<int32>& TriangleIndices : IslandTriangles)
		{
			for (int32 tid : TriangleIndices)
			{
				UE::Geometry::FIndex3i Triangle = Mesh->GetTriangle(tid);
				// Union vertices  B and C of this tri to vertex A, so that tri-connected bases/vertices will be in the same set
				GroupedVertexSets.Union(Triangle.B, Triangle.A);
				GroupedVertexSets.Union(Triangle.C, Triangle.A);
			}
		}

		// Collect all the unique seeds aka islands discovered
		TArray<int32> TriangleGroupIndexMap;
		int32 NumTriangleGroups = GroupedVertexSets.CompactedGroupIndexToGroupID(nullptr, &TriangleGroupIndexMap, 3);

		// THe disjointed island Triangles that will be returned
		TArray<TArray<int32>> DisjointedIslandTriangles;
		DisjointedIslandTriangles.SetNumZeroed(NumTriangleGroups);

		// Traverse mesh once again per triangles
		// and bin in their final triangle lists
		for (const TArray<int32>& TriangleIndices : IslandTriangles)
		{
			for (int32 tid : TriangleIndices)
			{
				UE::Geometry::FIndex3i Triangle = Mesh->GetTriangle(tid);
				int32 SeedId = GroupedVertexSets.Find(Triangle.A);
				checkSlow(SeedId == GroupedVertexSets.Find(Triangle.B));
				checkSlow(SeedId == GroupedVertexSets.Find(Triangle.C));

				int32 GroupId = TriangleGroupIndexMap[SeedId];
				DisjointedIslandTriangles[GroupId].Add(tid);

				// Record the group id as the base id in the mesh for consistency
				Mesh->SetBaseID(tid, GroupId);
			}
		}

		// And done
		OutIslandTriangles = MoveTemp(DisjointedIslandTriangles);
	}

	void CreateUVLayout_Evaluate3DDomainDimensions(FSectionDomainMapping& OutSectionDomainMapping, const FMeshData& InSectionMesh)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::CreateUVLayout_Evaluate3DDomainDimensions);

		// Measure the true 3d area of the mesh
		double Section_Area3D = 0;
		for (int32 TriangleId : InSectionMesh.TriangleIndicesItr())
		{
			Geometry::FIndex3i Triangle = InSectionMesh.GetTriangle(TriangleId);

			// Compute the triangle area and add up to the base's area
			FVector3d P0 = InSectionMesh.GetVertex(Triangle.A);
			FVector3d P1 = InSectionMesh.GetVertex(Triangle.B);
			FVector3d P2 = InSectionMesh.GetVertex(Triangle.C);
			double TriangleArea3D = 0.5 * FVector3d::CrossProduct((P1 - P0), (P2 - P0)).Length();
			Section_Area3D += TriangleArea3D;
		}

		// Project the 3d area corresponding to a fully filled uv domain
		// ignore the gutter area
		// And estimate the size of that "fully filled" 3d square
		double SectionSize = FMath::Sqrt(Section_Area3D);

		// From the required Texel size (in 3d unit) evaluate the ideal image resolution
		OutSectionDomainMapping.MinImageResolution = SectionSize / OutSectionDomainMapping.TexelSize3D; // Concrete image size per uv space unit [texel/uv]
		OutSectionDomainMapping.ImageResolution = OutSectionDomainMapping.MinImageResolution;

		OutSectionDomainMapping.Area3D = Section_Area3D;
		OutSectionDomainMapping.AreaUV = 1.0;
								
		OutSectionDomainMapping.Size3D = SectionSize;
		OutSectionDomainMapping.SizeUV = 1.0;
	}

	// When bUVsAreNormalized is true the UVs are assumed to be in their final packed [0,1] space, so Section_AreaUV represents actual packing density and is used
	// to size the texture (keeping RasterResolution and TexcoordMetrics consistent so world-texel-size matches the target). When false the UVs are in whatever
	// arbitrary scale the UV layout method picked pre-pack, so only Section_Area3D is used (the downstream packer renormalises to [0,1]).
	void CreateUVLayout_EvaluateUVDomainDimensions(FSectionDomainMapping& InOutSectionDomainMapping, const FMeshData& InSectionMesh, bool bUVsAreNormalized)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::CreateUVLayout_EvaluateUVDomainDimensions);

		// Validate TexelSize3D at entry: a non-positive value would cause division-by-zero / overflow
		// downstream when computing UVScaledImageResolution. Substitute the engine default.
		if (!(InOutSectionDomainMapping.TexelSize3D > 0.0))
		{
			UE_LOGF(LogMegaMeshEditor, Warning,
				"CreateUVLayout_EvaluateUVDomainDimensions: TexelSize3D must be > 0 (got %f). Falling back to %f.",
				InOutSectionDomainMapping.TexelSize3D,
				(double)FChannelTextureRenderer::DefaultTexelSize);
			InOutSectionDomainMapping.TexelSize3D = FChannelTextureRenderer::DefaultTexelSize;
		}

		// Measure the true 3d area of the mesh and the corresponding uv area
		double Section_Area3D = 0;
		double Section_AreaUV = 0;
		for (int32 TriangleId : InSectionMesh.TriangleIndicesItr())
		{
			Geometry::FIndex3i Triangle = InSectionMesh.GetTriangle(TriangleId);

			// Compute the triangle area and add up to the base's area
			FVector3d P0 = InSectionMesh.GetVertex(Triangle.A);
			FVector3d P1 = InSectionMesh.GetVertex(Triangle.B);
			FVector3d P2 = InSectionMesh.GetVertex(Triangle.C);
			double TriangleArea3D = 0.5 * FVector3d::CrossProduct((P1 - P0), (P2 - P0)).Length();
			Section_Area3D += TriangleArea3D;

			// Compute the UV triangle area and add up to the base's area
			FVector2f UV0 = InSectionMesh.GetChannelUV(Triangle.A);
			FVector2f UV1 = InSectionMesh.GetChannelUV(Triangle.B);
			FVector2f UV2 = InSectionMesh.GetChannelUV(Triangle.C);
			double TriangleAreaUV = 0.5 * FMath::Abs(FVector2f::CrossProduct((UV1 - UV0), (UV2 - UV0)));
			Section_AreaUV += TriangleAreaUV;
		}

		const double TexelSize_Unit = InOutSectionDomainMapping.TexelSize3D; // Size of one texel in position unit [u] assigned from definition

		// SectionSize is the world-space side length of a unit UV square (also fed to TexcoordMetrics as Size3D).
		// Post-pack with valid AreaUV: sqrt(Area3D / AreaUV) accounts for actual packing density, so RasterResolution and TexcoordMetrics stay consistent.
		// Otherwise (pre-pack with arbitrary AreaUV, or degenerate AreaUV == 0): fall back to sqrt(Area3D), treating the section as if UVs filled [0,1]. The downstream packer renormalises in the pre-pack case; Size3D is not consumed there.
		const double SectionSize = (bUVsAreNormalized && Section_AreaUV > 0) ? FMath::Sqrt(Section_Area3D / Section_AreaUV) : FMath::Sqrt(Section_Area3D);

		const double NaturalImageResolution = SectionSize / TexelSize_Unit;

		// Ensure the lower image resolution is at least 1 so that downstream texture allocation
		// receives a valid (>0) extent (e.g. a degenerate / sub-pixel mesh still gets a 1x1 texture).
		const int32 EffectiveMinImageResolution = FMath::Max(1, InOutSectionDomainMapping.MinImageResolution);

		// Clamp to [Min, Max] in floating-point space BEFORE the int32 cast so that +inf / values
		// exceeding INT32_MAX (e.g. from huge meshes or tiny TexelSize) cannot overflow into a
		// negative int32 and produce an invalid (negative-when-reinterpreted) RasterResolution.
		const double IdealImageResolution = FMath::Clamp(
			NaturalImageResolution,
			(double)EffectiveMinImageResolution,
			(double)InOutSectionDomainMapping.MaxImageResolution);

		// Round up to a multiple of 4 to maximize compatibility with texture compression while honoring ChannelTexelSize as closely as possible.
		int32 ImageResolution = FMath::DivideAndRoundUp(FMath::CeilToInt32(IdealImageResolution), 4) * 4;
		ImageResolution = FMath::Min(InOutSectionDomainMapping.MaxImageResolution, ImageResolution);


		InOutSectionDomainMapping.Size3D = SectionSize;
		InOutSectionDomainMapping.SizeUV = SectionSize;

		InOutSectionDomainMapping.Area3D = Section_Area3D;
		InOutSectionDomainMapping.AreaUV = Section_AreaUV;

		InOutSectionDomainMapping.ImageResolution = ImageResolution;
	}


	void CreateUVLayout_PackIslandsNormalizeUVDomain(FSectionDomainMapping& OutSectionDomainMapping,  FMeshData& InOutSectionMesh, const TArray<TArray<int32>>& InIslandTriangles)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::CreateUVLayout_PackIslandsNormalizeUVDomain);

		// UVs are in whatever scale the UV layout method picked; the packer renormalises to [0,1] just after this. AreaUV is not yet meaningful for sizing.
		MegaMeshChannelCollectionLocals::CreateUVLayout_EvaluateUVDomainDimensions(OutSectionDomainMapping, InOutSectionMesh, /*bUVsAreNormalized*/ false);



		int32 NumVertices = InOutSectionMesh.MaxVertexID();

		// Each vertex belongs to one island which should match all the tringle's BaseID referencing it
		TArray<int32> VertexIslandIDs;
		VertexIslandIDs.Init(INDEX_NONE, NumVertices);


		// Starting from the island triangles
		TArray<FBox2f> IslandUVBoxes;
		IslandUVBoxes.SetNumZeroed(InIslandTriangles.Num());

		for (int32 IslandID = 0; IslandID < InIslandTriangles.Num(); ++IslandID)
		{
			const TArray<int32> Triangles = InIslandTriangles[IslandID];
			FBox2f IslandUVBox;
			IslandUVBox.Init();

			for (int32 tid : Triangles)
			{
				FIndex3i Triangle = InOutSectionMesh.GetTriangle(tid);

				int32 TriangleBaseID = InOutSectionMesh.GetBaseID(tid);
				check(TriangleBaseID == IslandID);

				for (int32 j = 0; j < 3; ++j)
				{
					int32& VIID = VertexIslandIDs[Triangle[j]];
					if (VIID != IslandID)
					{
						if (VIID == INDEX_NONE)
						{
							VIID = IslandID;
						}
						else
						{
							// This means the vertex is referenced by more than one island. and that is not a suported case with our mesh.
							// this should be resolved higher up in the algorithm
							// which will propably create artifacts
							ensure(VIID == IslandID); // TODO : Fix this case
						}
					}

					FVector2f UV = InOutSectionMesh.GetChannelUV(Triangle[j]);
					IslandUVBox += UV;
				}
			}

			IslandUVBoxes[IslandID] = IslandUVBox;
		}

		// then run the atlas algortihm
		// Find the new UV domain for the section
		float IslandMargin = OutSectionDomainMapping.GetGutterUV();
		TArray<FBox2f> FreeRects;
		TArray<FBox2f> SectionIslandUVDomains = MegaMeshChannelCollectionLocals::Packer::CreateAtlas(IslandUVBoxes, IslandMargin, FreeRects);

		// Reassign the UV at their final position in the island placed rect somewhere in the normalized uv domain
		ParallelFor(NumVertices, [&](int VertexId)
		{
			if (!InOutSectionMesh.IsVertex(VertexId))
			{
				return;
			}

			int32 IslandID = VertexIslandIDs[VertexId];

			if (IslandID == INDEX_NONE)
			{
				// This means that we got a vertex that wasn't a part of a triangle, e.g. if a modifier
				//  appended disconnected vertices. If we disallow/ensure against this, it should be done
				//  further upstream.
				return;
			}

			FVector2f UV = InOutSectionMesh.GetChannelUV(VertexId);

			const FBox2f& SrcBox = IslandUVBoxes[IslandID];
			const FBox2f& DstBox = SectionIslandUVDomains[IslandID];

			FVector2f NUV = (UV - SrcBox.Min) / SrcBox.GetSize();

			FVector2f AUV = NUV * DstBox.GetSize() + DstBox.Min;

			UV = AUV;

			InOutSectionMesh.SetChannelUV(VertexId, AUV);
		});
	}

	/**
	 * Use the Volume Encoded UV (VEUV) algorithm to compute and layout UVs for the section mesh.
	 * @return true on success, false on failure. On failure, InOutSectionMesh will be restored to the original input mesh.
	 */
	bool CreateUVLayout_SetTriangleUVsThroughVolumeEncoding(FSectionDomainMapping& InSectionDomainMapping, FMeshData& InOutSectionMesh, EChannelUVUnwrapQuality Quality,
			const FVEUVLayoutOptions& VEUVOptions, const FTransform& WorldTransform)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::CreateUVLayout_SetTriangleUVsThroughVolumeEncoding);

		// Clear the mesh to rebuild w/ new UVs, keeping the old mesh to copy out attributes
		FMeshData OldMesh = MoveTemp(InOutSectionMesh);
		InOutSectionMesh.Clear();

		// Build a compact VEUV input mesh and a MeshVID->compact map so VEUV's FVertexSource indices
		// can be remapped back to OldMesh VIDs for attribute reads.
		const int32 MaxVID = OldMesh.MaxVertexID();
		TArray<int32> CompactToMeshVID;
		TArray<int32> MeshVIDToCompact;
		MeshVIDToCompact.Init(INDEX_NONE, MaxVID);

		VEUV::FMesh InputMesh;
		InputMesh.Vertices.Reserve(MaxVID);

		for (int32 VertexID : OldMesh.VertexIndicesItr())
		{
			MeshVIDToCompact[VertexID] = InputMesh.Vertices.Num();
			CompactToMeshVID.Add(VertexID);

			InputMesh.Vertices.Add(FVector3f(OldMesh.GetVertex(VertexID)));
		}

		InputMesh.Faces.Reserve(OldMesh.TriangleCount());
		for (int32 TriangleID : OldMesh.TriangleIndicesItr())
		{
			const FIndex3i Tri = OldMesh.GetTriangle(TriangleID);
			InputMesh.Faces.Add(FInt32Vector3(
				MeshVIDToCompact[Tri.A],
				MeshVIDToCompact[Tri.B],
				MeshVIDToCompact[Tri.C]));
		}

		FVEUVConfig Config = FVEUVConfig::Default;
		if (Quality == EChannelUVUnwrapQuality::Preview)
		{
			Config = FVEUVConfig::LowQuality();
		}

		Config.VoxelCount = VEUVOptions.VoxelCount;
		Config.Packing.AtlasPackWidth = FMath::Max(1, InSectionDomainMapping.ImageResolution);
		// Limit chart raster to 1024, for overlap detection
		Config.Charting.RasterizationWidth = FMath::RoundUpToPowerOfTwo(FMath::Clamp(InSectionDomainMapping.ImageResolution, 64, 1024));

		const double AreaInSquareMeters = InSectionDomainMapping.Area3D / 10000.0;
		Config.Sampling.TotalSamples = FMath::Max(1, FMath::RoundToInt(VEUVOptions.SamplesPerSquareMeter * AreaInSquareMeters));

		Config.DebugWorldTransform = WorldTransform;

		const VEUV::FResult Result = VEUV::FOptimizer::Compute(InputMesh, Config);

		auto IsResultAcceptable = [](const VEUV::FResult& InResult) -> bool
		{
			// Treat any chart-level failure flag other than Inverted as failure
			if (EnumHasAnyFlags(InResult.Status, ~VEUV::EChartStatus::Inverted))
			{
				UE_LOGF(LogMegaMeshEditor, Warning,
					"VEUV result rejected: chart-level failure flags%s%s%s (FailedChartCount=%d)",
					EnumHasAnyFlags(InResult.Status, VEUV::EChartStatus::Empty) ? " Empty" : "",
					EnumHasAnyFlags(InResult.Status, VEUV::EChartStatus::NonFinite) ? " NonFinite" : "",
					EnumHasAnyFlags(InResult.Status, VEUV::EChartStatus::PackingFailed) ? " PackingFailed" : "",
					InResult.Stats.FailedChartCount);
				return false;
			}

			// Output faces missing UVs
			if (InResult.Stats.UnassignedFaceCount > 0)
			{
				UE_LOGF(LogMegaMeshEditor, Warning,
					"VEUV result rejected: %d of %d output faces have no chart assignment",
					InResult.Stats.UnassignedFaceCount, InResult.Stats.OutputFaces);
				return false;
			}

			// Accept a small fraction of inverted triangles; only fail for larger folds
			constexpr float InvertedAreaToleranceFrac = 0.01f;
			if (InResult.Stats.InvertedUVArea > InResult.Stats.TotalUVArea * InvertedAreaToleranceFrac)
			{
				const double InvertedAreaPct = InResult.Stats.TotalUVArea > 0.0f ? 100.0 * InResult.Stats.InvertedUVArea / InResult.Stats.TotalUVArea : 0.0;
				UE_LOGF(LogMegaMeshEditor, Warning,
					"VEUV result rejected: %.3f%% of UV area inverted (threshold %.3f%%)",
					InvertedAreaPct, 100.0 * InvertedAreaToleranceFrac);
				return false;
			}

			return true;
		};

		if (!IsResultAcceptable(Result) && !VEUVOptions.bAcceptOutputOnFailure)
		{
			InOutSectionMesh = MoveTemp(OldMesh);
			return false;
		}

		// Init matching attribs
		const int32 NumSourceChannels = OldMesh.GetNumSourceUVChannels();
		InOutSectionMesh.SetNumSourceUVChannels(NumSourceChannels);
		const TArray<FName> WeightLayerNames = OldMesh.GetWeightLayerNames();
		for (const FName& WeightName : WeightLayerNames)
		{
			InOutSectionMesh.InitializeWeightLayer(WeightName);
		}

		auto BlendFromSources =
			[&OldMesh, &InOutSectionMesh, &CompactToMeshVID, NumSourceChannels, &WeightLayerNames]
			(const VEUV::FVertexSource& Src, int32 NewVID)
		{
			// Blend / re-normalize normals
			FVector3f Normal = FVector3f::ZeroVector;
			for (int32 J = 0; J < 3; ++J)
			{
				if (Src.Vertices[J] != INDEX_NONE)
				{
					Normal += OldMesh.GetVertexNormal(CompactToMeshVID[Src.Vertices[J]]) * Src.Weights[J];
				}
			}
			Normal.Normalize();
			InOutSectionMesh.SetVertexNormal(NewVID, Normal);

			// Blend UVs
			for (int32 ChannelIdx = 0; ChannelIdx < NumSourceChannels; ++ChannelIdx)
			{
				FVector2f UV = FVector2f::ZeroVector;
				for (int32 J = 0; J < 3; ++J)
				{
					if (Src.Vertices[J] != INDEX_NONE)
					{
						UV += OldMesh.GetVertexUV(CompactToMeshVID[Src.Vertices[J]], ChannelIdx) * Src.Weights[J];
					}
				}
				InOutSectionMesh.SetVertexUV(NewVID, UV, ChannelIdx);
			}

			// Blend weight layers
			for (const FName& WeightName : WeightLayerNames)
			{
				float Value = 0.0f;
				for (int32 J = 0; J < 3; ++J)
				{
					if (Src.Vertices[J] != INDEX_NONE)
					{
						Value += OldMesh.GetWeightLayerValue(WeightName, CompactToMeshVID[Src.Vertices[J]]) * Src.Weights[J];
					}
				}
				InOutSectionMesh.SetWeightLayerValue(WeightName, NewVID, Value);
			}
		};

		const int32 OutputVertexCount = Result.OutputMesh.Vertices.Num();
		TArray<int32> OutputToNewMeshVID;
		OutputToNewMeshVID.SetNumUninitialized(OutputVertexCount);

		for (int32 I = 0; I < OutputVertexCount; ++I)
		{
			const FVector3f& Pos = Result.OutputMesh.Vertices[I];
			const int32 NewVID = InOutSectionMesh.AppendVertex(FVector3d(Pos.X, Pos.Y, Pos.Z));
			OutputToNewMeshVID[I] = NewVID;

			InOutSectionMesh.SetChannelUV(NewVID, Result.VertexUVs[I]);
			BlendFromSources(Result.VertexSources[I], NewVID);
		}

		// Append triangles; use VEUV's per-face chart indices as the BaseID
		// (Note this will pass-through INDEX_NONE for unassigned faces)
		for (int32 FaceIdx = 0; FaceIdx < Result.OutputMesh.Faces.Num(); ++FaceIdx)
		{
			const FInt32Vector3& Face = Result.OutputMesh.Faces[FaceIdx];
			const int32 NewTID = InOutSectionMesh.AppendTriangle(FIndex3i(
				OutputToNewMeshVID[Face.X],
				OutputToNewMeshVID[Face.Y],
				OutputToNewMeshVID[Face.Z]));

			InOutSectionMesh.SetBaseID(NewTID, Result.FaceChartIndices[FaceIdx]);
		}

		return true;
	}
} //Namespace MegaMeshChannelCollectionLocals

void FChannelTextureRenderer::CreateSectionUVLayout(FMeshData& InOutSectionMesh, const FChannelCollectionUVLayoutOptions& GenerationOptions, const FTransform& OptionalWorldTransform)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FChannelTextureRenderer::CreateSectionUVLayout);

	
	FSectionDomainMapping DomainMapping{
		.TexelSize3D = GenerationOptions.TexelSize,
		.MaxImageResolution = GenerationOptions.MaxTextureResolution,
		.GutterTexelCount = GenerationOptions.GutterTexelCount
	};
	MegaMeshChannelCollectionLocals::CreateUVLayout_Evaluate3DDomainDimensions(DomainMapping, InOutSectionMesh);

	TArray<TArray<int32>> IslandTriangles;

	switch (GenerationOptions.UVLayoutMethod)
	{
	case EChannelCollectionUVLayoutMethod::FastBoxProject:
		MegaMeshChannelCollectionLocals::CreateUVLayout_SetTriangleUVsFromBoxProjection(InOutSectionMesh, IslandTriangles);
		MegaMeshChannelCollectionLocals::CreateUVLayout_PackIslandsNormalizeUVDomain(DomainMapping, InOutSectionMesh, IslandTriangles);
		break;

	case EChannelCollectionUVLayoutMethod::VolumeEncoded:
	{
		// VEUV packs charts into the atlas internally; only re-evaluate the final UV-domain dimensions.
PRAGMA_DISABLE_EXPERIMENTAL_WARNINGS
		const bool bVEUVSucceeded = MegaMeshChannelCollectionLocals::CreateUVLayout_SetTriangleUVsThroughVolumeEncoding(
			DomainMapping, InOutSectionMesh, GenerationOptions.UVQuality, GenerationOptions.VEUV, OptionalWorldTransform);
PRAGMA_ENABLE_EXPERIMENTAL_WARNINGS

		if (bVEUVSucceeded)
		{
			// VEUV packed charts into the [0,1] atlas, so the UVs are normalized at this point.
			MegaMeshChannelCollectionLocals::CreateUVLayout_EvaluateUVDomainDimensions(DomainMapping, InOutSectionMesh, /*bUVsAreNormalized*/ true);
			break;
		}
		// On failure, deliberately fall through to ReferenceBoxProject.
		[[fallthrough]];
	}

	case EChannelCollectionUVLayoutMethod::ReferenceBoxProject:
		MegaMeshChannelCollectionLocals::CreateUVLayout_SetTriangleUVsThroughDynamicMeshTools(DomainMapping, InOutSectionMesh, IslandTriangles);
		MegaMeshChannelCollectionLocals::CreateUVLayout_PackIslandsNormalizeUVDomain(DomainMapping, InOutSectionMesh, IslandTriangles);
		break;

	case EChannelCollectionUVLayoutMethod::PlaneProject:
		MegaMeshChannelCollectionLocals::CreateUVLayout_SetTriangleUVsFromPlaneProjection(InOutSectionMesh, GenerationOptions.PlaneProjection, IslandTriangles);
		MegaMeshChannelCollectionLocals::CreateUVLayout_PackIslandsNormalizeUVDomain(DomainMapping, InOutSectionMesh, IslandTriangles);
		break;

	default:
		checkNoEntry();
		break;
	}
}

void FChannelTextureRenderer::GenerateSectionMeshUVs(const FMeshData& InSectionMesh, FChannelTextureRenderer::FSection& InSection, float InTexelSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChannelTextureRenderer::GenerateSectionMeshUVs);

	// Texcoords have been generated in the processing step

	// Grab the section indices
	TArray<int32> SectionIndices;
	SectionIndices.Reserve(InSectionMesh.TriangleCount() * 3);
	for (int32 TriangleId : InSectionMesh.TriangleIndicesItr())
	{
		Geometry::FIndex3i Triangle = InSectionMesh.GetTriangle(TriangleId);
		SectionIndices.Append({ Triangle.A, Triangle.B, Triangle.C });
	}

	// Collect the section UVs from the section mesh
	int32 NumVertices = InSectionMesh.MaxVertexID();
	TArray<FVector2f> SectionUVs;
	SectionUVs.SetNumZeroed(NumVertices);
	ParallelFor(NumVertices, [&](int VertexId)
		{
			if (!InSectionMesh.IsVertex(VertexId))
			{
				return;
			}
			SectionUVs[VertexId] = InSectionMesh.GetChannelUV(VertexId);
		});

	// Collect the outline edges
	TArray<FInt32Vector3> SectionOutlineEdges;
	{
		int32 NumTriangles = SectionIndices.Num() / 3;
		TArray<FInt32Vector4> AllEdges; // Container of the 3 edges per triangle of ALL the triangles of the mesh
		AllEdges.SetNumUninitialized(3 * NumTriangles);

		// Collect ALL the edges
		ParallelFor(NumTriangles, [&](int TriangleId)
			{
				int32 ElementOffset = TriangleId * 3;
				int32 A = SectionIndices[ElementOffset + 0];
				int32 B = SectionIndices[ElementOffset + 1];
				int32 C = SectionIndices[ElementOffset + 2];

				// Use the (triangleid + 1) to identify the triangle and the sign indicate the edge direction 
				int32 TID = TriangleId + 1;

				AllEdges[ElementOffset + 0] = (A < B ? FIntVector4(A, B, TID, 0) : FIntVector4(B, A, -TID, 0));
				AllEdges[ElementOffset + 1] = (B < C ? FIntVector4(B, C, TID, 0) : FIntVector4(C, B, -TID, 0));
				AllEdges[ElementOffset + 2] = (C < A ? FIntVector4(C, A, TID, 0) : FIntVector4(A, C, -TID, 0));
			});

		// Sort the edges so the same edges are bundled sequentially
		AllEdges.Sort([](const FInt32Vector4& A, const FInt32Vector4& B)
			{
				if (A.X != B.X) // Sort by X first
				{
					return A.X < B.X;
				}
				else if (A.Y != B.Y) // Sort by Y second if X are the same
				{
					return A.Y < B.Y;
				}
				else // Sort by abs(Z) if XY are the same these are the same edges belonging to different triangles
				{
					return FMath::Abs(A.Z) < FMath::Abs(B.Z);
				}
			});

		// Extract the outline edges which are the the edges with no matching pair
		FInt32Vector4* CurrentEdge = nullptr;
		for (int32 eid = 0; eid < AllEdges.Num(); ++eid)
		{
			FIntVector4* ThisEdge = AllEdges.GetData() + eid;

			if (!CurrentEdge) // No current at this point so this new edge the current one
			{
				CurrentEdge = ThisEdge;
			}
			else if ((CurrentEdge->X == ThisEdge->X) && (CurrentEdge->Y == ThisEdge->Y))
			{
				// This edge is the matching pair of current edge
				CurrentEdge->W = ThisEdge->Z;
				CurrentEdge = nullptr;
			}
			else
			{
				// Current edge is different from this edge, which means it is an isolated outline edge
				SectionOutlineEdges.Add({ CurrentEdge->X, CurrentEdge->Y, CurrentEdge->Z });

				// And this edge is the new current
				CurrentEdge = ThisEdge;
			}
		}
	}

	int32 NumTriangles = SectionIndices.Num() / 3;
	TArray<int32> SectionIslandPerVertex;
	SectionIslandPerVertex.Init(-1, NumVertices);
	TArray<int32> SectionIslandPerTriangles;
	SectionIslandPerTriangles.Init(-1, NumTriangles);

	double OutUVDomainSize = 0;
	int32 OutImageResolution = 0;

	FSectionDomainMapping InOutDomainMapping{
		.TexelSize3D = InTexelSize,
		.MaxImageResolution = DefaultMaxImageResolution
	};
	// UVs in the mesh are the final packed [0,1] layout produced by the processing step, so account for actual packing density when sizing the texture.
	MegaMeshChannelCollectionLocals::CreateUVLayout_EvaluateUVDomainDimensions(InOutDomainMapping, InSectionMesh, /*bUVsAreNormalized*/ true);

	FBox2f UVDomain({ 0,0 }, { 1,1 });
	InSection.Islands_Bases = { {0} };
	InSection.Islands_UVBox = { UVDomain };
	InSection.SectionIslands_UVBox = { UVDomain };
	InSection.Bases_UVBox = { UVDomain };

	// FMeshData uses per-vertex UVs (not overlays), so just create a trivial mapping.
	// If/when FMeshData gets split uv support, this will need to be updated.
	TArray<int32> UVElementToVertexID;
	UVElementToVertexID.Reserve(NumVertices);
	for (int32 i = 0; i < NumVertices; ++i)
	{
		UVElementToVertexID.Add(i);
	}

	InSection.UVmesh.BaseIndicesRanges = { { 0, SectionIndices.Num() } };
	InSection.UVmesh.Indices = MoveTemp(SectionIndices);
	InSection.UVmesh.UVs = MoveTemp(SectionUVs);
	InSection.UVmesh.Outlines = MoveTemp(SectionOutlineEdges);
	InSection.UVmesh.UVElementToVertexID = MoveTemp(UVElementToVertexID);
	InSection.UVmesh.VertexCount = NumVertices;

	InSection.RasterResolution = FUint32Vector2((uint32)InOutDomainMapping.ImageResolution);
	InSection.TexcoordMetrics = FVector2f(InOutDomainMapping.Size3D);
	InSection.DomainMapping = InOutDomainMapping;
}

FChannelTextureRenderer::FSection FChannelTextureRenderer::BuildSectionFromCachedTopology(const FChannelRenderUVMeshTopology& InTopology)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(BuildSectionFromCachedTopology);
	
	FSection OutSection;

	OutSection.UVmesh.Indices = InTopology.Indices;
	OutSection.UVmesh.UVs = InTopology.UVs;
	OutSection.UVmesh.UVElementToVertexID = InTopology.UVElementToVertexID;
	OutSection.UVmesh.VertexCount = InTopology.VertexCount;
	
	OutSection.DomainMapping = InTopology.DomainMapping;
	OutSection.RasterResolution = InTopology.RasterResolution;
	OutSection.TexcoordMetrics = InTopology.TexcoordMetrics;

	FBox2f UVDomain({ 0,0 }, { 1,1 });
	OutSection.Islands_Bases = { {0} };
	OutSection.Islands_UVBox = { UVDomain };
	OutSection.SectionIslands_UVBox = { UVDomain };
	OutSection.Bases_UVBox = { UVDomain };
	
	OutSection.UVmesh.BaseIndicesRanges = { { 0, InTopology.Indices.Num() } };
	
	return OutSection;
}

MeshPartition::FChannelTextureRenderer::FChannelRenderUVMeshTopology FChannelTextureRenderer::GenerateUVMeshTopology(const Geometry::FDynamicMesh3& InSectionMesh, FVector InScale, float InTexelSize)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(GenerateUVMeshTopology);

	FChannelRenderUVMeshTopology Topology;

	const Geometry::FDynamicMeshAttributeSet* Attributes = InSectionMesh.Attributes();
	if (!Attributes || Attributes->NumUVLayers() == 0)
	{
		return Topology;
	}

	const Geometry::FDynamicMeshUVOverlay* UVOverlay = Attributes->GetUVLayer(0);
	if (!UVOverlay)
	{
		return Topology;
	}

	// FDynamicMesh3 indices are not necessarily compact:
	TArray<int32> VertexIDToIndex;
	{
		int32 DenseVertexIndex = 0;
		VertexIDToIndex.SetNum(InSectionMesh.MaxVertexID());
		for (int VertexID : InSectionMesh.VertexIndicesItr())
		{
			VertexIDToIndex[VertexID] = DenseVertexIndex++;
		}
	}

	TArray<FVector2f> SectionUVs;
	TArray<int32> UVElementToVertexID;
	TArray<int32> UVElementIDToDenseIndex;

	SectionUVs.Reserve(UVOverlay->ElementCount());
	UVElementToVertexID.Reserve(UVOverlay->ElementCount());
	UVElementIDToDenseIndex.SetNum(UVOverlay->MaxElementID());

	int32 DenseUVIndex = 0;
	for (int32 ElementID : UVOverlay->ElementIndicesItr())
	{
		const int32 ParentVertexID = UVOverlay->GetParentVertex(ElementID);
		if (ParentVertexID == Geometry::FDynamicMesh3::InvalidID)
		{
			// Orphan element (live in the overlay pool but never bound to a triangle). Safe to skip:
			// no triangle references it, so UVElementIDToDenseIndex[ElementID] won't be read below.
			continue;
		}

		FVector2f UV = UVOverlay->GetElement(ElementID);
		SectionUVs.Add(UV);

		const int32 DenseVertexIdx = VertexIDToIndex[ParentVertexID];
		UVElementToVertexID.Add(DenseVertexIdx);

		UVElementIDToDenseIndex[ElementID] = DenseUVIndex++;
	}

	TArray<int32> SectionIndices;
	SectionIndices.Reserve(InSectionMesh.TriangleCount() * 3);

	for (int32 TriangleId : InSectionMesh.TriangleIndicesItr())
	{
		Geometry::FIndex3i UVTriangle = UVOverlay->GetTriangle(TriangleId);
		if (!ensure(UVTriangle.A != Geometry::FDynamicMesh3::InvalidID))
		{
			continue;
		}

		for (int Index = 0; Index < 3; ++Index)
		{
			int32 DenseIndex = UVElementIDToDenseIndex[UVTriangle[Index]];
			UVTriangle[Index] = DenseIndex;
		}
		SectionIndices.Append({ UVTriangle.A, UVTriangle.B, UVTriangle.C });
	}


	// Compute domain dimensions
	FSectionDomainMapping DomainMapping{
		.TexelSize3D = InTexelSize,
		.MaxImageResolution = DefaultMaxImageResolution
	};

	// Validate TexelSize3D at entry: a non-positive value would cause division-by-zero / overflow
	// downstream when computing UVScaledImageResolution. Substitute the engine default.
	if (!(DomainMapping.TexelSize3D > 0.0))
	{
		UE_LOGF(LogMegaMeshEditor, Warning,
			"GenerateUVMeshTopology: TexelSize3D must be > 0 (got %f). Falling back to %f.",
			DomainMapping.TexelSize3D,
			(double)FChannelTextureRenderer::DefaultTexelSize);
		DomainMapping.TexelSize3D = FChannelTextureRenderer::DefaultTexelSize;
	}

	double Section_Area3D = 0;
	double Section_AreaUV = 0;

	for (int32 TriangleId : InSectionMesh.TriangleIndicesItr())
	{
		Geometry::FIndex3i VertexTriangle = InSectionMesh.GetTriangle(TriangleId);
		Geometry::FIndex3i UVTriangle = UVOverlay->GetTriangle(TriangleId);

		if (UVTriangle.A == Geometry::FDynamicMesh3::InvalidID)
		{
			continue; // Skip triangles without UVs
		}

		// Compute scaled 3D triangle area
		FVector3d P0 = InSectionMesh.GetVertex(VertexTriangle.A) * InScale;
		FVector3d P1 = InSectionMesh.GetVertex(VertexTriangle.B) * InScale;
		FVector3d P2 = InSectionMesh.GetVertex(VertexTriangle.C) * InScale;
		double TriangleArea3D = 0.5 * FVector3d::CrossProduct((P1 - P0), (P2 - P0)).Length();
		Section_Area3D += TriangleArea3D;

		// Compute UV triangle area
		FVector2f UV0 = UVOverlay->GetElement(UVTriangle.A);
		FVector2f UV1 = UVOverlay->GetElement(UVTriangle.B);
		FVector2f UV2 = UVOverlay->GetElement(UVTriangle.C);
		double TriangleAreaUV = 0.5 * FMath::Abs(FVector2f::CrossProduct((UV1 - UV0), (UV2 - UV0)));
		Section_AreaUV += TriangleAreaUV;
	}

	// Calculate texture resolution from area ratios
	double ProjectedSectionDomainArea3D = (Section_AreaUV <= 0 ? 0.0 : Section_Area3D / Section_AreaUV);
	double SectionSize = FMath::Sqrt(ProjectedSectionDomainArea3D);

	double TexelSize_Unit = DomainMapping.TexelSize3D;
	double UVScaledImageResolution = SectionSize / TexelSize_Unit;

	// Ensure the lower image resolution is at least 1 so that downstream texture allocation
	// receives a valid (>0) extent (e.g. a degenerate / sub-pixel mesh still gets a 1x1 texture).
	const int32 EffectiveMinImageResolution = FMath::Max(1, DomainMapping.MinImageResolution);

	// Clamp to [Min, Max] in floating-point space BEFORE the int32 cast so that +inf / values
	// exceeding INT32_MAX (e.g. from huge meshes or tiny TexelSize) cannot overflow into a
	// negative int32 and produce an invalid (negative-when-reinterpreted) RasterResolution.
	const double IdealImageResolution = FMath::Clamp(
		UVScaledImageResolution,
		(double)EffectiveMinImageResolution,
		(double)DomainMapping.MaxImageResolution);

	// Round up to a multiple of 4 to maximize compatibility with texture compression while honoring ChannelTexelSize as closely as possible.
	int32 ImageResolution = FMath::DivideAndRoundUp(FMath::CeilToInt32(IdealImageResolution), 4) * 4;
	ImageResolution = FMath::Min(DomainMapping.MaxImageResolution, ImageResolution);

	DomainMapping.Size3D = SectionSize;
	DomainMapping.SizeUV = SectionSize;
	DomainMapping.Area3D = Section_Area3D;
	DomainMapping.AreaUV = Section_AreaUV;
	DomainMapping.ImageResolution = ImageResolution;

	// Store in topology struct
	Topology.Indices = MoveTemp(SectionIndices);
	Topology.UVs = MoveTemp(SectionUVs);
	Topology.UVElementToVertexID = MoveTemp(UVElementToVertexID);
	Topology.VertexCount = InSectionMesh.VertexCount();

	Topology.DomainMapping = DomainMapping;
	Topology.RasterResolution = FUint32Vector2((uint32)ImageResolution);
	Topology.TexcoordMetrics = FVector2f(DomainMapping.Size3D);

	return Topology;
}

} // namespace UE::MeshPartition
