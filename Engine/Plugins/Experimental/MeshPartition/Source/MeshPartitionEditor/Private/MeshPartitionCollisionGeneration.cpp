// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionCollisionGeneration.h"

#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "IndexTypes.h"
#include "MeshPartitionEditorModule.h"
#include "Math/NumericLimits.h"
#include "MeshPartitionDependencyInterface.h"
#include "MeshAdapter/MeshVertexNormals.h"
#include "MeshSimplification.h"
#include "MeshConstraintsUtil.h"
#include "Operations/MeshClusterSimplifier.h"
#include "Operations/MeshIsoCurves.h"
#include "ProjectionTargets.h"

#include "ProfilingDebugging/ScopedTimers.h"

namespace UE::MeshPartitionCollisionGenerationLocals
{
	// struct to organize additional QEM simplifier configuration that is not currently exposed to the user
	struct FQEMSimplifyMeshOptions
	{
		bool bAllowSeamCollapse = true;
		bool bAllowSeamSmoothing = true;
		bool bAllowSeamSplits = true;
		bool bPreserveVertexPositions = false;
		bool bRetainQuadricMemory = true;
		bool bAutoCompact = false;
	};

	template<typename SimplificationType>
	void SimplifyDynamicMesh(UE::Geometry::FDynamicMesh3& EditMesh, const UE::MeshPartition::Collision::FCollisionSimplificationSettings& SimplifySettings)
	{
		using namespace UE::Geometry;

		if (SimplifySettings.EdgeLength < 0 && SimplifySettings.ErrorTolerance < 0)
		{
			// no simplification requested
			return;
		}

		FQEMSimplifyMeshOptions QEMOptions;

		// we need attributes for material IDs (if present), but can clear other attributes
		if (EditMesh.HasAttributes())
		{
			EditMesh.Attributes()->SetNumUVLayers(0);
			EditMesh.Attributes()->SetNumNormalLayers(0);
			EditMesh.Attributes()->SetNumWeightLayers(0);
		}

		// undo splitting done by uv layout
		FMergeCoincidentMeshEdges Welder(&EditMesh);
		Welder.Apply();

		// compute smooth vertex normals to guide error tolerances
		FMeshNormals::QuickComputeVertexNormals(EditMesh);
		
		// Standard simplifier setup
		SimplificationType Simplifier(&EditMesh);
		Simplifier.ProjectionMode = SimplificationType::ETargetProjectionMode::NoProjection;

		bool bUsingGeometricTolerance = SimplifySettings.ErrorTolerance >= 0;
		TUniquePtr<FDynamicMesh3> ProjectionMesh;
		TUniquePtr<FDynamicMeshAABBTree3> Spatial;
		FMeshProjectionTarget ProjectionTarget;
		if (bUsingGeometricTolerance)
		{
			ProjectionMesh = MakeUnique<FDynamicMesh3>(EditMesh);
			Spatial = MakeUnique<FDynamicMeshAABBTree3>(ProjectionMesh.Get(), true);
			ProjectionTarget.Mesh = ProjectionMesh.Get();
			ProjectionTarget.Spatial = Spatial.Get();
			Simplifier.SetProjectionTarget(&ProjectionTarget);
		}

		Simplifier.DEBUG_CHECK_LEVEL = 0;
		Simplifier.bRetainQuadricMemory = QEMOptions.bRetainQuadricMemory;
		Simplifier.bAllowSeamCollapse = QEMOptions.bAllowSeamCollapse;
		Simplifier.bLimitConstrainedSeamMovement = SimplifySettings.MaterialBoundaryDistanceTolerance > 0;
		if (QEMOptions.bAllowSeamCollapse)
		{
			Simplifier.SetEdgeFlipTolerance(1.e-5);
			if (EditMesh.HasAttributes())
			{
				EditMesh.Attributes()->SplitAllBowties();	// eliminate any bowties that might have formed on attribute seams.
			}
		}

		EEdgeRefineFlags MeshBoundaryConstraints = EEdgeRefineFlags::CollapseOnly;
		EEdgeRefineFlags GroupBorderConstraints = EEdgeRefineFlags::NoConstraint;
		EEdgeRefineFlags MaterialBorderConstraints = (EEdgeRefineFlags)((int)EEdgeRefineFlags::CollapseOnly | (int)EEdgeRefineFlags::NoTopologyMerge);

		FMeshConstraints Constraints;
		FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, EditMesh,
			MeshBoundaryConstraints, GroupBorderConstraints, MaterialBorderConstraints,
			QEMOptions.bAllowSeamSplits, QEMOptions.bAllowSeamSmoothing, QEMOptions.bAllowSeamCollapse);
		constexpr double MeshBoundaryCornerAngleThresholdDegrees = 45.;
		FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(Constraints, FMeshConstraintsUtil::EBoundaryType::Mesh, EditMesh, MeshBoundaryCornerAngleThresholdDegrees);
		// Note: Corner angles are not very useful for material boundaries, as boundaries can be jagged from initial translation of vertex weights to triangle labels
		constexpr double MaterialBoundaryCornerAngleThresholdDegrees = 180.;
		FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(Constraints, FMeshConstraintsUtil::EBoundaryType::MaterialID, EditMesh, MaterialBoundaryCornerAngleThresholdDegrees, SimplifySettings.MaterialBoundaryDistanceTolerance);
		Simplifier.SetExternalConstraints(MoveTemp(Constraints));

		if (QEMOptions.bPreserveVertexPositions)
		{
			Simplifier.CollapseMode = SimplificationType::ESimplificationCollapseModes::MinimalExistingVertexError;
		}
		Simplifier.RegularizeWeight = .0000001;

		// Scale error measure w/ the normal Z-alignment
		auto CosDegrees = [](float AngleDeg) -> double { return (double)FMath::Cos(FMath::DegreesToRadians(AngleDeg)); };
		TFunction<double(const FDynamicMesh3&, int VertexA, int VertexB)> CustomScaleF = nullptr;
		if (SimplifySettings.bScaleAccuracyViaNormal)
		{
			FVector3f AlignNormal = (FVector3f)SimplifySettings.ScaleAccuracyNormalDirection;
			if (!Normalize(AlignNormal))
			{
				AlignNormal = FVector3f::UnitZ();
			}
			CustomScaleF = [&SimplifySettings, AlignNormal,
				ScaleRange = FVector2d(CosDegrees(SimplifySettings.ScaleFromMaxNormalAngle), CosDegrees(SimplifySettings.ScaleToMinNormalAngle))]
			(const FDynamicMesh3& Mesh, int32 VA, int32 VB) -> double
			{
				float ZA = Mesh.GetVertexNormal(VA) | AlignNormal;
				float ZB = Mesh.GetVertexNormal(VB) | AlignNormal;
				return FMath::GetMappedRangeValueClamped<double>(ScaleRange, { 1., (double)SimplifySettings.LocalAccuracyScale }, FMath::Max(ZA, ZB));
			};
		}

		if (bUsingGeometricTolerance)
		{
			Simplifier.GeometricErrorConstraint = SimplificationType::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
			Simplifier.GeometricErrorTolerance = SimplifySettings.ErrorTolerance;
			Simplifier.CustomGeometricErrorScaleF = CustomScaleF;
		}

		if (SimplifySettings.EdgeLength > 0)
		{
			Simplifier.CustomEdgeLengthScaleF = CustomScaleF;
			Simplifier.SimplifyToEdgeLength(SimplifySettings.EdgeLength);
		}
		else
		{
			Simplifier.SimplifyToTriangleCount(1);
		}
	}

	// These checks reproduce tests in Chaos::CleanTrimesh
	bool ChaosIsDegenTri(const FVector3f& A, const FVector3f& B, const FVector3f& C)
	{
		if (A == B || A == C || B == C)
		{
			return true;
		}
		// anything that fails the first check should also fail this, but Chaos does both so doing the same here...
		const float SquaredArea = FVector3f::CrossProduct(A - B, A - C).SizeSquared();
		if (SquaredArea < UE_SMALL_NUMBER)
		{
			return true;
		}
		return false;
	}

	void DynamicMeshToCollisionMeshData(UE::Geometry::FDynamicMesh3& Mesh, FTriMeshCollisionData& CollisionData, bool bCopyMaterialIDs)
	{
		using namespace UE::Geometry;

		CollisionData.Vertices.Reset();
		CollisionData.Indices.Reset();
		CollisionData.MaterialIndices.Reset();

		TArray<int32> VertexMap;

		// copy vertices
		VertexMap.SetNum(Mesh.MaxVertexID());
		CollisionData.Vertices.Reserve(Mesh.VertexCount());
		for (int32 vid : Mesh.VertexIndicesItr())
		{
			VertexMap[vid] = CollisionData.Vertices.Add((FVector3f)Mesh.GetVertex(vid));
		}

		// copy triangles
		CollisionData.Indices.Reserve(Mesh.TriangleCount());
		const FDynamicMeshMaterialAttribute* MaterialAttrib = bCopyMaterialIDs && Mesh.HasAttributes() && Mesh.Attributes()->HasMaterialID() ? Mesh.Attributes()->GetMaterialID() : nullptr;
		if (MaterialAttrib)
		{
			CollisionData.MaterialIndices.Reserve(Mesh.TriangleCount());
		}
		
		for (int32 TID : Mesh.TriangleIndicesItr())
		{
			FIndex3i Tri = Mesh.GetTriangle(TID);
			FTriIndices Triangle;
			Triangle.v0 = VertexMap[Tri.A];
			Triangle.v1 = VertexMap[Tri.B];
			Triangle.v2 = VertexMap[Tri.C];

			// Filter out triangles which will cause physics system to emit degenerate-geometry warnings.
			const FVector3f& A = CollisionData.Vertices[Triangle.v0];
			const FVector3f& B = CollisionData.Vertices[Triangle.v1];
			const FVector3f& C = CollisionData.Vertices[Triangle.v2];
			if (!ChaosIsDegenTri(A, B, C))
			{
				CollisionData.Indices.Add(Triangle);

				if (MaterialAttrib)
				{
					int32 MaterialID = MaterialAttrib->GetValue(TID);
					uint16 ConvMaterialID = (MaterialID >= 0 && MaterialID <= TNumericLimits<uint16>::Max()) ? static_cast<uint16>(MaterialID) : (uint16)0;
					CollisionData.MaterialIndices.Add(ConvMaterialID);
				}
			}
		}
	}

	UE::Geometry::FTriangleMeshAdapterd GetCollisionMeshAdapter(FTriMeshCollisionData* CollisionData)
	{
		using namespace UE::Geometry;
		check(CollisionData);

		FTriangleMeshAdapterd Adapter;
		Adapter.GetTriangle = [CollisionData](int32 TID)
		{
			const FTriIndices& Inds = CollisionData->Indices[TID];
			return FIndex3i(Inds.v0, Inds.v1, Inds.v2);
		};
		int32 NumTris = CollisionData->Indices.Num();
		Adapter.IsTriangle = [NumTris](int32 TID) { return TID >= 0 && TID < NumTris; };
		Adapter.MaxTriangleID = [NumTris]() { return NumTris; };
		Adapter.TriangleCount = [NumTris]() { return NumTris; };
		int32 NumVerts = CollisionData->Vertices.Num();
		Adapter.IsVertex = [NumVerts](int32 VID) { return VID >= 0 && VID < NumVerts; };
		Adapter.MaxVertexID = [NumVerts]() { return NumVerts; };
		Adapter.VertexCount = [NumVerts]() { return NumVerts; };
		Adapter.GetVertex = [CollisionData](int32 VID)
		{
			return (FVector3d)CollisionData->Vertices[VID];
		};
		Adapter.GetChangeStamp = []() { return 0; }; // Not applicable here, but part of the adapter api
		return Adapter;
	}

	UE::Geometry::FDynamicMesh3 ConvertCollisionMeshToDynamicMesh(const FTriMeshCollisionData& MeshCollisionData)
	{
		using namespace UE::Geometry;

		FDynamicMesh3 Result;
		for (FVector3f V : MeshCollisionData.Vertices)
		{
			Result.AppendVertex((FVector3d)V);
		}

		bool bHasMaterials = !MeshCollisionData.MaterialIndices.IsEmpty();
		FDynamicMeshMaterialAttribute* MaterialIDs = nullptr;
		if (bHasMaterials)
		{
			Result.EnableAttributes();
			// Enable attributes only for material IDs; disable UVs and normals
			Result.Attributes()->SetNumUVLayers(0);
			Result.Attributes()->SetNumNormalLayers(0);
			Result.Attributes()->EnableMaterialID();
			MaterialIDs = Result.Attributes()->GetMaterialID();
		}
		for (int32 Idx = 0; Idx < MeshCollisionData.Indices.Num(); ++Idx)
		{
			FTriIndices TriInds = MeshCollisionData.Indices[Idx];
			FIndex3i Tri(TriInds.v0, TriInds.v1, TriInds.v2);
			int32 AddedTID = Result.AppendTriangle(Tri);
			if (AddedTID >= 0 && bHasMaterials)
			{
				MaterialIDs->SetValue(AddedTID, (int32)MeshCollisionData.MaterialIndices[Idx]);
			}
		}
		return Result;
	}

	// Convert mesh to a collision mesh, cutting triangles along physical material boundaries as needed
	FTriMeshCollisionData CutAndConvertMeshToCollisionMesh(const UE::MeshPartition::FMeshData& Mesh, TConstArrayView<UPhysicalMaterial*> ReferencePhysicalMaterials,
		TConstArrayView<UE::MeshPartition::FPhysicalMaterialChannel> PhysicalMaterialChannels, bool bCut, double VertexSnapTolerance)
	{
		using namespace UE::Geometry;

		// Convert initial geometry
		FTriMeshCollisionData CollisionMesh;
		CollisionMesh.Vertices.Reserve(Mesh.VertexCount());
		TArray<int32> CompactVertexMap;
		CompactVertexMap.Init(INDEX_NONE, Mesh.MaxVertexID());
		for (int32 VID = 0; VID < Mesh.MaxVertexID(); ++VID)
		{
			if (Mesh.IsVertex(VID))
			{
				CompactVertexMap[VID] = CollisionMesh.Vertices.Add((FVector3f)Mesh.GetVertex(VID));
			}
		}
		TArray<int32> CompactTriangleMap;
		CompactTriangleMap.Init(INDEX_NONE, Mesh.MaxTriangleID());
		CollisionMesh.Indices.Reserve(Mesh.TriangleCount());
		auto ToCompactTriIndices = [&CompactVertexMap](const FIndex3i& Tri) -> FTriIndices
		{
			FTriIndices TriInds;
			TriInds.v0 = CompactVertexMap[Tri.A];
			TriInds.v1 = CompactVertexMap[Tri.B];
			TriInds.v2 = CompactVertexMap[Tri.C];
			return TriInds;
		};
		for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
		{
			if (Mesh.IsTriangle(TID))
			{
				FIndex3i Tri = Mesh.GetTriangle(TID);
				CompactTriangleMap[TID] = CollisionMesh.Indices.Add(ToCompactTriIndices(Tri));
			}
		}

		// Setup materials
		int32 DefaultMaterialIndex = 0;
		TArray<int32> ChannelToMaterialIndex;
		ChannelToMaterialIndex.SetNumZeroed(PhysicalMaterialChannels.Num());
		{
			TMap<UPhysicalMaterial*, int32> PhysicalMaterialMap;
			PhysicalMaterialMap.Reserve(ReferencePhysicalMaterials.Num());
			for (int32 Idx = 0; Idx < ReferencePhysicalMaterials.Num(); ++Idx)
			{
				UPhysicalMaterial* PhysMat = ReferencePhysicalMaterials[Idx];
				if (!PhysicalMaterialMap.Contains(PhysMat))
				{
					PhysicalMaterialMap.Add(PhysMat, Idx);
				}
			}
			for (int32 ChannelIdx = 0; ChannelIdx < PhysicalMaterialChannels.Num(); ++ChannelIdx)
			{
				int32* MatIdx = PhysicalMaterialMap.Find(PhysicalMaterialChannels[ChannelIdx].Material.Get());
				if (ensure(MatIdx))
				{
					ChannelToMaterialIndex[ChannelIdx] = *MatIdx;
				}
			}
		}

		// If the materials are all default, can leave the array empty, and early-out as there is nothing to cut
		if (PhysicalMaterialChannels.IsEmpty())
		{
			return CollisionMesh;
		}

		CollisionMesh.MaterialIndices.SetNumZeroed(CollisionMesh.Indices.Num());

		// We use the MeshIsoCurve multi-labeller to assign per-triangle materials

		MeshIsoCurve::FMultiLabelIsoCurveAdapter CurveAdapter;
		// TODO: expose options for how the layer weights are combined
		// Normalize Per Layer will sequentially renormalize with each new layer, so their combined weight is 1, unless they are all zero in which case the default will still be used
		constexpr bool bNormalizePerLayer = true;
		// Adapter method to get the weight of a specific material
		CurveAdapter.GetVertexLabelWeight = [&Mesh, &PhysicalMaterialChannels, &ChannelToMaterialIndex, bNormalizePerLayer](int32 VID, int32 TargetMatIdx) -> float
		{
			float TrackedWeight = 0.f;
			bool bIsDefault = true;
			for (int32 Idx = 0; Idx < PhysicalMaterialChannels.Num(); ++Idx)
			{
				if (Mesh.HasWeightLayer(PhysicalMaterialChannels[Idx].ChannelName))
				{
					float LayerWeight = FMath::Clamp(Mesh.GetWeightLayerValue(PhysicalMaterialChannels[Idx].ChannelName, VID), 0.f, 1.f);
					if (LayerWeight >= PhysicalMaterialChannels[Idx].MinimumCollisionRelevanceWeight)
					{
						bIsDefault = false;
						int32 MatIdx = ChannelToMaterialIndex[Idx];
						if (bNormalizePerLayer)
						{
							TrackedWeight = (1 - LayerWeight) * TrackedWeight;
						}
						if (TargetMatIdx == MatIdx)
						{
							TrackedWeight += LayerWeight;
						}
					}
				}
			}
			if (bIsDefault)
			{
				return TargetMatIdx == 0 ? 1.f : 0.f;
			}
			return TrackedWeight;
		};
		// Adapter method to get the material w/ the highest weight
		CurveAdapter.LabelVertex = [&Mesh, &PhysicalMaterialChannels, &ChannelToMaterialIndex, bNormalizePerLayer](int32 VID, float& OutWeight)
		{
			int32 BestMatIdx = 0;
			float BestLayerWeight = 0.f;
			for (int32 Idx = 0; Idx < PhysicalMaterialChannels.Num(); ++Idx)
			{
				float LayerWeight = 0.f;
				if (Mesh.HasWeightLayer(PhysicalMaterialChannels[Idx].ChannelName))
				{
					LayerWeight = FMath::Clamp(Mesh.GetWeightLayerValue(PhysicalMaterialChannels[Idx].ChannelName, VID), 0.f, 1.f);
					// Ignore weight if it is less than the relevance threshold
					if (LayerWeight < PhysicalMaterialChannels[Idx].MinimumCollisionRelevanceWeight)
					{
						LayerWeight = 0.f;
					}
				}
				if (bNormalizePerLayer)
				{
					// renormalize the best weight w/ the new layer's weight
					BestLayerWeight = BestLayerWeight * (1.f - LayerWeight);
				}
				if (LayerWeight > BestLayerWeight)
				{
					BestLayerWeight = LayerWeight;
					BestMatIdx = ChannelToMaterialIndex[Idx];
				}
			}
			OutWeight = BestLayerWeight;
			return BestMatIdx;
		};
		// Use default cutting methods (linearly interpolating weights)
		CurveAdapter.SetDefaultFindCutFunctions(CurveAdapter.GetVertexLabelWeight);

		MeshIsoCurve::FMeshUpdateAdapter UpdateAdapter;
		UpdateAdapter.AddInterpolatedEdgeVertex = [&CollisionMesh, &CompactVertexMap](int32 A, int32 B, double T)
		{
			FVector3f InterpV = (FVector3f)FMath::LerpStable(CollisionMesh.Vertices[CompactVertexMap[A]], CollisionMesh.Vertices[CompactVertexMap[B]], (float)T);
			return CompactVertexMap.Add(CollisionMesh.Vertices.Add(InterpV));
		};
		UpdateAdapter.AddInterpolatedTriangleVertex = [&CollisionMesh, &CompactVertexMap](FIndex3i Tri, FVector3d Bary)
		{
			const FVector3f& A = CollisionMesh.Vertices[CompactVertexMap[Tri.A]];
			const FVector3f& B = CollisionMesh.Vertices[CompactVertexMap[Tri.B]];
			const FVector3f& C = CollisionMesh.Vertices[CompactVertexMap[Tri.C]];
			FVector3f Baryf = (FVector3f)Bary;
			FVector3f InterpV = A * Baryf.X + B * Baryf.Y + C * Baryf.Z;
			return CompactVertexMap.Add(CollisionMesh.Vertices.Add(InterpV));
		};
		auto ToTriLabel = [](int32 InLabel) -> uint16
		{
			if (!ensureMsgf(InLabel >= 0 && InLabel < TNumericLimits<uint16>::Max(), TEXT("Triangle label outside of supported range: %d"), InLabel))
			{
				InLabel = 0;
			}
			return static_cast<uint16>(InLabel);
		};
		UpdateAdapter.LabelTriangle = [&CollisionMesh, &CompactTriangleMap, &ToTriLabel](int32 TID, int32 Label)
		{
			CollisionMesh.MaterialIndices[CompactTriangleMap[TID]] = ToTriLabel(Label);
		};
		UpdateAdapter.ReplaceTriangle = [&CollisionMesh, &CompactTriangleMap, &ToCompactTriIndices, &ToTriLabel]
		(int32 ReplaceTriID, TConstArrayView<FIndex3i> NewTris, TConstArrayView<int32> NewTriLabels)
		{
			checkSlow(NewTris.Num() == NewTriLabels.Num());
			if (NewTris.IsEmpty())
			{
				return;
			}
			int32 CompactReplaceTID = CompactTriangleMap[ReplaceTriID];
			CollisionMesh.Indices[CompactReplaceTID] = ToCompactTriIndices(NewTris[0]);
			CollisionMesh.MaterialIndices[CompactReplaceTID] = ToTriLabel(NewTriLabels[0]);
			for (int32 Idx = 1; Idx < NewTris.Num() && Idx < NewTriLabels.Num(); ++Idx)
			{
				CollisionMesh.Indices.Add(ToCompactTriIndices(NewTris[Idx]));
				CollisionMesh.MaterialIndices.Add(ToTriLabel(NewTriLabels[Idx]));
			}
		};

		// Reference the original FMeshData mesh for its weight data
		UE::Geometry::TMeshWrapperAdapterd<const MeshPartition::FMeshData> InputAdapter(&Mesh);
		MeshIsoCurve::FMultiLabelCutSettings Settings;
		Settings.SnapToExistingTolerance = VertexSnapTolerance;
		Settings.bNeverCut = !bCut;
		MeshIsoCurve::MultiLabelCut(InputAdapter, CurveAdapter, UpdateAdapter, Settings);

		return CollisionMesh;
	}

	// Compact material indices in collision data to just those actually used in the mesh, and populate the output materials array
	void CompactMaterialIndices(FTriMeshCollisionData& CollisionData, TConstArrayView<UPhysicalMaterial*> ReferencePhysicalMaterials, TArray<TObjectPtr<UPhysicalMaterial>>& ResultMaterials)
	{
		ResultMaterials.Reset();

		// Early-out if we only have one material
		if (ReferencePhysicalMaterials.Num() <= 1)
		{
			// We expect there should at least be one reference material (the default material)
			if (ensure(!ReferencePhysicalMaterials.IsEmpty()))
			{
				ResultMaterials.Add(ReferencePhysicalMaterials[0]);
			}
			CollisionData.MaterialIndices.Empty();
		}
		if (CollisionData.MaterialIndices.IsEmpty())
		{
			return;
		}

		// Note CollisionData.MaterialIndices is set via CutAndConvertMeshToCollisionMesh's LabelVertex method, which only sets values in range [0, PhysicalMaterialChannels.Num()+1),
		// so we should not have material indices outside this range
		TArray<bool> FoundMaterials;
		FoundMaterials.SetNumZeroed(ReferencePhysicalMaterials.Num());
		int32 MaxMaterialID = -1;
		for (uint16& ID : CollisionData.MaterialIndices)
		{
			if (!ensureMsgf(FoundMaterials.IsValidIndex(ID), TEXT("Found invalid Physical Material Index %d; setting to default physical material instead"), (int32)ID))
			{
				ID = 0;
			}
			FoundMaterials[ID] = true;
			MaxMaterialID = FMath::Max((int32)ID, MaxMaterialID);
		}
		int32 FoundNum = 0;
		TArray<uint16> Remap;
		Remap.SetNumZeroed(ReferencePhysicalMaterials.Num());
		for (int32 Idx = 0; Idx < FoundMaterials.Num(); ++Idx)
		{
			if (FoundMaterials[Idx])
			{
				Remap[Idx] = static_cast<uint16>(FoundNum++);
				ResultMaterials.Add(ReferencePhysicalMaterials[Idx]);
			}
		}

		if (FoundNum <= 1)
		{
			// don't need an array for the no/single material case
			CollisionData.MaterialIndices.Empty();
		}
		else
		{
			// remap material array indices
			for (uint16& MatID : CollisionData.MaterialIndices)
			{
				MatID = Remap[MatID];
			}
		}
	}

	TArray<UPhysicalMaterial*> GetReferencePhysicalMaterials(UPhysicalMaterial* InDefaultPhysicalMaterial, TConstArrayView<UE::MeshPartition::FPhysicalMaterialChannel> InPhysicalMaterialChannels)
	{
		TSet<UPhysicalMaterial*> MatSet;
		// Note: TSet preserves ordering when no Remove calls are made, so default material will remain at index 0
		MatSet.Add(InDefaultPhysicalMaterial);
		for (const UE::MeshPartition::FPhysicalMaterialChannel& Channel : InPhysicalMaterialChannels)
		{
			MatSet.Add(Channel.Material.Get());
		}
		return MatSet.Array();
	}
}


namespace UE::MeshPartition::Collision
{
	void FCollisionSimplificationSettings::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
	{
		Dependencies += bSimplifyCollision;
		Dependencies += bCutMeshAlongMaterialBoundaries;
		Dependencies += MaterialBoundaryCutSnapTolerance;
		if (bSimplifyCollision)
		{	
			Dependencies += MaterialBoundaryDistanceTolerance;
			Dependencies += SimplifyMethod;
			Dependencies += EdgeLength;
			Dependencies += ErrorTolerance;
			Dependencies += bScaleAccuracyViaNormal;
			Dependencies += LocalAccuracyScale;
			Dependencies += ScaleToMinNormalAngle;
			Dependencies += ScaleFromMaxNormalAngle;
			Dependencies += ScaleAccuracyNormalDirection;
			// Version key to allow experimentation w/ the underlying methods, to be bumped with algorithm changes
			static FGuid SimplificationVersionKey(TEXT("8db69a3d-d5b7-4beb-a3e1-c0c69cd630f4"));
			Dependencies += SimplificationVersionKey;
		}
	}

	void ConvertMeshToCollisionData(const FMeshData& Mesh, FMeshPartitionCollisionData& CollisionData, const FMeshToCollisionSettings& Settings)
	{
		using namespace UE::Geometry;
		using namespace UE::MeshPartitionCollisionGenerationLocals;

		CollisionData.Mesh.Emplace();
		FTriMeshCollisionData& MeshCollisionData = *CollisionData.Mesh;

		const FCollisionSimplificationSettings& SimplifySettings = Settings.SimplificationSettings;
		const bool bApplyQEMSimplify = SimplifySettings.bSimplifyCollision &&
			SimplifySettings.SimplifyMethod == UE::MeshPartition::Collision::ECollisionSimplificationMethod::QEM;
		const bool bApplyClusterSimplify = SimplifySettings.bSimplifyCollision &&
			SimplifySettings.SimplifyMethod == UE::MeshPartition::Collision::ECollisionSimplificationMethod::Cluster &&
			SimplifySettings.EdgeLength > 0;

		TArray<UPhysicalMaterial*> ReferencePhysicalMaterials = GetReferencePhysicalMaterials(Settings.DefaultPhysicalMaterial, Settings.PhysicalMaterialChannels);
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConvertMeshToCollisionData::InitialCutConvert);
			
			double Time = 0.0;
			FDurationTimer Timer(Time);
			Timer.Start();
			
			MeshCollisionData = CutAndConvertMeshToCollisionMesh(Mesh, ReferencePhysicalMaterials, Settings.PhysicalMaterialChannels, 
				Settings.SimplificationSettings.bCutMeshAlongMaterialBoundaries, Settings.SimplificationSettings.MaterialBoundaryCutSnapTolerance);
			
			Timer.Stop();
			UE_LOGF(LogMegaMeshEditor, Verbose, "CutConvertMeshToComplexCollision: Prepared collision data, %.2f seconds, %d verts / %d tris", Time, MeshCollisionData.Vertices.Num(), MeshCollisionData.Indices.Num());
		}

		if (bApplyQEMSimplify)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConvertMeshToCollisionData::QEMSimplify);

			double Time = 0.0;
			FDurationTimer Timer(Time);
			Timer.Start();
			FDynamicMesh3 DMesh = ConvertCollisionMeshToDynamicMesh(MeshCollisionData);
			
			FQEMSimplifyMeshOptions Options;
			SimplifyDynamicMesh<FQEMSimplification>(DMesh, Settings.SimplificationSettings);
			DynamicMeshToCollisionMeshData(DMesh, MeshCollisionData, true);

			Timer.Stop();
			float ReducedVertPct = Mesh.VertexCount() ? 100.0f * (float)MeshCollisionData.Vertices.Num() / (float)Mesh.VertexCount() : 0.f;
			float ReducedTriPct = Mesh.TriangleCount() ? 100.0f * (float)MeshCollisionData.Indices.Num() / (float)Mesh.TriangleCount() : 0.f;
			UE_LOGF(LogMegaMeshEditor, Display, "ConvertMeshToComplexCollision: QEM simplified collision data, %.2f seconds, Initial: %d verts / %d tris, Simplified: %d (%.1f%%) verts / %d (%.1f%%) tris",
				Time, Mesh.VertexCount(), Mesh.TriangleCount(), MeshCollisionData.Vertices.Num(), ReducedVertPct, MeshCollisionData.Indices.Num(), ReducedTriPct);
		}
		else if (bApplyClusterSimplify)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(ConvertMeshToCollisionData::ClusterSimplify);

			double Time = 0.0;
			FDurationTimer Timer(Time);
			Timer.Start();

			FTriMeshCollisionData UnsimplifiedMesh = MoveTemp(MeshCollisionData);
			UE::Geometry::FTriangleMeshAdapterd InputAdapter = GetCollisionMeshAdapter(&UnsimplifiedMesh);
			
			// Compute vertex normals to use for local target length scaling
			TArray<FVector3f> VertexNormals;
			VertexNormals.SetNumUninitialized(UnsimplifiedMesh.Vertices.Num());
			Geometry::MeshNormals::ComputeVertexNormals(InputAdapter, VertexNormals, true);
			
			// reset the target mesh collision data, to be filled w/ simplified mesh
			MeshCollisionData = FTriMeshCollisionData();

			UE::Geometry::MeshClusterSimplify::FResultMeshAdapter ResultAdapter;

			// Use FResultMeshAdapter to write directly to the FTriMeshCollisionData
			MeshCollisionData.Vertices.Reserve(Mesh.VertexCount() / 2);
			MeshCollisionData.Indices.Reserve(Mesh.TriangleCount() / 2);
			ResultAdapter.AppendVertex = [&MeshCollisionData](FVector3d V) -> int32 { return MeshCollisionData.Vertices.Add(FVector3f(V));  };
			ResultAdapter.Clear = [&MeshCollisionData]() { MeshCollisionData.Vertices.Empty(); MeshCollisionData.Indices.Empty(); };
			ResultAdapter.GetVertex = [&MeshCollisionData](int32 VID) -> FVector3d { return (FVector3d)MeshCollisionData.Vertices[VID]; };
			ResultAdapter.AppendTriangle = [&MeshCollisionData](UE::Geometry::FIndex3i T)-> int32
			{
				FTriIndices TI;
				TI.v0 = T.A;
				TI.v1 = T.B;
				TI.v2 = T.C;
				return MeshCollisionData.Indices.Add(TI);
			};
			ResultAdapter.GetTriangle = [&MeshCollisionData](int32 TID) -> UE::Geometry::FIndex3i
			{
				FTriIndices TI = MeshCollisionData.Indices[TID];
				return { TI.v0, TI.v1, TI.v2 };
			};
			ResultAdapter.TransferPerTriangleAttributes = 
			[&UnsimplifiedMesh, &MeshCollisionData](TConstArrayView<int32> ResultToSourceTriangleID)
			{
				if (UnsimplifiedMesh.MaterialIndices.IsEmpty())
				{
					return;
				}

				MeshCollisionData.MaterialIndices.SetNumZeroed(ResultToSourceTriangleID.Num());
				for (int32 ResultTID = 0; ResultTID < ResultToSourceTriangleID.Num(); ++ResultTID)
				{
					int32 SourceTID = ResultToSourceTriangleID[ResultTID];
					if (SourceTID > -1)
					{
						MeshCollisionData.MaterialIndices[ResultTID] = UnsimplifiedMesh.MaterialIndices[SourceTID];
					}
				}
			};

			UE::Geometry::MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
			SimplifyOptions.PreserveEdges.SetSeamConstraints(UE::Geometry::MeshClusterSimplify::FSimplifyOptions::EConstraintLevel::Free);
			SimplifyOptions.bTransferAttributes = !UnsimplifiedMesh.MaterialIndices.IsEmpty();
			SimplifyOptions.TargetEdgeLength = SimplifySettings.EdgeLength;
			auto CosDegrees = [](float AngleDeg) -> double { return (double)FMath::Cos(FMath::DegreesToRadians(AngleDeg)); };
			SimplifyOptions.OptionalTargetEdgeLengthScale = [&VertexNormals, &SimplifySettings, 
				ScaleRange = FVector2d(CosDegrees(SimplifySettings.ScaleFromMaxNormalAngle),CosDegrees(SimplifySettings.ScaleToMinNormalAngle))](int32 VID)
				{
					double ErrScale = FMath::GetMappedRangeValueClamped<double>(ScaleRange, { 1., (double)SimplifySettings.LocalAccuracyScale }, VertexNormals[VID].Z);
					// invert metric scales to get target length scale
					return (ErrScale > 0.) ? 1. / ErrScale : 1.;
				};
			bool bRet = UE::Geometry::MeshClusterSimplify::Simplify(InputAdapter, ResultAdapter, SimplifyOptions);
			ensure(bRet);

			Timer.Stop();
			float ReducedVertPct = Mesh.VertexCount() ? 100.0f * (float)MeshCollisionData.Vertices.Num() / (float)Mesh.VertexCount() : 0.f;
			float ReducedTriPct = Mesh.TriangleCount() ? 100.0f * (float)MeshCollisionData.Indices.Num() / (float)Mesh.TriangleCount() : 0.f;
			UE_LOGF(LogMegaMeshEditor, Display, "ConvertMeshToComplexCollision: Prepared simplified collision data with edge length %f, %.2f seconds, Initial: %d verts / %d tris, Simplified: %d (%.1f%%) verts / %d (%.1f%%) tris",
				SimplifyOptions.TargetEdgeLength, Time, Mesh.VertexCount(), Mesh.TriangleCount(), MeshCollisionData.Vertices.Num(), ReducedVertPct, MeshCollisionData.Indices.Num(), ReducedTriPct);
		}
		else
		{
			// No simplification to apply

			UE_LOGF(LogMegaMeshEditor, Verbose, "ConvertMeshToComplexCollision: Prepared collision data. %d verts / %d tris", MeshCollisionData.Vertices.Num(), MeshCollisionData.Indices.Num());
		}
		// Compact the material IDs, transferring any used physical materials to the result
		CompactMaterialIndices(MeshCollisionData, ReferencePhysicalMaterials, CollisionData.PhysicalMaterials);

		// Apply general settings
		MeshCollisionData.bFastCook = Settings.bFastCook;
		MeshCollisionData.bDisableActiveEdgePrecompute = Settings.bDisableActiveEdgePrecompute;
		MeshCollisionData.bReducedMemoryRepresentation = true;

		// note default winding for physics meshes is opposite of winding, so this flag lets us build the mesh w/ the render mesh's triangle winding
		MeshCollisionData.bFlipNormals = true;
	}

} // namespace UE::MeshPartition::Collision