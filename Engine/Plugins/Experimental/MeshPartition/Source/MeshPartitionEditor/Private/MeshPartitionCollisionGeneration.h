// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionChannel.h"
#include "MeshPartitionCollisionComponent.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionMeshData.h"

#include "MeshPartitionCollisionGeneration.generated.h"

// Support for generating collision from Mesh Partition meshes
namespace UE::MeshPartition::Collision
{
	UENUM()
	enum class ECollisionSimplificationMethod
	{
		Cluster,
		QEM
	};

	USTRUCT()
	struct FCollisionSimplificationSettings
	{
		GENERATED_BODY()

	public:

		// Whether to cut original mesh triangles along physical material boundaries. Otherwise, triangles will be assigned discretely to a material based on the average vertex weights.
		// Note this introduces additional triangles initially, but can create smoother, more-accurate material boundaries that can be better-simplified.
		UPROPERTY(EditAnywhere, Category = "Material Boundaries")
		bool bCutMeshAlongMaterialBoundaries = true;

		// When cutting material boundaries, tolerance distance at which to snap material boundary to existing geometry rather than cutting the mesh
		UPROPERTY(EditAnywhere, Category = "Material Boundaries", meta = (Units = "cm", EditCondition = bCutMeshAlongMaterialBoundaries))
		float MaterialBoundaryCutSnapTolerance = 10.f;

		UPROPERTY(EditAnywhere, Category = "Simplification",
				  meta = (ToolTip = "Enable collision simplification on the separate collision components"))
		bool bSimplifyCollision = false;

		UPROPERTY(EditAnywhere, Category = "Simplification")
		ECollisionSimplificationMethod SimplifyMethod = ECollisionSimplificationMethod::QEM;

		// Constrain how much simplification can move material boundaries away from their original positions
		UPROPERTY(EditAnywhere, Category = "Simplification", meta = (Units = "cm", EditCondition = "SimplifyMethod==ECollisionSimplificationMethod::QEM"))
		float MaterialBoundaryDistanceTolerance = 50.f;

		// If positive, set a geometric error tolerance
		UPROPERTY(EditAnywhere, Category = "Simplification", meta = (Units = "cm", EditCondition = "SimplifyMethod==ECollisionSimplificationMethod::QEM"))
		float ErrorTolerance = 10.f;

		// If positive, set a target edge length
		UPROPERTY(EditAnywhere, Category = "Simplification", meta = (Units = "cm"))
		float EdgeLength = -1.f;


		// Whether to scale requested collision accuracy (error tolerance / edge length) locally based on surface normal's alignment with a target direction (e.g. to favor accuracy on 'walkable' surfaces)
		UPROPERTY(EditAnywhere, Category = "Simplification")
		bool bScaleAccuracyViaNormal = true;

		// Scale up the requested accuracy (error metric and/or measured edge length) by up-to this amount for surfaces w/ normal aligned to ScaleAccuracyNormalDirection
		UPROPERTY(EditAnywhere, Category = "Simplification|ScaleAccuracy", meta = (EditCondition="bScaleAccuracyViaNormal"))
		float LocalAccuracyScale = 2.5f;

		// Min normal alignment angle vs ScaleAccuracyNormalDirection, at which to finish scaling up simplifier accuracy, in degrees
		UPROPERTY(EditAnywhere, Category = "Simplification|ScaleAccuracy", meta = (EditCondition = "bScaleAccuracyViaNormal", ClampMin = 0, ClampMax = 180, Units = Degrees))
		float ScaleToMinNormalAngle = 26.f;

		// Max normal alignment angle vs ScaleAccuracyNormalDirection, at which to start scaling up simplifier accuracy, in degrees
		UPROPERTY(EditAnywhere, Category = "Simplification|ScaleAccuracy", meta = (EditCondition = "bScaleAccuracyViaNormal", ClampMin=0, ClampMax=180, Units = Degrees))
		float ScaleFromMaxNormalAngle = 53.f;

		// The normal direction that should have highest accuracy
		UPROPERTY(EditAnywhere, Category = "Simplification|ScaleAccuracy", meta = (EditCondition = "bScaleAccuracyViaNormal"))
		FVector3d ScaleAccuracyNormalDirection = FVector(0., 0., 1.);

		#if WITH_EDITOR
		void MESHPARTITIONEDITOR_API GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const;
		#endif
	};

	struct FMeshToCollisionSettings
	{
		UE::MeshPartition::Collision::FCollisionSimplificationSettings SimplificationSettings;
		
		/** Physical material channels; vertex weight channels will be translated to per - triangle physical materials via this mapping */
		TArray<UE::MeshPartition::FPhysicalMaterialChannel> PhysicalMaterialChannels;
		
		/** Default physical material, to set if channels not found */
		UPhysicalMaterial* DefaultPhysicalMaterial = nullptr;

		/** Prioritize cooking speed over runtime speed */
		bool bFastCook = true;

		/** Turn off ActiveEdgePrecompute (This makes cooking faster, but will slow contact generation) */
		bool bDisableActiveEdgePrecompute = true;
	};

	void ConvertMeshToCollisionData(const FMeshData& Mesh, FMeshPartitionCollisionData& CollisionData, const FMeshToCollisionSettings& Settings);
} // namespace UE::MeshPartition::Collision
