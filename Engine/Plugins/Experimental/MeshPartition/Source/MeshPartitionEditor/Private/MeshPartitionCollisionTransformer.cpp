// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionCollisionTransformer.h"

#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionCollisionGeneration.h"
#include "PhysicalMaterials/PhysicalMaterial.h"

namespace UE::MeshPartition
{

namespace MeshPartitionCollisionTransformerLocals
{
	
} // namespace MeshPartitionCollisionTransformerLocals

void FCollisionTransformer::Initialize(const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InVariant)
{
	if (!ensure(InDefinition != nullptr))
	{
		return;
	}

	PhysicalMaterialChannels = InDefinition->GetPhysicalMaterialChannels();
	DefaultPhysicalMaterial = InDefinition->GetDefaultPhysicalMaterial();
}

bool FCollisionTransformer::Execute(MeshPartition::FTransformerContext& InTransformerContext) const
{
	// Currently using one collision surface per preview mesh
	TArray<TSharedPtr<FMeshPartitionCollisionData>> CollisionData;
	TArray<UE::Tasks::FTask> BuildCollisionTasks;

	const int32 TaskNumber = InTransformerContext.TransformerUnits.Num();

	BuildCollisionTasks.SetNum(TaskNumber);

	for (int32 Index = 0; Index < TaskNumber; ++Index)
	{
		CollisionData.Emplace(MakeShared<FMeshPartitionCollisionData>());
	}

	for (int32 Index = 0; Index < TaskNumber; ++Index)
	{
		BuildCollisionTasks[Index] = UE::Tasks::Launch(TEXT("FCollisionTransformer::BuildCollisionTask"), [this,
			&InTransformerContext,
			&CollisionData,
			Index]()
			{
				BuildCollisionData(InTransformerContext, Index, CollisionData);
			});
	}

	UE::Tasks::FTask FinalizeCollisions = UE::Tasks::Launch(TEXT("FCollisionTransformer::FinalizeCollisions"), [this,
		&InTransformerContext,
		&CollisionData]() mutable
		{
			FinalizeCollisionData(InTransformerContext, CollisionData);
		},
		UE::Tasks::Prerequisites(BuildCollisionTasks),
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);

	FinalizeCollisions.Wait();

	return true;
}

void FCollisionTransformer::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	CollisionSimplificationSettings.GatherDependencies(InDependencies);
	InDependencies += CollisionProfile.Name;
	InDependencies += bCanEverAffectNavigation;
	InDependencies += bFastCook;
	InDependencies += bDisableActiveEdgePrecompute;
}

void FCollisionTransformer::BuildCollisionData(const MeshPartition::FTransformerContext& InTransformerContext, const int32 InEntryIndex, TArray<TSharedPtr<FMeshPartitionCollisionData>>& OutCollisionData) const
{
	if (InTransformerContext.bWasCancelled)
	{
		return;
	}
	    	
	check(InEntryIndex < InTransformerContext.TransformerUnits.Num());
	const MeshPartition::FTransformerUnit& TransformerUnit = InTransformerContext.TransformerUnits[InEntryIndex];
			
	if (!ensure(TransformerUnit.MeshData != nullptr))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get MeshData.");
		return;
	}
	    		
	Collision::FMeshToCollisionSettings Settings;
			
	Settings.SimplificationSettings = CollisionSimplificationSettings;
	Settings.bFastCook = bFastCook; //todo(luc.eygasier): should we take into account preview vs compiled sections? (true also for disable active edge precompute)
	Settings.bDisableActiveEdgePrecompute = bDisableActiveEdgePrecompute;
	Settings.PhysicalMaterialChannels = PhysicalMaterialChannels;
	Settings.DefaultPhysicalMaterial = DefaultPhysicalMaterial;
			
	check(InEntryIndex < OutCollisionData.Num());
	Collision::ConvertMeshToCollisionData(*TransformerUnit.MeshData, *OutCollisionData[InEntryIndex], Settings);
}

void FCollisionTransformer::FinalizeCollisionData(const MeshPartition::FTransformerContext& InTransformerContext, const TArray<TSharedPtr<FMeshPartitionCollisionData>>& InCollisionData) const
{
	check(IsInGameThread());
	check(InCollisionData.Num() == InTransformerContext.TransformerUnits.Num());
		  	
	if (InTransformerContext.bWasCancelled)
	{
		return;
	}
		  	
	for (int32 Index = 0; Index < InTransformerContext.TransformerUnits.Num(); ++Index)
	{
		const MeshPartition::FTransformerUnit& TransformerUnit = InTransformerContext.TransformerUnits[Index];
		  	    
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if (Section == nullptr)
		{
			continue;
		}

		Section->Modify(true);

		MeshPartition::UMeshPartitionCollisionComponent* CollisionComponent = NewObject<MeshPartition::UMeshPartitionCollisionComponent>(Section, MakeUniqueObjectName(Section, MeshPartition::UMeshPartitionCollisionComponent::StaticClass(), TEXT("CollisionComponent")));
		CollisionComponent->OnComponentCreated();

		CollisionComponent->SetCollisionData(InCollisionData[Index], CollisionProfile.Name);
		CollisionComponent->SetupAttachment(Section->GetRootComponent());
		CollisionComponent->SetMobility(EComponentMobility::Static);
		CollisionComponent->SetCollisionProfileName(CollisionProfile.Name);
		CollisionComponent->SetCanEverAffectNavigation(bCanEverAffectNavigation);

		if (MeshPartition::APreviewSection* PreviewSection = Cast<MeshPartition::APreviewSection>(Section))
		{
			PreviewSection->AddCollisionComponent(CollisionComponent);
		}
		else if (MeshPartition::ACompiledSection* CompiledSection = Cast<MeshPartition::ACompiledSection>(Section))
		{
			constexpr bool bAllowAsyncBuild = false;
			CollisionComponent->RebuildIfNeeded(bAllowAsyncBuild);

			CompiledSection->AddCollisionComponent(CollisionComponent);
		}
		else
		{
			checkf(false, TEXT("This should never happen. Transformers should only be applied on Preview or Compiled sections at the moment."));
		}
	}
}


} // namespace UE::MeshPartition
