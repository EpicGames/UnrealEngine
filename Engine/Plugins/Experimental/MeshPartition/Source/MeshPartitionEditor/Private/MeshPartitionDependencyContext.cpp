// Copyright Epic Games, Inc. All Rights Reserved.
 
#include "MeshPartitionDependencyContext.h"

#include "Net/Core/Misc/ResizableCircularQueue.h"
#include "UObject/Class.h"
#include "UDynamicMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "Components/SplineComponent.h"
#include "Curves/CurveFloat.h"
#include "AssetRegistry/IAssetRegistry.h"

namespace UE::MeshPartition
{
TArray<FName> FDependencyContext::GetPackageDependencies(const IAssetRegistry& InAssetRegistry) const
{
	// convert object dependencies into package dependencies
	TResizableCircularQueue<FAssetIdentifier> RecursivePackageDependencyQueue;
	TSet<FName> PackageNames;						// set of all package dependencies discovered
	TSet<FName> RecursivelyCheckedPackageNames;		// set of packages checked recursively
	for (TPair<const UObject*, MeshPartition::EDependencyFlags> Pair : ObjectDependencies)
	{
		const UObject* Object = Pair.Key;
		FAssetIdentifier ObjectAssetId((UObject*)Object, FName());

		if (EnumHasAnyFlags(Pair.Value, MeshPartition::EDependencyFlags::RecurseOnPackageDependencies))
		{
			RecursivePackageDependencyQueue.Enqueue(ObjectAssetId);
		}
		else
		{
			PackageNames.Add(ObjectAssetId.PackageName);
		}
	}

	// resolve recursive package dependencies
	TArray<FAssetIdentifier> ImmediateDependencies;
	while (RecursivePackageDependencyQueue.Count() > 0)
	{
		const FAssetIdentifier& CheckAsset = RecursivePackageDependencyQueue.Peek();
		RecursivePackageDependencyQueue.PopNoCheck();

		bool bAlreadyRecursivelyChecked = false;
		if (RecursivelyCheckedPackageNames.FindOrAdd(CheckAsset.PackageName, &bAlreadyRecursivelyChecked), !bAlreadyRecursivelyChecked)
		{
			PackageNames.FindOrAdd(CheckAsset.PackageName);

			// recurse on dependencies and queue them up for checking
			ImmediateDependencies.Empty();
			InAssetRegistry.GetDependencies(CheckAsset, ImmediateDependencies);

			for (const FAssetIdentifier& ImmediateDependency : ImmediateDependencies)
			{
				if (!RecursivelyCheckedPackageNames.Contains(ImmediateDependency.PackageName))
				{
					RecursivePackageDependencyQueue.Enqueue(ImmediateDependency);
				}
			}
		}
	}

	// put them in a deterministic order
	TArray<FName> Result = PackageNames.Array();
	Result.Sort([](const FName& A, const FName& B)
		{
			return A.ToString() > B.ToString();
		});

	return Result;
}

TArray<const UClass*> FDependencyContext::GetClassDependencies() const
{
	TArray<const UClass*> Result = ClassDependencies.Array();
	Result.Sort([](const UClass& A, const UClass& B)
		{
			FSoftClassPath PathA(&A);
			FSoftClassPath PathB(&B);
			return PathA.ToString() > PathB.ToString();
		});

	return Result;
}

void FDependencyContext::AddPackageDependency(const UObject* InObject, MeshPartition::EDependencyFlags InDependencyFlags)
{
	if (InObject != nullptr)
	{
		MeshPartition::EDependencyFlags& CurrentDependencySetting = ObjectDependencies.FindOrAdd(InObject);
		EnumAddFlags(CurrentDependencySetting, InDependencyFlags);
	}
}

void FDependencyContext::AddPackageDependency(TArrayView<UObject*> InObjectArray)
{
	for (UObject* Object : InObjectArray)
	{
		AddPackageDependency(Object, MeshPartition::EDependencyFlags::None);
	}
}

void FDependencyContext::AddClassDependency(const UClass* InClass)
{
	if (InClass != nullptr)
	{
		ClassDependencies.Add(InClass);
	}
}

void FDependencyContext::operator+=(const UDynamicMesh& InMesh)
{
	AddPackageDependency(&InMesh);
}

void FDependencyContext::operator+=(const UStaticMesh& InMesh)
{
	AddPackageDependency(&InMesh);
}

void FDependencyContext::operator+=(const UTexture& InTexture)
{
	AddPackageDependency(&InTexture);
}

void FDependencyContext::operator+=(const UTexture2D& InTexture)
{
	*this += reinterpret_cast<const UTexture&>(InTexture);
}

void FDependencyContext::operator+=(const UCurveFloat& InCurveFloat)
{
	AddPackageDependency(&InCurveFloat);
}

void FDependencyContext::operator+=(const USplineComponent& InSpline)
{
	AddPackageDependency(&InSpline);
}

void FDependencyContext::operator+=(const UObject& InObject)
{
	AddPackageDependency(&InObject);
}

FGuid FDependencyHash::GetDependentDataHash() const
{
	return FGuid::NewGuidFromHash(HashArchive.GetHash());
}

void FDependencyHash::operator+=(const UDynamicMesh& InMesh)
{
	// no shortcut available currently for dynamic mesh, have to hash its serialized data
	// this const cast should be safe as we know the serializer is read only
	const_cast<UDynamicMesh*>(&InMesh)->Serialize(HashArchive);
}

void FDependencyHash::operator+=(const UStaticMesh& InMesh)
{
	// we use the lighting guid as a fast alternative to hashing the mesh contents
	FGuid MeshHash = InMesh.GetLightingGuid();
	HashArchive << MeshHash;
}

void FDependencyHash::operator+=(const UTexture& InTexture)
{
	if (InTexture.Source.IsValid())
	{
		// we use source ID as a fast alternative to hashing the texture contents
		// const_cast<UTexture*>(&InTexture)->Source.UseHashAsGuid();		// TODO: do we do this to ensure we're getting a proper hash?  break const..
		FGuid SourceId = InTexture.Source.GetId();
		HashArchive << SourceId;
	}
	else
	{
		// LightingGuid is a fallback, less stable as it's just a random GUID that's regenerated on change
		// (but still mostly stable as long as the texture does not change)
		FGuid LightingGuid = InTexture.GetLightingGuid();
		HashArchive << LightingGuid;
	}
}

void FDependencyHash::operator+=(const UTexture2D& InTexture)
{
	*this += reinterpret_cast<const UTexture&>(InTexture);
}

void FDependencyHash::operator+=(const UCurveFloat& InCurveFloat)
{
	HashArchive.IgnoreObjectReferences(
		[this, &InCurveFloat]
		{
			const_cast<UCurveFloat&>(InCurveFloat).Serialize(HashArchive);
		}
	);
}

void FDependencyHash::operator+=(const USplineComponent& InSpline)
{
	const_cast<USplineComponent*>(&InSpline)->Serialize(HashArchive);
}

void FDependencyHash::operator+=(const UObject& InObject)
{
	const_cast<UObject*>(&InObject)->Serialize(HashArchive);
}
} // namespace UE::MeshPartition