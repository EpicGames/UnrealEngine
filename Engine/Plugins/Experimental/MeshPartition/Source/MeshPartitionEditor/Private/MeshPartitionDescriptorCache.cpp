// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionDescriptorCache.h"

#include "MeshPartition.h"
#include "MeshPartitionEditorModule.h"			// LogMegaMeshEditor
#include "MeshPartitionEditorUtils.h"			// PackageHashIsUpToDate
#include "MeshPartitionDefinition.h"
#include "MeshPartitionMeshBuilder.h"			// FindLayerPriorityIndexFromName
#include "MeshPartitionCompiledSection.h"		// FCompiledSectionDescriptor
#include "MeshPartitionModifierUtils.h"			// ForEachMegaMeshDescInWorldPartition, BetterHashCombine
#include "MeshPartitionEditorSubsystem.h"		// FPIEPathFixer
#include "WorldPartition/WorldPartition.h"
#include "Algo/AnyOf.h"
#include "Algo/ForEach.h"
#include "EngineUtils.h"					// TActorIterator

namespace UE::MeshPartition
{

FCachedDescriptors::FCachedDescriptors(const FGuid& InMeshPartitionGUID, const AMeshPartition* InMeshPartitionActor)
{
	check(InMeshPartitionGUID.IsValid());
	if (InMeshPartitionActor)
	{
		check(InMeshPartitionActor->GetActorGuid() == InMeshPartitionGUID);
	}

	this->MeshPartitionGUID = InMeshPartitionGUID;
	this->MeshPartitionActor = InMeshPartitionActor;
}

FCachedDescriptors::FCachedDescriptors(const AMeshPartition* InMeshPartition, TConstArrayView<MeshPartition::FModifierDesc> InAllModifiersSorted, TArray<FCompiledSectionDescriptor>&& InCompiledSections)
{
	check(InMeshPartition != nullptr);
	this->MeshPartitionActor = InMeshPartition;
	this->MeshPartitionGUID = InMeshPartition->GetActorGuid();
	check(this->MeshPartitionGUID.IsValid());

	UWorld* World = InMeshPartition->GetWorld();
	check(World != nullptr);

	UWorldPartition* WorldPartition = World->GetWorldPartition();
	check(WorldPartition != nullptr)

	const UMeshPartitionDefinition* MeshPartitionDefinition = InMeshPartition->GetMeshPartitionDefinition();
	if (MeshPartitionDefinition == nullptr)
	{
		UE_LOGF(LogCore, Warning, "No MeshPartition Definition provided for %ls (%.6ls), using default settings.", *InMeshPartition->GetName(), *InMeshPartition->GetActorGuid().ToString());
		MeshPartitionDefinition = UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
	}

	// copy the sorted modifier list into the group
	for (const MeshPartition::FModifierDesc& ModifierDesc : InAllModifiersSorted)
	{
		AllModifiersGroup.Add(ModifierDesc);
	}

	// Sort the Modifiers by priority:
	AllModifiersGroup.Sort(MeshPartitionDefinition->GetModifierTypePriorities());

	this->CompiledSections = MoveTemp(InCompiledSections);

	// iterate all build variants
	TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariantArray = MeshPartitionDefinition->GetCompiledSectionBuildVariants();
	check(BuildVariantArray.Num() > 0);
	for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : BuildVariantArray)
	{
		// break the Mesh Partition into groups, using the build variant settings
		this->BuildVariants.Add(BuildVariant.Name, MeshPartition::FModifierGrouping(BuildVariant, *MeshPartitionDefinition, AllModifiersGroup));
	}
}

FModifierGrouping::FModifierGrouping(const MeshPartition::FCompiledSectionBuildVariant& InBuildVariant, const UMeshPartitionDefinition& Definition, const MeshPartition::FModifierGroup& AllModifiersGroupSorted)
{
	BuildVariantName = InBuildVariant.Name;

	MeshPartition::FBuilderSettings BuilderSettings;
	// these are the only two properties that affect grouping
	BuilderSettings.TypePriorities = Definition.GetModifierTypePriorities();
	BuilderSettings.MaxSectionComplexity = InBuildVariant.MaxSectionComplexity;

	ModifierGroups = BuildModifierGroups(AllModifiersGroupSorted.AllModifierDescs(), BuilderSettings);
}

MeshPartition::FModifierGroup FCachedDescriptors::BuildModifierGroupForBaseModifierPaths(TConstArrayView<FSoftObjectPath> BaseModifierPaths) const
{
	// convert BaseModifierPaths into Descriptors
	TArray<MeshPartition::FModifierDesc> BaseSet;
	{
		for (FSoftObjectPath Path : BaseModifierPaths)
		{
			// NOTE that the base modifier paths passed in have to correspond to serialized paths (they cannot be temporary PIE paths)
			const MeshPartition::FModifierDesc* Desc = AllModifiersGroup.AllModifierDescs().FindByPredicate([&Path](const MeshPartition::FModifierDesc& Desc) { return Desc.ModifierPath == Path; });
			check(Desc);
			BaseSet.Add(*Desc);
		}
	}

	return BuildModifierGroupForBase(BaseSet, AllModifiersGroup.ModifierDescs());
}

void FCachedDescriptors::ForAllBuildVariants(TFunctionRef<bool(const MeshPartition::FModifierGrouping&)> InFunc) const
{
	using FVariantNameCachedBuildVariantPair = TPair<FName, MeshPartition::FModifierGrouping>;

	TArray<FName> Keys;
	Keys.Reserve(BuildVariants.Num());
	for (const FVariantNameCachedBuildVariantPair& VariantNameCachedBuildVariantPair : BuildVariants)
	{
		Keys.Emplace(VariantNameCachedBuildVariantPair.Key);
	}

	for (const FName& Key : Keys)
	{
		const MeshPartition::FModifierGrouping* BuildVariant = BuildVariants.Find(Key);

		if (BuildVariant == nullptr)
		{
			continue;
		}

		if (!InFunc(*BuildVariant))
		{
			return;
		}
	}
}

void FModifierDescriptorCache::AddMeshPartitionToGuidMap(const AMeshPartition* MeshPartition)
{
	if (MeshPartition == nullptr)
	{
		return;
	}

	const FGuid& MeshPartitionGuid = MeshPartition->GetActorGuid();
	if (MeshPartitionGuid.IsValid())
	{
		FSoftObjectPath MeshPartitionPath(MeshPartition);
		GuidToMeshPartitionPath.AddUnique(MeshPartitionGuid, MeshPartitionPath);
	}
}

void FModifierDescriptorCache::UpdateCacheForAllMeshPartitionsInWorld(UWorld* InWorld, TArray<FGuid> *OutAllMeshPartitionGuidsInWorld)
{
	using namespace Utils;

	TMap<FGuid, TTuple<TArray<MeshPartition::FModifierDesc>, TArray<FCompiledSectionDescriptor>>> MeshPartitionsInWorld;
	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();
	if (WorldPartition == nullptr)
	{
		return;
	}

	UE::MeshPartition::EditorUtils::FPIEPathFixer& PIEPathFixer = UMeshPartitionEditorSubsystem::GetPIEPathFixer();
	int32 CompiledSections = 0;
	int32 Modifiers = 0;
	int32 SectionGroups = 0;

	// check if this is a PIE world
	bool bIsPIE = InWorld->IsPlayInEditor();

	// NOTE: this only grabs actor descs from the Main world/level -- not from any level instances
	ForEachMegaMeshDescInWorldPartition(WorldPartition,
		[&MeshPartitionsInWorld, &Modifiers, &PIEPathFixer, bIsPIE](MeshPartition::FModifierDesc& Descriptor)
		{
			if (Descriptor.MegaMeshGuid.IsValid())
			{
				TArray<MeshPartition::FModifierDesc>& ModifierDescriptors = MeshPartitionsInWorld.FindOrAdd(Descriptor.MegaMeshGuid).Get<0>();

				// TODO: this may not correctly modify a level instanced path.. we may need to strip PIE paths from those individually?
				FSoftObjectPath ModifierPath = Descriptor.ModifierPath;
				if (bIsPIE)
				{
					// replace the PIE package name with the non-PIE package name
					PIEPathFixer.FixInPlace(Descriptor.ModifierPath);
				}

				ModifierDescriptors.Add(Descriptor);
				Modifiers++;
			}
		},
		[&MeshPartitionsInWorld, &CompiledSections](FCompiledSectionDescriptor& Descriptor)
		{
			// ignore old descriptors that don't have mesh partition GUID populated
			if (Descriptor.Info.MegaMeshGUID.IsValid())
			{
				TArray<FCompiledSectionDescriptor>& CompiledSectionDescriptors = MeshPartitionsInWorld.FindOrAdd(Descriptor.Info.MegaMeshGUID).Get<1>();
				CompiledSectionDescriptors.Add(Descriptor);
				CompiledSections++;
			}
		});

	// Build the section groups for each mesh partition in the world
	// NOTE: this will not gather spatially streaming AMeshPartition actors
	// (currently only the section actors should be streaming, not the AMeshPartition actor)
	for (TActorIterator<AMeshPartition> It(InWorld); It; ++It)
	{
		const AMeshPartition* MeshPartition = *It;
		const FGuid MeshPartitionGuid = MeshPartition->GetActorGuid();

		// populate the GUID -> MeshPartition with the MeshPartitions that exist in the world
		AddMeshPartitionToGuidMap(MeshPartition);

		TTuple<TArray<MeshPartition::FModifierDesc>, TArray<FCompiledSectionDescriptor>>* Descriptors = MeshPartitionsInWorld.Find(MeshPartitionGuid);
		if ((Descriptors == nullptr) || (Descriptors->Get<0>().Num() <= 0))
		{
			// no descriptors found.. this mesh partition will be empty
			UE_LOGF(LogMegaMeshEditor, Warning, "MeshPartition '%ls' (GUID '%ls') does not have any modifiers in the world, the MeshPartition will be empty",
				*FSoftObjectPath(MeshPartition).ToString(),
				*MeshPartitionGuid.ToString());
			continue;
		}

		// and try to build the section groups for it
		FSoftObjectPath ObjectPath(MeshPartition);
		TSharedPtr<MeshPartition::FCachedDescriptors>* CacheEntryPtr = CachedDescriptors.Find(ObjectPath);
		TSharedPtr<MeshPartition::FCachedDescriptors> CacheEntry;
		TConstArrayView<MeshPartition::FModifierDesc> AllModifiersSorted = Descriptors->Get<0>();
		TArray<FCompiledSectionDescriptor>& AllCompiledSections = Descriptors->Get<1>();
		if (CacheEntryPtr == nullptr)
		{
			CacheEntry = MakeShared<MeshPartition::FCachedDescriptors>(MeshPartition, AllModifiersSorted, MoveTemp(AllCompiledSections));
			CachedDescriptors.Emplace(ObjectPath, CacheEntry);
		}
		else
		{
			// TODO: we could validate and/or update the existing section...  for now we just stomp it
			CacheEntry = MakeShared<MeshPartition::FCachedDescriptors>(MeshPartition, AllModifiersSorted, MoveTemp(AllCompiledSections));
			CachedDescriptors.Emplace(ObjectPath, CacheEntry);
		}
	}
	
	UE_LOGF(LogCore, Verbose, "UpdateCacheForAllMeshPartitionsInWorld (%d Modifiers, %d Compiled Sections)", Modifiers, CompiledSections);

	if (OutAllMeshPartitionGuidsInWorld != nullptr)
	{
		MeshPartitionsInWorld.GetKeys(*OutAllMeshPartitionGuidsInWorld);
	}
}

TSharedPtr<MeshPartition::FCachedDescriptors> FModifierDescriptorCache::GetCachedDescriptorsByPath(FSoftObjectPath MeshPartitionPath)
{
	if (TSharedPtr<MeshPartition::FCachedDescriptors>* CacheEntryPtr = CachedDescriptors.Find(MeshPartitionPath))
	{
		return *CacheEntryPtr;
	}
	return nullptr;
}

TSharedPtr<MeshPartition::FCachedDescriptors> FModifierDescriptorCache::GetCachedDescriptorsByGuid(FGuid MeshPartitionGUID)
{
	TSharedPtr<MeshPartition::FCachedDescriptors>* CacheEntryPtr = nullptr;

	TArray<FSoftObjectPath> MeshPartitionPaths;
	GuidToMeshPartitionPath.MultiFind(MeshPartitionGUID, MeshPartitionPaths);

	for (FSoftObjectPath MeshPartitionPath : MeshPartitionPaths)
	{
		CacheEntryPtr = CachedDescriptors.Find(MeshPartitionPath);
		if (CacheEntryPtr != nullptr)
		{
			return *CacheEntryPtr;
		}
	}

	return nullptr;
}

void FModifierDescriptorCache::ForAllCachedDescriptors(TFunctionRef<bool(const FSoftObjectPath&, const TSharedPtr<MeshPartition::FCachedDescriptors>&)> InFunc) const
{
	using FMeshPartitionDescriptorsPair = TPair<FSoftObjectPath, TSharedPtr<MeshPartition::FCachedDescriptors>>;

	TArray<FSoftObjectPath> Keys;

    Keys.Reserve(CachedDescriptors.Num());

	for (const FMeshPartitionDescriptorsPair& MeshPartitionDescriptorsPair : CachedDescriptors)
	{
		Keys.Emplace(MeshPartitionDescriptorsPair.Key);
	}

	for (const FSoftObjectPath& Key : Keys)
	{
		const TSharedPtr<MeshPartition::FCachedDescriptors>* CachedDescriptor = CachedDescriptors.Find(Key);

		if (CachedDescriptor == nullptr)
		{
			continue;
		}

		if (!InFunc(Key, *CachedDescriptor))
		{
			return;
		}
	}
}

TSharedPtr<MeshPartition::FCachedDescriptors> FModifierDescriptorCache::GetCachedDescriptors(
	const AMeshPartition* MeshPartition)
{
	check(MeshPartition);
	FSoftObjectPath MeshPartitionPath(MeshPartition);
	
	if (TSharedPtr<MeshPartition::FCachedDescriptors>* CacheEntryPtr = CachedDescriptors.Find(MeshPartitionPath))
	{
		return *CacheEntryPtr;
	}
	else
	{
		// try looking it up by GUID... this will effectively grab the cached descriptors from the editor-world
		FGuid MeshPartitionGuid = MeshPartition->GetActorGuid();
		return GetCachedDescriptorsByGuid(MeshPartitionGuid);
	}
}

TSharedPtr<MeshPartition::FCachedDescriptors> FWorldCachedDescriptors::GetCachedDescriptorsByPath(const FSoftObjectPath& MegaMeshPath) const
{
	if (const TSharedPtr<MeshPartition::FCachedDescriptors>* CacheEntryPtr = CachedDescriptorsByPath.Find(MegaMeshPath))
	{
		return *CacheEntryPtr;
	}
	return nullptr;
}

TSharedPtr<MeshPartition::FCachedDescriptors> FWorldCachedDescriptors::GetCachedDescriptorsByGuid(const FGuid& MeshPartitionGUID) const
{
	if (const TSharedPtr<MeshPartition::FCachedDescriptors>* CacheEntryPtr = CachedDescriptorsByGUID.Find(MeshPartitionGUID))
	{
		return *CacheEntryPtr;
	}
	return nullptr;
}

TSharedPtr<MeshPartition::FCachedDescriptors> FWorldCachedDescriptors::GetCachedDescriptors(const AMeshPartition& MeshPartitionActor) const
{
	// first try to look up by path
	FSoftObjectPath MeshPartitionPath(&MeshPartitionActor);
	if (const TSharedPtr<MeshPartition::FCachedDescriptors>* CacheEntryPtr = CachedDescriptorsByPath.Find(MeshPartitionPath))
	{
		return *CacheEntryPtr;
	}
	else
	{
		// otherwise look it up by guid
		const FGuid& MeshPartitionGUID = MeshPartitionActor.GetActorGuid();
		return GetCachedDescriptorsByGuid(MeshPartitionGUID);
	}
}

TSharedPtr<MeshPartition::FCachedDescriptors> FWorldCachedDescriptors::FindOrAddByActor(const AMeshPartition& MeshPartitionActor)
{
	TSharedPtr<MeshPartition::FCachedDescriptors> Descriptors = GetCachedDescriptors(MeshPartitionActor);
	if (Descriptors.IsValid())
	{
		check(Descriptors->MeshPartitionActor == &MeshPartitionActor);
		check(Descriptors->MeshPartitionGUID == MeshPartitionActor.GetActorGuid());
	}
	else
	{
		// create a new entry
		const FGuid& MeshPartitionGUID = MeshPartitionActor.GetActorGuid();
		check(MeshPartitionGUID.IsValid());

		FSoftObjectPath MeshPartitionPath(&MeshPartitionActor);
		check(MeshPartitionPath.IsValid());

		Descriptors = MakeShared<MeshPartition::FCachedDescriptors>(MeshPartitionGUID, &MeshPartitionActor);

		// populate the lookups
		AllMeshPartitionsCachedDescriptors.Add(Descriptors);
		CachedDescriptorsByGUID.Add(MeshPartitionGUID, Descriptors);
		CachedDescriptorsByPath.Add(MeshPartitionPath, Descriptors);
	}
	return Descriptors;
}

TSharedPtr<MeshPartition::FCachedDescriptors> FWorldCachedDescriptors::FindOrAddByGUID(const FGuid& MeshPartitionGUID)
{
	check(MeshPartitionGUID.IsValid());
	TSharedPtr<MeshPartition::FCachedDescriptors> Descriptors = GetCachedDescriptorsByGuid(MeshPartitionGUID);
	if (Descriptors.IsValid())
	{
		check(Descriptors->MeshPartitionGUID == MeshPartitionGUID);
	}
	else
	{
		// create a new entry (with no path)
		Descriptors = MakeShared<MeshPartition::FCachedDescriptors>(MeshPartitionGUID, nullptr);

		// populate the lookups (except path lookup)
		AllMeshPartitionsCachedDescriptors.Add(Descriptors);
		CachedDescriptorsByGUID.Add(MeshPartitionGUID, Descriptors);
	}
	return Descriptors;
}

TSharedPtr<FWorldCachedDescriptors> FModifierDescriptorCache::GetDescriptorsForAllMeshPartitionsInWorld(UWorld* InWorld)
{
	using namespace Utils;

	check(InWorld);

	TSharedPtr<FWorldCachedDescriptors> WorldDescriptors = MakeShared<FWorldCachedDescriptors>();

	// check if this is a PIE world
	bool bIsPIE = InWorld->IsPlayInEditor();

	// first iterate the mesh partitions that exist, and add them to MegaMeshesInWorld
	// (this does not include spatially streaming AMeshPartitions or those inside of level instances, but we don't support either of those cases)
	for (TActorIterator<AMeshPartition> It(InWorld); It; ++It)
	{
		const AMeshPartition* MeshPartition = *It;
		if (ensure(MeshPartition))
		{
			// populate an entry for this mesh partition actor
			WorldDescriptors->FindOrAddByActor(*MeshPartition);
		}
	}

	UWorldPartition* WorldPartition = InWorld->GetWorldPartition();
	if (WorldPartition == nullptr)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "World '%ls' is not World Partitioned, and does not support MeshPartition operations", *FSoftObjectPath(InWorld).ToString());
		return WorldDescriptors;
	}

	UE::MeshPartition::EditorUtils::FPIEPathFixer& PIEPathFixer = UMeshPartitionEditorSubsystem::GetPIEPathFixer();

	// Now iterate all of the descriptors in the world, and record them into the corresponding FCachedDescriptors
	// NOTE: this only grabs actor descs from the Main world/level -- not from any level instances
	ForEachMegaMeshDescInWorldPartition(WorldPartition,
		[&WorldDescriptors, bIsPIE, &PIEPathFixer](MeshPartition::FModifierDesc& InDescriptor)
		{
			if (!InDescriptor.MegaMeshGuid.IsValid())
			{
				// TODO: we should warn about this modifier without an assigned mesh partition
				return;
			}

			if (bIsPIE)
			{
					// replace the PIE package name with the non-PIE package name
					PIEPathFixer.FixInPlace(InDescriptor.ModifierPath);
			}

			TSharedPtr<MeshPartition::FCachedDescriptors> Descriptors = WorldDescriptors->FindOrAddByGUID(InDescriptor.MegaMeshGuid);
			// TODO: this is a slow add as it may insert in the middle, for O(N^2) worst case.  If we allowed modifier groups to be in an unsorted state, we could use a faster "AddUnsorted" instead
			Descriptors->AllModifiersGroup.Add(InDescriptor);
			WorldDescriptors->TotalModifiers++;
		},
		[&WorldDescriptors](FCompiledSectionDescriptor& Descriptor)
		{
			// ignore old descriptors that don't have megamesh GUID populated (these should not exist, outside of old test levels)
			if (!Descriptor.Info.MegaMeshGUID.IsValid())
			{
				UE_LOGF(LogMegaMeshEditor, Warning, "Encountered Compiled Section descriptor without a valid GUID");
				return;
			}

			TSharedPtr<MeshPartition::FCachedDescriptors> Descriptors = WorldDescriptors->FindOrAddByGUID(Descriptor.Info.MegaMeshGUID);
			Descriptors->CompiledSections.Add(Descriptor);
			WorldDescriptors->TotalCompiledSections++;
		});

	// Now iterate our collection of mesh partitions and Build the section groups for each
	for (TSharedPtr<MeshPartition::FCachedDescriptors> Descriptors : WorldDescriptors->AllMeshPartitionsCachedDescriptors)
	{
		const FGuid& MeshPartitionGUID = Descriptors->MeshPartitionGUID;

		const AMeshPartition* MeshPartitionActor = Descriptors->MeshPartitionActor.Get();
		if (MeshPartitionActor == nullptr)
		{
			// no mesh partition actor -- possibly deleted, marked spatially streaming (unsupported), or moved to a level instance (unsupported)
			// this is ok if we just have compiled sections, they could be leftover from before the mesh partition was deleted, and just need to be cleaned up.
			// however if we still have modifiers referencing this mesh partition, then the user needs to remove/fix those
			if (Descriptors->GetAllModifiers().Num() > 0)
			{
				UE_LOGF(LogMegaMeshEditor, Warning, "Could not find mesh partition actor with GUID '%ls', referenced by %d modifiers", *MeshPartitionGUID.ToString(), Descriptors->GetAllModifiers().Num());
			}
		}
		else
		{
			check(MeshPartitionActor->GetActorGuid() == MeshPartitionGUID);

			const UMeshPartitionDefinition* MeshPartitionDefinition = MeshPartitionActor->GetMeshPartitionDefinition();
			if (MeshPartitionDefinition == nullptr)
			{
				UE_LOGF(LogCore, Warning, "No MegaMesh Definition provided for %ls (%.6ls), using default settings.", *MeshPartitionActor->GetName(), *MeshPartitionGUID.ToString());
				MeshPartitionDefinition = UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
			}

			// Sort the Modifiers by priority (and Base/Non-Base)
			Descriptors->AllModifiersGroup.Sort(MeshPartitionDefinition->GetModifierTypePriorities());

			if (Descriptors->AllModifiersGroup.BaseDescs().IsEmpty())
			{
				// no modifiers found.. this megamesh will be empty
				FSoftObjectPath MeshPartitionActorPath(MeshPartitionActor);
				UE_LOGF(LogMegaMeshEditor, Warning, "Mesh Partition '%ls' (GUID %ls) does not have any base modifiers in the world, the mesh partition will be empty", *MeshPartitionActorPath.ToString(), *MeshPartitionGUID.ToString());
			}

			// iterate all build variants and build the section grouping for each
			TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariantArray = MeshPartitionDefinition->GetCompiledSectionBuildVariants();
			check(BuildVariantArray.Num() > 0);
			for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : BuildVariantArray)
			{
				FModifierGrouping& NewGroup = Descriptors->BuildVariants.Add(BuildVariant.Name, MeshPartition::FModifierGrouping(BuildVariant, *MeshPartitionDefinition, Descriptors->AllModifiersGroup));
				WorldDescriptors->TotalBuildVariants++;
				WorldDescriptors->TotalSectionGroups += NewGroup.GetModifierGroups().Num();
			}
		}
	}
	
	UE_LOGF(LogCore, Verbose, "GetDescriptorsForAllMeshPartitionsInWorld (%d Modifiers, %d Compiled Sections, %d Variants, %d Groups)", WorldDescriptors->TotalModifiers, WorldDescriptors->TotalCompiledSections, WorldDescriptors->TotalBuildVariants, WorldDescriptors->TotalSectionGroups);

	return WorldDescriptors;
}

} // namespace UE::MeshPartition
