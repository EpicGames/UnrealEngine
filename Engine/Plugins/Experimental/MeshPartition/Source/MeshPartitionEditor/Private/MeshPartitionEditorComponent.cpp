// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorComponent.h"

#include "ActionableMessageSubsystem.h"
#include "Algo/AllOf.h"
#include "Algo/AnyOf.h"
#include "AssetCompilingManager.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "Components/DynamicMeshComponent.h"
#include "CoreGlobals.h" // GUndo
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMeshToMeshDescription.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Editor.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSourceData.h"
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "Engine/Texture2DArray.h"
#include "Engine/TextureCollection.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/TextureRenderTarget2DArray.h"
#include "Engine/World.h"
#include "Framework/Application/SlateApplication.h"
#include "MaterialDomain.h"
#include "Misc/UObjectToken.h"
#include "Logging/MessageLog.h"

#include "MeshPartition.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorUtils.h"
#include "MeshPartitionGroupRegistry.h"
#include "MeshPartitionInteractiveSection.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "MeshPartitionChannelCollection.h"

#include "Materials/Material.h"
#include "Materials/MaterialInstanceConstant.h"

#include "MeshPartitionEditorModule.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionStaticMeshComponent.h"
#include "Misc/ScopedSlowTask.h"
#include "Selection.h"
#include "Tasks/Task.h"
#include "TextureResource.h"
#include "VisualLogger/VisualLogger.h"
#include "MeshPartitionModifierTaskGraph.h"
#include "RenderGraphUtils.h"
#include "SceneInterface.h"
#include "ScreenPass.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionEditorWorldSubsystem.h"
#include "MeshPartitionMeshBuilder.h"
#include "MaterialCache/MaterialCache.h"

#include "StructUtils/InstancedStruct.h"

#define LOCTEXT_NAMESPACE "MegaMeshEditorComponent"

namespace UE::MeshPartition
{

// Defined in MeshPartitionChannelCollection.cpp next to CVarMeshPartition_UnwrapUVsMethod
extern TAutoConsoleVariable<int32> CVarMeshPartition_PreviewSectionUVQuality;

static TAutoConsoleVariable<bool> CVarPauseTransformerPipeline(TEXT("MegaMesh.Preview.PauseTransformerPipeline"),
														       true,
														       TEXT("Pauses the transformer pipeline when modifier are selected (static mesh, collisions, far field, subsections builds) ."));

static TAutoConsoleVariable<bool> CVarRecomputeTangents(TEXT("MegaMesh.Preview.RecomputeTangents"),
													    false,
													    TEXT("Recompute tangents when static mesh build is paused."));

static TAutoConsoleVariable<bool> CVarPreviewForceSynchronousBuild(TEXT("MegaMesh.Preview.ForceSynchronousBuild"),
													    	       false,
													    	       TEXT("If enabled, preview section build will be synchronous (modifiers execution would still be asynchronous)."));

static TAutoConsoleVariable<bool> CVarEnablePreviewSectionDDCWrite(TEXT("MegaMesh.Preview.EnableDDCWrite"),
													    	       true,
													    	       TEXT("If enabled, preview section builds will write built mesh data to ddc."));

static TAutoConsoleVariable<bool> CVarEnablePreviewSectionDDCRead(TEXT("MegaMesh.Preview.EnableDDCRead"),
													    	       true,
													    	       TEXT("If enabled, preview section builds will attempt to use DDC to retrieve built mesh data without computing it if possible."));

static TAutoConsoleVariable<bool> CVarInteractiveModeEnabled(TEXT("MegaMesh.InteractiveMode.Enable"),
													    	       false,
													    	       TEXT("If enabled, will freeze the stack and apply modifications only from the currently selected modifiers."));

static TAutoConsoleVariable<bool> CVarValidatePreviewSectionsGroupRegistry(TEXT("MegaMesh.Preview.ValidateGroupRegistry.Enable"),
															               false,
															               TEXT("If enabled, check if the GroupRegistry and PreviewSections are in sync for each frame."));

TAutoConsoleVariable<bool> CVarMegaMeshEnablePreviewSimplification(TEXT("MegaMesh.Preview.EnableSimplification"),
															 	   false,
															 	   TEXT("If enabled, will use simplifier in the builder and disable any remeshing operations to speedup processing the mesh."));

TAutoConsoleVariable<float> CVarMegaMeshPreviewSimplificationEdgeLength(TEXT("MegaMesh.Preview.SimplificationEdgeLength"),
																  	    10,
																  	    TEXT("The edge length to be used with the simplifier."));

TAutoConsoleVariable<int32> CVarMegaMeshPreviewSimplificationMinVertexNumber(TEXT("MegaMesh.Preview.SimplificationMinVertexNumber"),
																		     1000000,
																		     TEXT("The threshold to reach before the simplification kicks in."));


namespace MegaMeshEditorComponentLocals
{
	UMaterialInstanceDynamic* CreateMaterialInstance(const UMaterialInterface* InBaseMegaMeshMaterial, UObject* InOuter, const FName& InName, EObjectFlags InAdditionalObjectFlags = RF_NoFlags);
	UMaterialInstanceDynamic* GetOrCreateMaterialInstance(UMaterialInstanceDynamic* InMID, const UMaterialInterface* InBaseMegaMeshMaterial, UObject* InOuter, const FName& InName, EObjectFlags InAdditionalObjectFlags = RF_NoFlags);
	UMaterialInterface* CreateDefinitionRuntimeMaterial(UObject* InOuter, MeshPartition::UMeshPartitionDefinition* InDefinition);
	const FName SimplifiedPreviewSectionTag = TEXT("SimplifiedPreviewSection");

	void ValidatePreviewSectionsGroupRegistry(const TArray<TObjectPtr<MeshPartition::APreviewSection>>& InPreviewSections, const MeshPartition::FModifierGroupRegistry& InGroupRegistry)
	{
		ensure(InPreviewSections.Num() == InGroupRegistry.GetGroups().Num());
	
		for (MeshPartition::APreviewSection* PreviewSection : InPreviewSections)
		{
			FGuid PreviewSectionGroupRegistryKey = PreviewSection->GetGroupRegistryKey();
			const MeshPartition::FModifierGroup* GroupRegistryGroup = InGroupRegistry.FindGroup(PreviewSectionGroupRegistryKey);

			if (!ensure(GroupRegistryGroup != nullptr))
			{
				continue;
			}

			int32 BaseNumber = 0;
			const TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>>& PreviewSectionBaseModifiers = PreviewSection->GetBaseModifiers();
			
			GroupRegistryGroup->ForEachBase([&BaseNumber, &PreviewSectionBaseModifiers](MeshPartition::UModifierComponent* Base)
			{
				ensure(PreviewSectionBaseModifiers.Find(Base) != INDEX_NONE);
				++BaseNumber;

				return true;
			});
			
			ensure(PreviewSectionBaseModifiers.Num() == BaseNumber);

			int32 ModifierNumber = 0;
			const TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>>& PreviewSectionModifiers = PreviewSection->GetModifiers();
			
			GroupRegistryGroup->ForAllModifiers([&ModifierNumber, &PreviewSectionModifiers](MeshPartition::UModifierComponent* Modifier)
			{
				ensure(PreviewSectionModifiers.Find(Modifier) != INDEX_NONE);
				++ModifierNumber;

				return true;
			});
			
			ensure(PreviewSectionModifiers.Num() == ModifierNumber);
		}
	}
}

UMeshPartitionEditorComponent::UMeshPartitionEditorComponent()
{
	bTickInEditor = false;
	PrimaryComponentTick.bCanEverTick = false;
}

UMeshPartitionEditorComponent::~UMeshPartitionEditorComponent()
{
}

void UMeshPartitionEditorComponent::OnUnregister()
{
	// Skip the slow task dialog while the user is interacting with a Slate widget that captured the mouse
	// (since otherwise it can steal focus and break the drag + potentially mess up active transactions)
	const bool bSkipDialog = FSlateApplication::IsInitialized() && FSlateApplication::Get().HasAnyMouseCaptor();

	// Here we are probably nested under the "Clearing existing world" SlowTask. Using anything besides MakeDialog could prevent any of the slowtask showing.
	FScopedSlowTask SlowTask(PreviewSectionTransformerContexts.Num(), LOCTEXT("PreviewSectionTaskCompletion_SlowTask", "Waiting Mesh Partition Preview Section tasks to complete..."), !bSkipDialog);
	SlowTask.MakeDialog();
	
	// First mark all tasks as cancelled in case some did not start yet and will benefit the early branchout.
	for (const TUniquePtr<MeshPartition::FTransformerContext>& TransformerContext : PreviewSectionTransformerContexts)
	{
		TransformerContext->bWasCancelled = true;
	}

	for (const TUniquePtr<MeshPartition::FTransformerContext>& TransformerContext : PreviewSectionTransformerContexts)
	{
		SlowTask.EnterProgressFrame(1.f);

		if (!TransformerContext->JoinTask.IsValid())
		{
			continue;
		}

		WaitOnGameThread(*TransformerContext);
	}

	constexpr bool bCompleteAllTasks = true;
	FinalizeAsyncTasks(bCompleteAllTasks);

	Super::OnUnregister();
}

void UMeshPartitionEditorComponent::CheckForErrors()
{
	Super::CheckForErrors();

	const UWorld* World = GetWorld();
	if (!IsTemplate() && World && World->WorldType != EWorldType::EditorPreview)
	{
		if (World->GetWorldPartition() == nullptr)
		{
			FMessageLog("MapCheck").AddMessage(FTokenizedMessage::Create(EMessageSeverity::Error)
						->AddToken(FUObjectToken::Create(this))
						->AddToken(FTextToken::Create(
							LOCTEXT("MapCheck_Message_MeshPartitionInNonWPWorld", "Mesh Partition requires a map with World Partition enabled.")
						)));
		}
	}
}

void UMeshPartitionEditorComponent::PostRegisterMegaMeshComponents()
{
	// This is necessary because actor deletion will null out our CurrentModifiers before we get a chance to uninitialize
	//  them in OnLevelActorListChanged.
	GEngine->OnLevelActorDeleted().AddUObject(this, &UMeshPartitionEditorComponent::OnLevelActorDeleted);
	
	if (MeshPartition::UMeshPartitionDefinition* Definition = GetMegaMeshDefinition())
	{
		Definition->GetOnDefinitionModified().AddUObject(this, &UMeshPartitionEditorComponent::OnDefinitionModified);

		UpdateMaterial();
	}
	
	SetBaseModifiersHidden(true);
	SetPreviewSectionsVisibility(true);

	USelection::SelectionChangedEvent.AddUObject(this, &UMeshPartitionEditorComponent::OnSelectionChanged);

	UWorld* World = GetWorld();
	
	if (!ensure(World != nullptr))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get World.");
		return;
	}
	
	if (World->PersistentLevel)
	{
		World->PersistentLevel->OnLoadedActorAddedToLevelPostEvent.AddUObject(this, &UMeshPartitionEditorComponent::OnLoadedActorAddedToLevel);
		World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.AddUObject(this, &UMeshPartitionEditorComponent::OnLoadedActorRemovedFromLevel);
	}
	// Auto-managed by the pipeline; transient so transactions never capture it.
	
	InteractiveSection = SpawnTransientActor<MeshPartition::AInteractiveSection>();

	constexpr bool bIsHidden = true;
	InteractiveSection->SetParent(Cast<AMeshPartition>(GetOwner()));
	InteractiveSection->SetIsTemporarilyHiddenInEditor(bIsHidden);

	// Forces modifier rediscovery on the next Update (e.g. when this register comes from undoing a delete).
	bPendingModifierListChange = true;

	// Queue a rebuild on every register. PostUnregister always tears the previews down, and not
	// every register path naturally re-queues a build (e.g. transform undo/redo). Coalesces with
	// any natural triggers via PendingChangedBounds, so it doesn't add extra build work.
	ForceRebuildAllSections(MeshPartition::EChangeType::StateChange);

	if (IConsoleVariable* ConsoleVariable = CVarMegaMeshEnablePreviewSimplification.AsVariable())
	{
		ConsoleVariable->OnChangedDelegate().AddUObject(this, &UMeshPartitionEditorComponent::OnPreviewSectionSimplificationEnabledChanged);
	}
}

void UMeshPartitionEditorComponent::PostUnregisterMegaMeshComponents()
{
	if (IConsoleVariable* ConsoleVariable = CVarMegaMeshEnablePreviewSimplification.AsVariable())
	{
		ConsoleVariable->OnChangedDelegate().RemoveAll(this);
	}

	UWorld* World = GetWorld();

	// Always tear down sub-actors here. This wastes work for unregisters that pair with a re-register
	// (e.g. transform undo/redo), but the self-heal in Update() rebuilds in those cases, and trying
	// to filter only destroy-driven unregisters is unreliable.
	if (ensure(World != nullptr))
	{
		if (InteractiveSection != nullptr)
		{
			World->DestroyActor(InteractiveSection);
			InteractiveSection = nullptr;
		}

		// Pass a copy: DestroyPreviewSections mutates PreviewSections as it iterates the input.
		TArray<TObjectPtr<MeshPartition::APreviewSection>> PreviewSectionsCopy = PreviewSections;
		DestroyPreviewSections(PreviewSectionsCopy);

		if (!ensureMsgf(InteractivePreviewSections.IsEmpty(), TEXT("Clearing PreviewSections failed to clear InteractivePreviewSections, which should be a subset")))
		{
			TArray<TObjectPtr<MeshPartition::APreviewSection>> InteractivePreviewSectionsCopy = InteractivePreviewSections;
			DestroyPreviewSections(InteractivePreviewSectionsCopy);
		}

		if (World->PersistentLevel)
		{
			World->PersistentLevel->OnLoadedActorAddedToLevelPostEvent.RemoveAll(this);
			World->PersistentLevel->OnLoadedActorRemovedFromLevelEvent.RemoveAll(this);
		}
	}
	else
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get World.");
	}
	
	USelection::SelectionChangedEvent.RemoveAll(this);

    if (MeshPartition::UMeshPartitionDefinition* Definition = GetMegaMeshDefinition())
    {
        Definition->GetOnDefinitionModified().RemoveAll(this);
    }

	GEngine->OnLevelActorDeleted().RemoveAll(this);
}

void UMeshPartitionEditorComponent::Update()
{
	UpdateModifierList();

	FinalizeAsyncTasks();

	if (!PendingChangedBounds.IsEmpty() || !PendingChangedModifiers.IsEmpty())
	{
		TSet<FBox> BoundsToBuild;
		for (const TPair<MeshPartition::EChangeType, TSet<FBox>>& ChangedBounds : PendingChangedBounds)
		{
			BoundsToBuild.Append(ChangedBounds.Value);
		}

		for (const TPair<FSoftObjectPath, TArray<MeshPartition::FModifierChangeInfo>>& Pair : PendingChangedModifiers)
		{
			for (const MeshPartition::FModifierChangeInfo& ModifierChange : Pair.Value)
			{
				BoundsToBuild.Append(ModifierChange.ChangedBounds);
			}
		}

		BuildMegaMeshPreviewSections(BoundsToBuild.Array());

		if (UWorld* World = GetWorld())
		{
			if (UMeshPartitionEditorSubsystem* EditorSubsystem = UMeshPartitionEditorSubsystem::Get())
			{
				EditorSubsystem->OnMegaMeshChanged().Broadcast({this, PendingChangedModifiers, PendingChangedBounds});
			}
		}

		PendingChangedBounds.Reset();
		PendingChangedModifiers.Reset();
	}

	FinalizePreviewSectionBuilds();
	
	if (CVarValidatePreviewSectionsGroupRegistry.GetValueOnGameThread() && !IsRunningCommandlet())
	{
		MegaMeshEditorComponentLocals::ValidatePreviewSectionsGroupRegistry(PreviewSections, GroupRegistry);
	}
}

void  UMeshPartitionEditorComponent::SetBuildModifierFilterFunction(MeshPartition::FModifierFilterFunc InFunc)
{
	BuildModifierFilterFunc = InFunc;
}

void  UMeshPartitionEditorComponent::ClearBuildModifierFilterFunction()
{
	BuildModifierFilterFunc.Reset();
}

void UMeshPartitionEditorComponent::ForceRebuildAllSections(const MeshPartition::EChangeType InChangeType)
{
	FBox InfiniteBounds(FVector(-FLT_MAX), FVector(FLT_MAX));
	OnBoundsChanged({ InfiniteBounds }, InChangeType);
}

void UMeshPartitionEditorComponent::BuildMegaMeshPreviewSections(TConstArrayView<FBox> InBoundingBoxes)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::BuildMegaMeshPreviewSections);
	
	const AMeshPartition* MegaMesh = GetOwner<const AMeshPartition>();
	
	if (!ensure(MegaMesh != nullptr))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get MegaMesh.");
		return;
	}

	if (InBoundingBoxes.IsEmpty())
	{
		return;
	}

	if (!IsPreviewSectionBuildEnabled())
	{
		return;
	}

	FPreviewSectionBuildContext& PreviewSectionBuildContext = PreviewSectionBuildContexts.Emplace_GetRef();
	TArray<FBox> BoundsToBuild = GetIntersectingPreviewSections(InBoundingBoxes, PreviewSectionBuildContext.IntersectingPreviewSections);

	
	UMeshPartitionDefinition* Definition = GetMegaMeshDefinition();
	if (Definition == nullptr)
	{
		Definition = UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
	}

	MeshPartition::FBuilderSettings Settings;
	Settings.BuildType = MeshPartition::EBuildType::PreviewSection;
	Settings.Transform = MegaMesh->GetTransform();
	Settings.TypePriorities = Definition->GetModifierTypePriorities();
	Settings.MaxSectionComplexity = GetPreviewBuildVariant().MaxSectionComplexity;
	Settings.bRecomputeNormals = true;
	Settings.bRecomputeTangents = ShouldRecomputeTangents();
	Settings.bCacheResult = true;
	Settings.bAllowDDCWrite = CVarEnablePreviewSectionDDCWrite.GetValueOnGameThread();
	Settings.bAllowDDCRead = CVarEnablePreviewSectionDDCRead.GetValueOnGameThread();

	Settings.TexcoordGenerationOptions = FChannelCollectionUVLayoutOptions::GetFromDefinition(GetMegaMeshDefinition());
	// Editor preview path: quality is set via MeshPartition.Preview.UVQuality. Compiled-section paths always use UVQuality at Full
	Settings.TexcoordGenerationOptions->UVQuality = CVarMeshPartition_PreviewSectionUVQuality.GetValueOnGameThread() == 1 ? EChannelUVUnwrapQuality::Preview : EChannelUVUnwrapQuality::Full;

	MeshPartition::FBuilderSettings::FChannelRenderSettings ChannelRenderSettings;
	ChannelRenderSettings.ChannelMap = Definition->GetChannelMap();
	ChannelRenderSettings.TexelSize = Definition->GetChannelTexelSize();
	Settings.ChannelRenderSettings.Emplace(MoveTemp(ChannelRenderSettings));

	if (BuildModifierFilterFunc.IsSet())
	{
		Settings.ModifierFilter = BuildModifierFilterFunc;
	}

	if (CVarMegaMeshEnablePreviewSimplification.GetValueOnGameThread())
	{
		Settings.BuildType = MeshPartition::EBuildType::SimplifiedPreviewSection;
		Settings.SimplifierOptions.Emplace(MeshPartition::FBuilderSettings::FSimplifierOptions
		{
			CVarMegaMeshPreviewSimplificationEdgeLength.GetValueOnGameThread(),
			CVarMegaMeshPreviewSimplificationMinVertexNumber.GetValueOnGameThread()
		});
	}

	TSet<MeshPartition::UModifierComponent*> ModifiersToProcess;
	GetModifiersToProcessForPreviews(ModifiersToProcess, PreviewSectionBuildContext.IntersectingPreviewSections, BoundsToBuild);
	InvalidateConflictingPreviewSectionBuildContexts(ModifiersToProcess, PreviewSectionBuildContext.IntersectingPreviewSections, PreviewSectionBuildContext);

	Settings.ModifiersToProcess = ModifiersToProcess.Array();
	
	PreviewSectionBuildContext.BuildTasks = UE::MeshPartition::Build::LaunchBuilds(Settings);
	
	for (const MeshPartition::FBuildTaskHandle& BuildTask : PreviewSectionBuildContext.BuildTasks)
	{
		const MeshPartition::FModifierGroup& Group = BuildTask.GetTask()->GetGroup();
		GroupRegistry.StageGroup(Group);
	}

	if (IsSynchronousPreviewSectionBuildForced())
	{
		UE::MeshPartition::Build::Wait(PreviewSectionBuildContext.BuildTasks);
		FinalizePreviewSectionBuilds();
	}

	UMeshPartitionEditorSubsystem* Subsystem = UMeshPartitionEditorSubsystem::Get();

	if (ensure(Subsystem))
	{
		Subsystem->SetPreviewSectionBuildNumber(this, GetPreviewSectionBuildNumber());
	}
}

TArray<MeshPartition::ACompiledSection*> UMeshPartitionEditorComponent::PrepareCompiledSections(const MeshPartition::FCompiledSectionBuildInfo&		InBuildInfo,
																								const MeshPartition::FCompiledSectionBuildVariant&	InBuildVariant,
																								MeshPartition::FPrepareCompiledSectionsParams&		InOutParams) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::PrepareCompiledSections);

	UWorld* World = GetWorld();
	const AMeshPartition* MeshPartition = GetOwner<const AMeshPartition>();

	if (!ensure(World != nullptr))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "PrepareCompiledSections: Cannot get World.");
		return {};
	}

	if (!ensure(MeshPartition != nullptr))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "PrepareCompiledSections: Cannot get MeshPartition.");
		return {};
	}

	// Resolve the base material once for all sections.
	UMaterialInterface* BaseMaterial = const_cast<UMaterialInterface*>(GetDefinitionMaterial());

	if (!IsValid(BaseMaterial))
	{
		BaseMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
	}

	// Spawns or reuses a single ACompiledSection with material setup and build info.
	auto PrepareSection = [&](const MeshPartition::FCompiledSectionBuildInfo&	InSectionBuildInfo,
							  MeshPartition::ACompiledSection*					InReuseSection = nullptr,
							  bool												bInStaticMobility = true) -> MeshPartition::ACompiledSection*
	{
		MeshPartition::ACompiledSection* Section = InReuseSection;

		if (Section == nullptr)
		{
			FString BaseName = TEXT("CompiledSection_") + InSectionBuildInfo.BuildVariantName.ToString();
			Section = World->SpawnActor<MeshPartition::ACompiledSection>(MeshPartition::ACompiledSection::StaticClass(), FTransform::Identity);

			if (Section == nullptr)
			{
				UE_LOGF(LogMegaMeshEditor, Error, "Failed to spawn ACompiledSection for variant '%ls'", *InSectionBuildInfo.BuildVariantName.ToString());
				return nullptr;
			}

			Section->SetActorLabel(BaseName);
		}

		Section->SetParent(Cast<AMeshPartition>(GetOwner()));

		if (Section->GetRootComponent())
		{
			Section->GetRootComponent()->SetMobility(bInStaticMobility ? EComponentMobility::Static : EComponentMobility::Stationary);
		}

		Section->SetBuildInfo(InSectionBuildInfo);

		UMaterialInstanceConstant* MIC = NewObject<UMaterialInstanceConstant>(Section->GetPackage(), MakeUniqueObjectName(Section, UMaterialInstanceConstant::StaticClass(), TEXT("CompiledSectionMIC")));
		MIC->SetParentEditorOnly(BaseMaterial);
		Section->SetMaterialInstance(MIC);

		return Section;
	};

	TArray<MeshPartition::ACompiledSection*> Sections;
	const bool bIsGridSplitValid = InBuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid && InOutParams.GridSettings.IsGridSplit();

	if (InBuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid && !bIsGridSplitValid && !InOutParams.ReuseSection)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "bSplitSectionsToMatchWorldPartitionRuntimeGrid is true but GridSettings.CellSize could not be resolved -- falling back to single-section build.");
	}

	if (bIsGridSplitValid)
	{
		InOutParams.OutCellMeshes = MeshPartition::GridHelpers::BuildGridCellMeshes(InOutParams.FullMesh, InOutParams.GridSettings, MeshPartition->GetActorTransform());

		Sections.Reserve(InOutParams.OutCellMeshes.Num());

		for (TPair<FIntVector, MeshPartition::FMeshData>& CellEntry : InOutParams.OutCellMeshes)
		{
			MeshPartition::FCompiledSectionBuildInfo CellBuildInfo = InBuildInfo;
			CellBuildInfo.GridCellCoord = CellEntry.Key;
			CellBuildInfo.GridSettings = InOutParams.GridSettings;

			if (MeshPartition::ACompiledSection* Section = PrepareSection(CellBuildInfo))
			{
				Sections.Add(Section);
			}
		}
	}
	else
	{
		if (MeshPartition::ACompiledSection* Section = PrepareSection(InBuildInfo, InOutParams.ReuseSection, InOutParams.bUseStaticMobility))
		{
			Sections.Add(Section);
		}
	}

	return Sections;
}

void UMeshPartitionEditorComponent::ExecuteTransformers(MeshPartition::UPreviewMeshComponent* InPreviewMeshComponent, const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InBuildVariant)
{
	if (!ensure(InPreviewMeshComponent != nullptr) || (InBuildVariant.TransformerPipeline == nullptr))
	{
		return;
	}

	for (const TInstancedStruct<UE::MeshPartition::FTransformer>& Transformer : InBuildVariant.TransformerPipeline->GetTransformers())
	{
		if (!Transformer.IsValid())
		{
			continue;
		}

		Transformer.Get().Execute(*InPreviewMeshComponent, InDefinition, InBuildVariant);
	}
}

TUniquePtr<MeshPartition::FTransformerContext> UMeshPartitionEditorComponent::LaunchTransformers(TArray<MeshPartition::FTransformerUnit>&& InTransformerUnits, const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InBuildVariant)
{
	TUniquePtr<MeshPartition::FTransformerContext> TransformersContext;

	const bool bContainsValidTransformer = (InBuildVariant.TransformerPipeline != nullptr) && Algo::AnyOf(InBuildVariant.TransformerPipeline->GetTransformers(), [](const TInstancedStruct<UE::MeshPartition::FTransformer>& Transformer) { return Transformer.IsValid(); });

	if (!bContainsValidTransformer)
	{
		return TransformersContext;
	}

	TransformersContext = MakeUnique<MeshPartition::FTransformerContext>();

	for (const TInstancedStruct<UE::MeshPartition::FTransformer>& Transformer : InBuildVariant.TransformerPipeline->GetTransformers())
	{
		if (!Transformer.IsValid())
		{
			continue;
		}

		TInstancedStruct<MeshPartition::FTransformer>& CopiedTransformer = TransformersContext->Transformers.Emplace_GetRef(Transformer);
		CopiedTransformer.GetMutable().Initialize(InDefinition, InBuildVariant);
	}

	TransformersContext->TransformerUnits = MoveTemp(InTransformerUnits);

	for (int32 Index = 0; Index < TransformersContext->Transformers.Num(); ++Index)
	{
		TArray<UE::Tasks::FTask> Prerequsites;

		if (TransformersContext->JoinTask.IsValid())
		{
			Prerequsites.Emplace(TransformersContext->JoinTask);
		}

		TransformersContext->JoinTask = UE::Tasks::Launch(TEXT("Transformer_Task"), [this,
			Transformer = TransformersContext->Transformers[Index],
			TransformersContextPtr = TransformersContext.Get()]() mutable
			{
				if (TransformersContextPtr->bWasCancelled)
			  	{
					return;
			  	}

				Transformer.Get().Execute(*TransformersContextPtr);
			}, UE::Tasks::Prerequisites(Prerequsites));
	}

	return TransformersContext;
}

void UMeshPartitionEditorComponent::BuildMegaMeshCompiledSectionTextures(MeshPartition::ACompiledSection* InCompiledSection, const UE::MeshPartition::FModifierGroup& InModifierGroup, MeshPartition::FMeshData& InOutBuiltMesh)
{
	// Build the section texture channel specifically parented to the compiled section
	FChannelMap ChannelMap;
	float ChannelTexelSize = 100;
	float MaterialCacheTexelSize = 1;
	
	if (UMeshPartitionDefinition* Definition = GetMegaMeshDefinition())
	{
		ChannelMap = Definition->GetChannelMap();
		ChannelTexelSize = Definition->GetChannelTexelSize();
		MaterialCacheTexelSize = Definition->GetMaterialCacheTexelSize();
	}

	constexpr bool bDownloadToAsset = true;
	MeshPartition::FSectionChannels SectionChannelTexture = FChannelTextureRenderer::BuildTextureForSection(InOutBuiltMesh, InCompiledSection, bDownloadToAsset, ChannelMap, ChannelTexelSize).GetResult();

	MeshPartition::ApplyChannels(InCompiledSection, SectionChannelTexture, MaterialCacheTexelSize);

	// Coming out from the ChannelCollection section generation the ChannelTexture is ready to be used and parented to the CompiledSection
	if (!SectionChannelTexture.Texture)
	{
		UE_LOGF(LogMegaMeshEditor, Error, "No channel texture for section (%ls).", *InCompiledSection->GetName());
	}

	UPackage* ActorPackage = InCompiledSection->GetOutermost(); // get the actor's package
	ActorPackage->MarkPackageDirty();
}

AActor* UMeshPartitionEditorComponent::SpawnBaseModifier(FDynamicMesh3&& InMesh, const TArray<UMaterialInterface*>& InMaterials, const FTransform& InTransform, const bool bRegisterModifier)
{
	UWorld* World = GetWorld();
	AMeshPartition* MeshPartition = GetOwner<AMeshPartition>();

	if (!ensure(MeshPartition != nullptr))
	{
		return nullptr;	
	}
	
	FActorSpawnParameters Params;
	Params.bCreateActorPackage = true;

	AActor* NewActor = World->SpawnActor<AActor>(Params);

	if (NewActor == nullptr)
	{
		return nullptr;
	}

	NewActor->SetActorLabel(TEXT("MeshPartitionBase"));
	// Is needed for now because MeshPartition::UMeshProviderModifier is EditorOnly but its bounds are needed.
	// If an actor isn't EditorOnly but a component is, the streamingbounds will be invalid in the ActorDesc.
	NewActor->bIsEditorOnlyActor = true;

	// New base modifiers should be flagged spatially loaded by default, so we can unload them
	NewActor->SetIsSpatiallyLoaded(true);

	MeshPartition::UMeshProviderModifier* MeshProvider = NewObject<MeshPartition::UMeshProviderModifier>(
		NewActor, TEXT("MegaMeshMeshProvider"), RF_Transactional);
	MeshProvider->SetIgnoreChanged(true);

	MeshProvider->SetAffectedMeshPartition(MeshPartition);
	MeshProvider->SetMesh(Forward<FDynamicMesh3&&>(InMesh));

	for (int MaterialIndex = 0; MaterialIndex < InMaterials.Num(); ++MaterialIndex)
	{
		MeshProvider->SetMaterial(MaterialIndex, InMaterials[MaterialIndex]);
	}

	MeshProvider->SetIsTemporarilyHiddenInEditor(ShouldHideBaseModifiers());
	MeshProvider->SetIgnoreChanged(false);

	NewActor->SetRootComponent(MeshProvider);
	NewActor->AddInstanceComponent(MeshProvider);
	if (bRegisterModifier)
	{
		MeshProvider->RegisterComponent();
	}

	MeshProvider->SetWorldTransform(InTransform);

	if (bRegisterModifier)
	{
		CurrentModifiers.Emplace(MeshProvider);
	}

	return NewActor;
}

AActor* UMeshPartitionEditorComponent::SpawnTransientActor(TSubclassOf<AActor> InClass, const FTransform& InTransform)
{
	UWorld* World = GetWorld();
	check(World);
	FActorSpawnParameters SpawnParams;
	SpawnParams.ObjectFlags = RF_Transient;
	SpawnParams.OverrideLevel = World->PersistentLevel;
	return World->SpawnActor(InClass, &InTransform, SpawnParams);
}

void UMeshPartitionEditorComponent::UpdateMaterial()
{
	for (MeshPartition::APreviewSection* Section : PreviewSections)
	{
		Section->SetMaterialInstance(MegaMeshEditorComponentLocals::GetOrCreateMaterialInstance(Section->GetMaterialInstance(), GetDefinitionMaterial(), nullptr, TEXT("PreviewMaterialMID")));
	}

	if (InteractiveSection != nullptr)
	{
		InteractiveSection->SetMaterialInstance(MegaMeshEditorComponentLocals::GetOrCreateMaterialInstance(InteractiveSection->GetMaterialInstance(), GetDefinitionMaterial(), nullptr, TEXT("InteractiveMaterialMID")));
	}
}

void UMeshPartitionEditorComponent::PostBuildSectionMesh(AActor* InSection, const MeshPartition::FMeshData& InBuiltMesh, TConstArrayView<TWeakObjectPtr<MeshPartition::UModifierComponent>> InModifiers) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::PostBuildSectionMesh);

	// The PostBuildSectionMesh calls are not currently asynchronous
	ensure(IsInGameThread());

	const FBox SectionBounds(InBuiltMesh.GetBounds());

	for (TWeakObjectPtr<MeshPartition::UModifierComponent> Modifier : InModifiers)
	{
		// A modifier can be destroyed between when an async build is queued and when it finalizes
		if (!Modifier.IsValid())
		{
			continue;
		}

		// When sections are grid-split, a modifier may not overlap this cell — skip it.
		if (SectionBounds.IsValid && !Modifier->IntersectsAnyBounds({ SectionBounds }))
		{
			continue;
		}

		Modifier->PostBuildSectionMesh(InSection, InBuiltMesh);
		Modifier->UpdateLastAppliedBounds();
	}
}

void UMeshPartitionEditorComponent::PostProcessSection(AActor* InSection, TConstArrayView<TWeakObjectPtr<MeshPartition::UModifierComponent>> InModifiers) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::PostProcessSection);

	// The PostProcessSection calls are not currently asynchronous
	ensure(IsInGameThread());

	const FBox SectionBounds = InSection->GetComponentsBoundingBox();

	for (TWeakObjectPtr<MeshPartition::UModifierComponent> Modifier : InModifiers)
	{
		// A modifier can be destroyed between when an async build is queued and when it finalizes
		if (!Modifier.IsValid())
		{
			continue;
		}

		// When sections are grid-split, a modifier may not overlap this cell — skip it.
		if (SectionBounds.IsValid && !Modifier->IntersectsAnyBounds({ SectionBounds }))
		{
			continue;
		}

		Modifier->PostProcessSection(InSection);
		Modifier->UpdateLastAppliedBounds();
	}
}

void UMeshPartitionEditorComponent::ApplyBuiltMeshToSection(MeshPartition::ACompiledSection*					InSection,
															TSharedPtr<const MeshPartition::FMeshData>			InMesh,
															const MeshPartition::FSectionChannels&				InChannels,
															const MeshPartition::FModifierGroup&				InModifierGroup)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::ApplyBuiltMeshToSection);

	const MeshPartition::FCompiledSectionBuildInfo& BuildInfo = InSection->GetBuildInfo();

	// Resolve definition and build variant from the section's build info.
	MeshPartition::UMeshPartitionDefinition* Definition = TSoftObjectPtr<MeshPartition::UMeshPartitionDefinition>(FSoftObjectPath(BuildInfo.MegaMeshDefinitionPath)).Get();

	if (!Definition)
	{
		Definition = MeshPartition::UMeshPartitionDefinition::StaticClass()->GetDefaultObject<MeshPartition::UMeshPartitionDefinition>();
	}

	const MeshPartition::FCompiledSectionBuildVariant& BuildVariant = Definition->GetCompiledSectionBuildVariantByName(BuildInfo.BuildVariantName);

	// Prepare section (material, build info, reuse existing placeholder actor)
	MeshPartition::FPrepareCompiledSectionsParams Params{.FullMesh = *InMesh, .ReuseSection = InSection, .bUseStaticMobility = true};
	TArray<MeshPartition::ACompiledSection*> PreparedSections = PrepareCompiledSections(BuildInfo, BuildVariant, Params);

	if (!ensure(!PreparedSections.IsEmpty()))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "ApplyBuiltMeshToSection: PrepareCompiledSections returned no sections — skipping.");
		return;
	}

	ensureMsgf(PreparedSections[0] == InSection, TEXT("PrepareCompiledSections did not reuse the provided section actor."));

	MeshPartition::ApplyChannels(InSection, InChannels, Definition->GetMaterialCacheTexelSize());

	TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> ModifierPtrs = InModifierGroup.AllResolvedModifierPtrs();
	PostBuildSectionMesh(InSection, *InMesh, ModifierPtrs);

	TArray<MeshPartition::FTransformerUnit> Units;
	Units.Add(MeshPartition::MakeTransformerUnit(InSection, InMesh));

	TUniquePtr<MeshPartition::FTransformerContext> TransformerContext = LaunchTransformers(MoveTemp(Units), Definition, BuildVariant);

	if (TransformerContext.IsValid() && TransformerContext->JoinTask.IsValid() && !TransformerContext->JoinTask.IsCompleted())
	{
		MeshPartition::WaitOnGameThread(*TransformerContext);
	}

	PostProcessSection(InSection, ModifierPtrs);
}

void UMeshPartitionEditorComponent::OnModifierChanged(MeshPartition::UModifierComponent* InChangedModifier, TConstArrayView<FBox> InBoundingBoxes, const MeshPartition::EChangeType InChangeType)
{
	if (!IsInteractiveModeReady())
	{
		QueuePendingModifierChange(InChangedModifier, InBoundingBoxes, InChangeType);
	}
	else
	{
		// #todo: Interactive sections are currently consuming all modifier change events without appropriately broadcasting them when you exit interactive mode.
		// Interactive sections should remember the "strongest" change types that occurred while interacting and they should then broadcast that type after the user exits interactive mode.
		InteractiveSection->OnModifierChanged();
	}
}

void UMeshPartitionEditorComponent::OnBoundsChanged(TConstArrayView<FBox> InBoundingBoxes, const MeshPartition::EChangeType InChangeType)
{
	if (IsInteractiveModeReady())
	{
		InteractiveSection->OnModifierChanged();
	}
	else
	{
		TSet<FBox>& ChangedBounds = PendingChangedBounds.FindOrAdd(InChangeType);
		ChangedBounds.Append(InBoundingBoxes);
	}
}

void UMeshPartitionEditorComponent::OnModifierAssigned()
{
	bPendingModifierListChange = true;
}

TArray<MeshPartition::UModifierComponent*> UMeshPartitionEditorComponent::GetModifiersFiltered(TFunctionRef<bool(const MeshPartition::UModifierComponent*)> InFilter) const
{
	UWorld* World = GetWorld();
	const AMeshPartition* MegaMesh = GetOwner<const AMeshPartition>();
	check(MegaMesh);

	TArray<MeshPartition::UModifierComponent*> Result;

	Algo::CopyIf(CurrentModifiers, Result, [MegaMesh, &InFilter](const MeshPartition::UModifierComponent* Modifier)
	{
		return Modifier != nullptr
			&& Modifier->GetAffectedMeshPartition() == MegaMesh
			&& InFilter(Modifier);
	});

	return Result;
}

void UMeshPartitionEditorComponent::ForAllCurrentModifiers(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc) const
{
	for (MeshPartition::UModifierComponent* Modifier : CurrentModifiers)
	{
		if (Modifier == nullptr)
		{
			continue;
		}

		if (!InFunc(Modifier))
		{
			break;
		}
	}
}

void UMeshPartitionEditorComponent::ForAllPreviewSections(TFunctionRef<bool(MeshPartition::APreviewSection*)> InFunc) const
{
	for (MeshPartition::APreviewSection* PreviewSection : PreviewSections)
	{
		if (PreviewSection == nullptr)
		{
			continue;
		}

		if (!InFunc(PreviewSection))
		{
			break;
		}
	}
}

void UMeshPartitionEditorComponent::SetBaseModifiersHidden(bool bInHideBaseModifiers)
{
	bHideBaseModifiers = bInHideBaseModifiers;

	ForAllCurrentModifiers([&](MeshPartition::UModifierComponent* Modifier)
	{
		if (Modifier->IsBase())
		{
			Modifier->SetIsTemporarilyHiddenInEditor(ShouldHideBaseModifiers());
		}
		return true;
	});
}

uint32 UMeshPartitionEditorComponent::GetModifierLayerIndex(const FName& InModifierLayer) const
{
	if (const MeshPartition::UMeshPartitionDefinition* Definition = GetMegaMeshDefinition(); Definition != nullptr)
	{
		return UE::MeshPartition::FilterHelpers::FindLayerPriorityIndexFromName(Definition->GetModifierTypePriorities(), InModifierLayer);
	}
	else
	{
		return UE::MeshPartition::FilterHelpers::FindLayerPriorityIndexFromName(TConstArrayView<FName>(), InModifierLayer);
	}
}

void UMeshPartitionEditorComponent::SetPreviewSectionsVisibility(const bool bInArePreviewSectionsVisible)
{
	bArePreviewSectionsVisible = bInArePreviewSectionsVisible;
	
	for (MeshPartition::APreviewSection* PreviewSection : PreviewSections)
	{
		if (PreviewSection == nullptr)
		{
			continue;
		}
		
		PreviewSection->SetIsTemporarilyHiddenInEditor(!bInArePreviewSectionsVisible);
	}
}

void UMeshPartitionEditorComponent::SetToolTargetVisibility(TArray<MeshPartition::APreviewSection*> InPreviewSections, const bool bInVisible)
{
	const AMeshPartition* MegaMesh = GetOwner<AMeshPartition>();

	if (MegaMesh == nullptr)
	{
		return;
	}

	bool bInteractiveSectionVisible = false;
	FBox InteractiveSectionBounds;

	if (IsInteractiveModeReady())
	{
		InteractiveSection->SetIsTemporarilyHiddenInEditor(!bInVisible);
		
		constexpr bool bNonColliding = true;
		constexpr bool bIncludeFromChildActors = false;

		InteractiveSectionBounds = InteractiveSection->GetComponentsBoundingBox(bNonColliding, bIncludeFromChildActors);

		bInteractiveSectionVisible = bInVisible;
	}

	for (MeshPartition::APreviewSection* PreviewSection : InPreviewSections)
	{
		if ((PreviewSection == nullptr) || (PreviewSection->GetParent() != MegaMesh))
		{
			continue;
		}

		if (bInteractiveSectionVisible && InteractiveSectionBounds.Intersect(PreviewSection->GetPreviewMeshBounds()))
		{
			constexpr bool bHidden = true;
			PreviewSection->SetIsTemporarilyHiddenInEditor(bHidden);
		}
		else
		{
			PreviewSection->SetIsTemporarilyHiddenInEditor(!bInVisible);
		}
	}
}

MeshPartition::UMeshPartitionDefinition* UMeshPartitionEditorComponent::GetMegaMeshDefinition() const
{
	const AMeshPartition* MeshPartition = GetOwner<AMeshPartition>();

	if (MeshPartition == nullptr)
	{
		return nullptr;
	}
	
	return MeshPartition->GetMeshPartitionDefinition();
}

void UMeshPartitionEditorComponent::OnDefinitionChanged(MeshPartition::UMeshPartitionDefinition* InNewDefinition)
{
	UpdateMaterial();
	
	ForAllCurrentModifiers([InNewDefinition](MeshPartition::UModifierComponent* Modifier)
	{
		Modifier->OnMegaMeshDefinitionChanged(InNewDefinition);
		return true;
	});

	ForAllPreviewSections([InNewDefinition](MeshPartition::APreviewSection* PreviewSection)
	{
		PreviewSection->OnMeshPartitionDefinitionChanged(InNewDefinition);
		return true;	
	});
}

void UMeshPartitionEditorComponent::OnModifierLoaded(MeshPartition::UModifierComponent* InModifier)
{
	if (InModifier == nullptr)
	{
		return;
	}

	InModifier->InitializeModifier();
	if (InModifier->IsBase())
	{
		InModifier->SetIsTemporarilyHiddenInEditor(ShouldHideBaseModifiers());
	}

	if (IsInteractiveModeReady() && (LastActorSelected != nullptr))
	{
		constexpr bool bNonColliding = true;
		constexpr bool bIncludeFromChildActors = false;

		FBox LastActorSelectedBounds = LastActorSelected->GetComponentsBoundingBox(bNonColliding, bIncludeFromChildActors);

		if (InModifier->ComputeCombinedBounds().Intersect(LastActorSelectedBounds))
		{
			InteractiveSection->AddModifier(InModifier);
			InModifier->SetIsInteractive(true);
		}
	}
}

void UMeshPartitionEditorComponent::OnModifierUnloaded(MeshPartition::UModifierComponent* InModifier)
{
	if (InModifier == nullptr)
	{
		return;
	}

	InModifier->UninitializeModifier();

	if (MeshPartition::APreviewSection* PreviewSection = InModifier->GetPreviewSection())
	{
		PreviewSection->RemoveBaseModifier(InModifier);
		InModifier->SetPreviewSection(nullptr);
	}

	if (IsInteractiveModeReady())
	{
		InteractiveSection->RemoveModifier(InModifier);
		InModifier->SetIsInteractive(false);
	}
}

bool UMeshPartitionEditorComponent::IsSynchronousPreviewSectionBuildForced() const
{
	return CVarPreviewForceSynchronousBuild.GetValueOnGameThread() || bForceSynchronousPreviewSectionBuild;
}

void UMeshPartitionEditorComponent::OnLoadedActorAddedToLevel(const TArray<AActor*>&)
{
	bPendingModifierListChange = true;
}

void UMeshPartitionEditorComponent::OnLoadedActorRemovedFromLevel(AActor&)
{
	bPendingModifierListChange = true;

	// Trigger the update to the modifier list immediately to ensure that OnModifierUnloaded is correctly
	// called for unloaded modifiers and all the resources are cleaned up. This has to happen before the modifier
	// is destroyed by the engine so we don't allow a 1-tick delay which would normally occur if this function 
	// were not immediately executed.
	UpdateModifierList();
}

void UMeshPartitionEditorComponent::OnLevelActorDeleted(AActor* Actor)
{
	// We do the same thing that we do for unloaded modifiers in UpdateModifierList(). By the time
	//  we get an OnLevelActorListChanged callback, the modifier will be nulled out, and we would
	//  not get a chance to call OnModifierUnloaded on them.
	TArray<MeshPartition::UModifierComponent*> ActorModifiers;
	Actor->GetComponents<MeshPartition::UModifierComponent>(ActorModifiers);
	for (MeshPartition::UModifierComponent* Modifier : ActorModifiers)
	{
		if (CurrentModifiers.Remove(Modifier) != 0)
		{
			OnModifierUnloaded(Modifier);
			QueuePendingModifierChange(Modifier, Modifier->ComputeBounds(), MeshPartition::EChangeType::StateChange);

			bPendingModifierListChange = true;
			bModifierListChanged = true;
		}
	}
}

void UMeshPartitionEditorComponent::UpdateModifierList()
{
	ON_SCOPE_EXIT
	{ 
		bPendingModifierListChange = false;
		bModifierListChanged = false;
	};

	if (!bPendingModifierListChange)
	{
		return;
	}

	UWorld* World = GetWorld();
	const AMeshPartition* MeshPartition = CastChecked<const AMeshPartition>(GetOwner());

	TSet<MeshPartition::UModifierComponent*> LoadedModifiers;

	for (const AActor* Actor : TActorRange<AActor>(World))
	{
		if (Actor == nullptr)
		{
			continue;
		}

		TArray<MeshPartition::UModifierComponent*> ActorModifiers;
		Actor->GetComponents<MeshPartition::UModifierComponent>(ActorModifiers);

		Algo::CopyIf(ActorModifiers, LoadedModifiers, [MeshPartition](MeshPartition::UModifierComponent* Modifier)
		{
			if (IsValid(Modifier) && Modifier->GetAffectedMeshPartition() == MeshPartition)
			{
				Modifier->MarkAsRegisteredWithMeshPartition(MeshPartition);
				return true;
			}
			return false;
		});
	}

	// We split into loaded/unloaded arrays for ease of debugging
	TArray<MeshPartition::UModifierComponent*> RecentlyLoadedModifiers;

	Algo::CopyIf(LoadedModifiers, RecentlyLoadedModifiers, [this](MeshPartition::UModifierComponent* Modifier)
	{
		return (Modifier != nullptr) && (!CurrentModifiers.Contains(Modifier));
	});
		
	TArray<MeshPartition::UModifierComponent*> RecentlyUnloadedModifiers;

	Algo::CopyIf(CurrentModifiers, RecentlyUnloadedModifiers, [&LoadedModifiers](const MeshPartition::UModifierComponent* Modifier)
	{
		return (Modifier != nullptr) && !LoadedModifiers.Contains(Modifier);
	});

	// Nothing to do if nothing changed.
	if (RecentlyLoadedModifiers.IsEmpty() && RecentlyUnloadedModifiers.IsEmpty() && !bModifierListChanged)
	{
		return;
	}

	for (MeshPartition::UModifierComponent* LoadedModifier : RecentlyLoadedModifiers)
	{
		OnModifierLoaded(LoadedModifier);

		// Calling OnChanged here instead of just adding the bounds to our list has the benefit of updating
		//  its last submitted bounds.
		LoadedModifier->OnChanged(LoadedModifier->ComputeBounds(), MeshPartition::EChangeType::TransientChange);
	}

	for (MeshPartition::UModifierComponent* UnloadedModifier : RecentlyUnloadedModifiers)
	{
		// We can't do an OnChanged call like we do for the loaded ones above because we determined
		//  that the megamesh pointer on the modifier is not set to this megamesh.
		QueuePendingModifierChange(UnloadedModifier, UnloadedModifier->ComputeBounds(), MeshPartition::EChangeType::TransientChange);

		OnModifierUnloaded(UnloadedModifier);
	}

	CurrentModifiers = LoadedModifiers.Array();

	// No AMeshPartitionModifier is currently loaded, we need to remove all preview representation.
	if (CurrentModifiers.Num() == 0)
	{
		GEditor->GetSelectedActors()->DeselectAll(MeshPartition::APreviewSection::StaticClass());

		GroupRegistry.Reset();

		for (MeshPartition::APreviewSection* PreviewSection : PreviewSections)
		{
			World->DestroyActor(PreviewSection);
			PreviewSection = nullptr;
		}

		PreviewSections.Empty();

		return;
	}
}

void UMeshPartitionEditorComponent::OnSelectionChanged(UObject* InObject)
{
	if (USelection* Selection = Cast<USelection>(InObject))
	{
		TSet<MeshPartition::UModifierComponent*> SelectedModifiers;
		TArray<TWeakObjectPtr<UObject>> SelectedObjects;
		Selection->GetSelectedObjects(SelectedObjects);

		// Something is selected, we cannot see what in this call. Most of the time it's because some operations are calling this on actor and component in two calls.
		if (SelectedObjects.IsEmpty() && (Selection->Num() > 0))
		{
			return;
		}

		const AMeshPartition* MeshPartition = Cast<const AMeshPartition>(GetOwner());
		check(MeshPartition != nullptr);

		for (TWeakObjectPtr<UObject> SelectedObject : SelectedObjects)
		{
			MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(SelectedObject.Get());
			
			if ((Modifier != nullptr) && (Modifier->GetAffectedMeshPartition() == MeshPartition))
			{
				SelectedModifiers.Emplace(Modifier);
			}

			AActor* Actor = nullptr;

			if (UActorComponent* Component = Cast<UActorComponent>(SelectedObject.Get()))
			{
				Actor = Component->GetOwner();
			}
			else
			{
				Actor = Cast<AActor>(SelectedObject.Get());
			}
			
			if (Actor != nullptr)
			{
				TSet<UActorComponent*> Components = Actor->GetComponents();

				for (UActorComponent* Component : Components)
				{
					MeshPartition::UModifierComponent* ModifierComponent = Cast<MeshPartition::UModifierComponent>(Component);

					if ((ModifierComponent != nullptr) && (ModifierComponent->GetAffectedMeshPartition() == MeshPartition))
					{
						SelectedModifiers.Emplace(ModifierComponent);
					}
				}

				if (!SelectedModifiers.IsEmpty())
				{
					LastActorSelected = Actor;
				}
			}
		}

		SetInteractiveModifiers(SelectedModifiers.Array());
		SetPauseTransformerPipeline(!SelectedModifiers.IsEmpty());
	}
}

void UMeshPartitionEditorComponent::QueuePendingModifierChange(MeshPartition::UModifierComponent* InChangedModifier, TConstArrayView<FBox> InBoundingBoxes, MeshPartition::EChangeType InChangeType)
{
	TArray<MeshPartition::FModifierChangeInfo>& ModifierChanges = PendingChangedModifiers.FindOrAdd(InChangedModifier->GetPathName());
	ModifierChanges.Add({ *InChangedModifier, TArray<FBox>(InBoundingBoxes), InChangeType });
}

TArray<FBox> UMeshPartitionEditorComponent::GetIntersectingPreviewSections(TConstArrayView<FBox> InBounds, TArray<TObjectPtr<MeshPartition::APreviewSection>>& OutIntersectingPreviewSections)
{
	TArray<FBox> UpdatedBounds(InBounds);
	TSet<MeshPartition::APreviewSection*> UniquePreviewSections;

	UpdatedBounds.RemoveAll([](const FBox& BoundsToTest) { return BoundsToTest.GetSize().Length() == 0; });
	
	// Cleanup any stale resource if any
	PreviewSections.RemoveAll([](const MeshPartition::APreviewSection* PreviewSection) { return PreviewSection == nullptr; });
	
	Algo::CopyIf(PreviewSections, UniquePreviewSections, [InBounds, &UpdatedBounds](const MeshPartition::APreviewSection* PreviewSection)
	{
		const FBox PreviewSectionBounds = PreviewSection->GetPreviewMeshBounds();

		for (const FBox& BoundsToCheck : InBounds)
		{
			if (PreviewSectionBounds.Intersect(BoundsToCheck))
			{
				UpdatedBounds.Emplace(PreviewSectionBounds);
				return true;
			}
		}
	 	
		return false;
	});

	OutIntersectingPreviewSections = UniquePreviewSections.Array();

	return UpdatedBounds;
}

void UMeshPartitionEditorComponent::DestroyPreviewSections(TArray<TObjectPtr<MeshPartition::APreviewSection>>& InSectionsToDestroy)
{
	UWorld* World = GetWorld();
	
	// Safety measure in case some handle still exist on actors we are about to destroy.
	GEditor->GetSelectedActors()->DeselectAll(MeshPartition::APreviewSection::StaticClass());
	
	TArray<UObject*> ObjectsToCancel;
	for (MeshPartition::APreviewSection* PreviewSection : InSectionsToDestroy)
	{
		if (PreviewSection == nullptr)
		{
			continue;
		}

		for (TWeakObjectPtr<MeshPartition::UModifierComponent> BaseModifier : PreviewSection->GetBaseModifiers())
		{
			if (!BaseModifier.IsValid())
			{
				continue;
			}

			BaseModifier->SetPreviewSection(nullptr);
		}
		
		for (UStaticMesh* StaticMesh : PreviewSection->GetMeshes())
		{
			ObjectsToCancel.Add(StaticMesh);
		}

		GroupRegistry.RemoveGroup(PreviewSection->GetGroupRegistryKey());
		PreviewSections.Remove(PreviewSection);
		InteractivePreviewSections.Remove(PreviewSection);

		bool bIsPreviewSectionMeshBeingBuilt = false;

		for (const TUniquePtr<MeshPartition::FTransformerContext>& TransformerContext : PreviewSectionTransformerContexts)
		{
			for (MeshPartition::FTransformerUnit& TransformerUnit : TransformerContext->TransformerUnits)
			{
				if ((PreviewSection != nullptr) && (TransformerUnit.Section.Get() == PreviewSection))
				{
					// The preview section should be destroyed but we cannot at the moment. Just hiding it to avoid confusion later.
					PreviewSection->SetIsTemporarilyHiddenInEditor(true);
					PreviewSectionsToDestroy.Emplace(PreviewSection);

					TransformerContext->bWasCancelled = true;
					bIsPreviewSectionMeshBeingBuilt = true;
					break;
				}
			}
		}

		if (!bIsPreviewSectionMeshBeingBuilt)
		{
			if (UMeshPartitionEditorSubsystem* Subsystem = UMeshPartitionEditorSubsystem::Get())
			{
				Subsystem->TrackGarbagePreviewSection(PreviewSection);
			}

			constexpr bool bNetForce = false;
			constexpr bool bShouldModifyLevel = false;
			World->DestroyActor(PreviewSection, bNetForce, bShouldModifyLevel);
			
			PreviewSection = nullptr;
		}
	}

	if (ObjectsToCancel.Num())
	{
		// Try to reduce CPU waste by sending a cancelation event to ongoing compilations.
		// This is fast and non-blocking.
		FAssetCompilingManager::Get().MarkCompilationAsCanceled(ObjectsToCancel);
	}
}

void UMeshPartitionEditorComponent::UpdateLinks(const UE::MeshPartition::FModifierGroup& InModifierGroup, MeshPartition::APreviewSection* InPreviewSection)
{
	if (InPreviewSection == nullptr)
	{
		return;
	}

	InPreviewSection->SetGroupRegistryKey(MeshPartition::FModifierGroupRegistry::GetGroupRegistryKey(InModifierGroup));

	InModifierGroup.ForAllModifiers([InPreviewSection](MeshPartition::UModifierComponent* Modifier)
	{
		if (Modifier->IsBase())
		{
			Modifier->SetPreviewSection(InPreviewSection);
			InPreviewSection->AddBaseModifier(Modifier);
		}

		InPreviewSection->AddModifier(Modifier);

		return true;
	});
}

void UMeshPartitionEditorComponent::GetModifiersToProcessForPreviews(TSet<MeshPartition::UModifierComponent*>& 		OutModifiers,
																	 const TArray<MeshPartition::APreviewSection*>& InPreviewSections,
																	 TConstArrayView<FBox> 							InBoundsToBuild) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::GetModifiersToProcessForPreviews);

	UWorld* World = GetWorld();
	const AMeshPartition* MeshPartition = Cast<const AMeshPartition>(GetOwner());

	check(MeshPartition != nullptr);

	Algo::CopyIf(CurrentModifiers, OutModifiers, [MeshPartition, InBoundsToBuild](const MeshPartition::UModifierComponent* Modifier)
	{
		return (Modifier != nullptr)
			&& (Modifier->GetAffectedMeshPartition() == MeshPartition)
			&& !Modifier->GetIsDisabledFlag()	 				// exclude modifiers which are disabled via persistent flag
			&& Modifier->IsRegistered()
			&& (!Modifier->IsBase() || Modifier->IsFree())
			&& Modifier->IntersectsAnyBounds(InBoundsToBuild);
	});

	for (const MeshPartition::APreviewSection* PreviewSection : InPreviewSections)
	{
		if (PreviewSection == nullptr)
		{
			continue;
		}

		Algo::TransformIf(PreviewSection->GetBaseModifiers(), OutModifiers, [](TWeakObjectPtr<MeshPartition::UModifierComponent> BaseModifier)
		{
			return BaseModifier.IsValid() && BaseModifier->IsRegistered();
		},
		[](TWeakObjectPtr<MeshPartition::UModifierComponent> BaseModifier)
		{
			return BaseModifier.Get();
		});
	}
}

void UMeshPartitionEditorComponent::InvalidateConflictingPreviewSectionBuildContexts(TSet<MeshPartition::UModifierComponent*>& 		InOutModifiers,
																					 const TArray<MeshPartition::APreviewSection*>& InPreviewSections,
																					 const FPreviewSectionBuildContext& 			InCurrentBuildContext)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::InvalidateConflictingPreviewSectionBuildContexts);

	// Iterate through pending build contexts.
	for (FPreviewSectionBuildContext& PendingPreviewSectionBuildContext : PreviewSectionBuildContexts)
	{
		if (&PendingPreviewSectionBuildContext == &InCurrentBuildContext)
		{
			continue;
		}

		// Removing intersecting preview sections from other build contexts so the responsibility for cleanup is now for this build context.
		PendingPreviewSectionBuildContext.IntersectingPreviewSections.RemoveAll([&InPreviewSections] (MeshPartition::APreviewSection* IntersectingPreviewSection)
		{
			return (InPreviewSections.Find(IntersectingPreviewSection) != INDEX_NONE);
		});

		// Marks common bases so other build contexts know they shouldn't commit their result.
		// Also grabs all the modifiers of the invalidated section, which might extend outside the currently built previews.
		for (MeshPartition::FBuildTaskHandle& TaskHandle : PendingPreviewSectionBuildContext.BuildTasks)
		{
			const UE::MeshPartition::FModifierGroup& TaskGroup = TaskHandle.GetTask()->GetGroup();
			bool bHasTaskBeenCancelled = false;

			TaskGroup.ForEachBase([&TaskHandle, &InOutModifiers, &bHasTaskBeenCancelled](MeshPartition::UModifierComponent* BaseModifier)
			{
				if (ensure(BaseModifier != nullptr) && (InOutModifiers.Find(BaseModifier) != nullptr))
				{
					TaskHandle.Cancel();
					bHasTaskBeenCancelled = true;
					return false;
				}

				return true;
			});

			if (bHasTaskBeenCancelled)
			{
				GroupRegistry.RevertStagedGroup(MeshPartition::FModifierGroupRegistry::GetGroupRegistryKey(TaskGroup));

				TaskGroup.ForAllModifiers([&InOutModifiers, &PendingPreviewSectionBuildContext](MeshPartition::UModifierComponent* Modifier)
				{
					if (ensure(Modifier != nullptr))
					{
						// Don't keep any modifiers that have been destroyed (for instance due to a construction script rerun). The
						//  registration check is also necessary because when undoing modifier placement, the modifier is not actually
						//  killed, so the pointer would be valid and we would end up processing a modifier that is not actually in the
						// world.
						if (Modifier->IsRegistered())
						{
							InOutModifiers.Add(Modifier);
						}

						if (Modifier->IsBase())
						{
							PendingPreviewSectionBuildContext.OutdatedBases.Add(Modifier);
						}
					}

					return true;
				}, true /*bSkipInvalidModifiers; ok for null modifiers to be in the list here*/);
			}
		}
	}
}

void UMeshPartitionEditorComponent::GetModifiersWithinBounds(TArray<MeshPartition::UModifierComponent*>& OutModifiers, TConstArrayView<const FBox> InBoundsToBuild) const
{
	UWorld* World = GetWorld();
	const AMeshPartition* MeshPartition = Cast<const AMeshPartition>(GetOwner());

	check(MeshPartition != nullptr);

	Algo::CopyIf(CurrentModifiers, OutModifiers, [MeshPartition, InBoundsToBuild](const MeshPartition::UModifierComponent* Modifier)
	{
		return (Modifier != nullptr) && (Modifier->GetAffectedMeshPartition() == MeshPartition) && Modifier->IsRegistered() && Modifier->IntersectsAnyBounds(InBoundsToBuild);
	});
}

void UMeshPartitionEditorComponent::GetModifiersAffectingBounds(TArray<MeshPartition::UModifierComponent*>& OutModifiers, TConstArrayView<const FBox> InBoundsToBuild) const
{
	TArray<MeshPartition::FModifierGroup> Groups = GroupRegistry.FindGroupsIntersectingBounds(InBoundsToBuild);

	for (const MeshPartition::FModifierGroup& Group : Groups)
	{
		Group.ForAllModifiers([&OutModifiers](MeshPartition::UModifierComponent* Modifier)
		{
			if (ensure(Modifier != nullptr))
			{
				// Multiple modifier groups may contain the same modifier
				OutModifiers.AddUnique(Modifier);
			}

			return true;
		});
	}
}

void UMeshPartitionEditorComponent::GetModifiersAffectingModifiers(TArray<MeshPartition::UModifierComponent*>& OutModifiers, const TArray<MeshPartition::UModifierComponent*>& InModifiers) const
{
	TArray<MeshPartition::FModifierDesc> ModifierDescs;
	Algo::Transform(InModifiers, ModifierDescs, [](const MeshPartition::UModifierComponent* Modifier) { return MeshPartition::FModifierDesc(*Modifier); });

	TArray<MeshPartition::FModifierGroup> Groups = GroupRegistry.FindGroupsContainingModifiers(ModifierDescs);

	for (const MeshPartition::FModifierGroup& Group : Groups)
	{
		Group.ForAllModifiers([&OutModifiers](MeshPartition::UModifierComponent* Modifier)
		{
			if (ensure(Modifier != nullptr))
			{
				// Multiple modifier groups may contain the same modifier
				OutModifiers.AddUnique(Modifier);
			}

			return true;
		});
	}
}

const MeshPartition::FCommonBuildVariant& UMeshPartitionEditorComponent::GetPreviewBuildVariant() const 
{
	static MeshPartition::FCommonBuildVariant DefaultPreviewBuildVariant {};

	return GetMegaMeshDefinition() ? GetMegaMeshDefinition()->GetPreviewSectionBuildVariant() : DefaultPreviewBuildVariant;
}

void UMeshPartitionEditorComponent::FinalizeAsyncTasks(const bool bInCompleteAllTasks)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorComponent::FinalizeAsyncTasks);

	UWorld* World = GetWorld();

	const MeshPartition::FCommonBuildVariant& PreviewBuildVariant = GetPreviewBuildVariant();

	for (auto Iterator = PreviewSectionTransformerContexts.CreateIterator(); Iterator; ++Iterator)
	{
		const TUniquePtr<MeshPartition::FTransformerContext>& PreviewSectionTransformerContext = *Iterator;
		
		if (PreviewSectionTransformerContext->JoinTask.IsValid() && (!PreviewSectionTransformerContext->JoinTask.IsCompleted()))
		{
			continue;
		}

		TSet<MeshPartition::APreviewSection*> ProcessedPreviewSections;

		for (const MeshPartition::FTransformerUnit& TransformerUnit : PreviewSectionTransformerContext->TransformerUnits)
		{
			MeshPartition::APreviewSection* PreviewSection = Cast<MeshPartition::APreviewSection>(TransformerUnit.Section.Get());

			if (PreviewSection == nullptr)
			{
				continue;
			}

			ProcessedPreviewSections.Emplace(PreviewSection);
		}

		for (auto ProcessedPreviewSectionIterator = ProcessedPreviewSections.CreateIterator(); ProcessedPreviewSectionIterator; ++ProcessedPreviewSectionIterator)
		{
			MeshPartition::APreviewSection* ProcessedPreviewSection = *ProcessedPreviewSectionIterator;

			if (PreviewSectionsToDestroy.Contains(ProcessedPreviewSection))
			{
				constexpr bool bNetForce = false;
				constexpr bool bShouldModifyLevel = false;

				World->DestroyActor(ProcessedPreviewSection, bNetForce, bShouldModifyLevel);
				PreviewSectionsToDestroy.Remove(ProcessedPreviewSection);
				ProcessedPreviewSectionIterator.RemoveCurrent();
			}
		}
		
		// Cancelled contexts may reference modifiers that have already been destroyed -- skip post-processing.
		if (!PreviewSectionTransformerContext->bWasCancelled)
		{
			for (MeshPartition::APreviewSection* PreviewSection : ProcessedPreviewSections)
			{
				PostProcessSection(PreviewSection, PreviewSection->GetModifiers());
			}
		}

		Iterator.RemoveCurrentSwap();
		
		if (!bInCompleteAllTasks)
		{
			break;
		}
	}
}

void UMeshPartitionEditorComponent::FinalizePreviewSectionBuilds()
{
	UWorld* World = GetWorld();

	if (!ensure(World != nullptr))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get World.");
		return;
	}

	// We're about to delete/create preview sections, and we're doing this in a tick, potentially while
	//  we have a transaction open due to dragging something in the editor. This would cause us to transact
	//  this creation/deletion, which we don't want to do. So make sure we don't by temporarily swapping
	//  out the GUndo pointer.
	ITransaction* UndoState = GUndo;
	GUndo = nullptr; // Pretend we're not in a transaction
	ON_SCOPE_EXIT{ GUndo = UndoState; }; // Revert later

	FBox InteractiveSectionBounds;
	bool bIsInteractiveSectionActive = false;

	if (IsInteractiveModeReady())
	{
		constexpr bool bNonColliding = true;
		constexpr bool bIncludeFromChildActors = false;

		InteractiveSectionBounds = InteractiveSection->GetComponentsBoundingBox(bNonColliding, bIncludeFromChildActors);

		bIsInteractiveSectionActive = true;
	}

	for (auto Iterator = PreviewSectionBuildContexts.CreateIterator(); Iterator; ++Iterator)
	{
		FPreviewSectionBuildContext& PreviewSectionBuildContext = *Iterator;

		if (!UE::MeshPartition::Build::AreAllTasksComplete(PreviewSectionBuildContext.BuildTasks))
		{
			continue;
		}

		TRACE_CPUPROFILER_EVENT_MANUAL_START(TEXT("UMeshPartitionEditorComponent::CompletePreviewSectionBuilds_Iteration"))

		DestroyPreviewSections(PreviewSectionBuildContext.IntersectingPreviewSections);

		for (MeshPartition::FBuildTaskHandle& TaskHandle : PreviewSectionBuildContext.BuildTasks)
		{
			const UE::MeshPartition::FModifierGroup& ModifierGroup = TaskHandle.GetTask()->GetGroup();

			if (TaskHandle.IsCancelled())
			{
				bool bAllBaseFound = true;

				ModifierGroup.ForEachBase([&PreviewSectionBuildContext, &bAllBaseFound](MeshPartition::UModifierComponent* Modifier)
				{
					if (ensure(Modifier != nullptr))
					{
						const bool bBaseFoundInOutdatedBases = (PreviewSectionBuildContext.OutdatedBases.Find(Modifier) != nullptr);
						bAllBaseFound &= bBaseFoundInOutdatedBases;
					}

					return true;
				});

				ensureMsgf(bAllBaseFound, TEXT("UMeshPartitionEditorComponent: some bases were not taken into account when invalidating a pending preview section build. This will most likely result in a corrupt preview section."));
				continue;
			}

			TSharedPtr<const MeshPartition::FMeshData> ProcessedMesh = TaskHandle.GetTask()->GetMesh();

			MeshPartition::APreviewSection* PreviewSection = SpawnTransientActor<MeshPartition::APreviewSection>(TaskHandle.GetTask()->GetTransform());

			TSharedPtr<const MeshPartition::FSectionChannels> SectionChannel = TaskHandle.GetTask()->GetSectionChannels();

			PreviewSection->SetParent(Cast<AMeshPartition>(GetOwner()));
			const MeshPartition::FCommonBuildVariant& PreviewSectionBuildVariant = GetPreviewBuildVariant();

			UMaterialInstanceDynamic* PreviewSectionMID = MegaMeshEditorComponentLocals::CreateMaterialInstance(GetDefinitionMaterial(), PreviewSection,
				MakeUniqueObjectName(PreviewSection, UMaterialInstanceDynamic::StaticClass(), *FString::Printf(TEXT("PreviewSectionMID"))));
			PreviewSection->SetMaterialInstance(PreviewSectionMID);

			PreviewSection->SetPreviewMesh(ProcessedMesh.ToSharedRef(), UCollisionProfile::BlockAll_ProfileName, /* bCanEverAffectNavigation = */ false);
			PreviewSection->SetBuildPerfStats(TaskHandle.GetTask()->GetBuildPerfStats());

			if (SectionChannel.IsValid())
			{
				PreviewSection->SetChannelTexture(SectionChannel->Texture.Get());
				PreviewSection->SetChannelData(SectionChannel->Table, SectionChannel->TexcoordMetrics);

				if (SectionChannel->Mesh.IsSet())
				{
					PreviewSection->SetChannelGenerationMesh(SectionChannel->Mesh.GetValue());
				}

				if (UMeshPartitionDefinition* Definition = GetMegaMeshDefinition())
				{
					if (SectionChannel->Domain.AreaUV > 0)
					{
						PreviewSection->SetMaterialCacheTileCount(GetMaterialCacheTileCount(
							SectionChannel->Domain.Area3D / SectionChannel->Domain.AreaUV,
							Definition->GetMaterialCacheTexelSize()
						));

						PreviewSection->RecreateMaterialCacheTextures();
					}
				}
			}

			PreviewSections.Emplace(PreviewSection);

			GroupRegistry.CommitStagedGroup(MeshPartition::FModifierGroupRegistry::GetGroupRegistryKey(ModifierGroup));
			UpdateLinks(ModifierGroup, PreviewSection);

			TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> ModifierPtrs = ModifierGroup.AllResolvedModifierPtrs();

			PostBuildSectionMesh(PreviewSection, *PreviewSection->GetPreviewMesh(), ModifierPtrs);

			PreviewSection->SetIsTransformerPipelinePaused(bIsTransformerPipelinePaused);

			ExecuteTransformers(PreviewSection->GetPreviewMeshComponent(), GetMegaMeshDefinition(), GetPreviewBuildVariant());

			if (!bIsTransformerPipelinePaused)
			{
				TArray<MeshPartition::FTransformerUnit> InitialUnits;
				InitialUnits.Add(MeshPartition::MakeTransformerUnit(PreviewSection, ProcessedMesh, /*bRecomputeNormals=*/ false, ShouldRecomputeTangents()));

				TUniquePtr<MeshPartition::FTransformerContext> TransformerContext = LaunchTransformers(MoveTemp(InitialUnits), GetMegaMeshDefinition(), GetPreviewBuildVariant());

				if (TransformerContext.IsValid())
				{
					PreviewSectionTransformerContexts.Emplace(MoveTemp(TransformerContext));
				}
			}

			bool bIsInteractive = false;

			if (bIsInteractiveSectionActive)
			{
				constexpr bool bNonColliding = true;
				constexpr bool bIncludeFromChildActors = false;

				FBox PreviewSectionBounds = PreviewSection->GetComponentsBoundingBox(bNonColliding, bIncludeFromChildActors);

				if (PreviewSectionBounds.Intersect(InteractiveSectionBounds))
				{
					bIsInteractive = true;
					InteractivePreviewSections.Emplace(PreviewSection);
				}
			}

			PreviewSection->SetIsTemporarilyHiddenInEditor(!ArePreviewSectionsVisible() || bIsInteractive);
			const TOptional<MeshPartition::EBuildType>& BuildType = ModifierGroup.GetBuildType();

			if (BuildType.IsSet() && BuildType.GetValue() == MeshPartition::EBuildType::SimplifiedPreviewSection)
			{
				PreviewSection->Tags.AddUnique(MegaMeshEditorComponentLocals::SimplifiedPreviewSectionTag);
			}
		}

		Iterator.RemoveCurrentSwap();

		UMeshPartitionEditorSubsystem* Subsystem = UMeshPartitionEditorSubsystem::Get();

		if (ensure(Subsystem))
		{
			Subsystem->SetPreviewSectionBuildNumber(this, GetPreviewSectionBuildNumber());
		}

		TRACE_CPUPROFILER_EVENT_MANUAL_END()
	}
}

uint32 UMeshPartitionEditorComponent::GetPreviewSectionBuildNumber() const
{
	return Algo::Accumulate(PreviewSectionBuildContexts, 0, [](const uint32 Total, const FPreviewSectionBuildContext& PendingPreviewSectionBuildContext)
	{
		return (Total + PendingPreviewSectionBuildContext.BuildTasks.Num());
	});
}

void UMeshPartitionEditorComponent::SetPauseTransformerPipeline(const bool bInShouldPauseTransformerPipeline)
{
	if (bIsTransformerPipelinePaused == bInShouldPauseTransformerPipeline)
	{
		return;
	}

	const FName MegaMeshMessageProvider = TEXT("MegaMesh");
	UWorld* World = GetWorld();
	UActionableMessageSubsystem* ActionableMessageSubsystem = (World == nullptr) ? nullptr : World->GetSubsystem<UActionableMessageSubsystem>();

	if (bInShouldPauseTransformerPipeline && (CVarPauseTransformerPipeline.GetValueOnGameThread()))
	{
		bIsTransformerPipelinePaused = true;

		if ((ActionableMessageSubsystem != nullptr) && GIsEditor)
		{
			FActionableMessage ActionableMessage;
		
			ActionableMessage.bForceExpand = true;
			ActionableMessage.Message = LOCTEXT("MeshPartitionTransformerPipelinePaused.Message", "MeshPartition: the transformer pipeline is temporarily disabled");
			ActionableMessage.Tooltip = LOCTEXT("MeshPartitionTransformerPipelinePaused.Tooltip",
												"The transformer pipeline is temporarily disabled; features like creating nanite static mesh are temporarily paused to allow faster iteration times at the cost of some rendering performance.\n"
												"While paused, nanite tessellation may not be visible on the surface of the mesh.\n"
												"To unpause, unselect everything or select any actor not related to the MeshPartition system.");
		
			ActionableMessageSubsystem->SetActionableMessage(MegaMeshMessageProvider, ActionableMessage);
		}
	}
	else
	{
		bIsTransformerPipelinePaused = false;

		if ((ActionableMessageSubsystem != nullptr) && GIsEditor)
		{
			ActionableMessageSubsystem->ClearActionableMessage(MegaMeshMessageProvider);
		}

		ForAllPreviewSections([this](MeshPartition::APreviewSection* PreviewSection)
		{
			if (PreviewSection->IsTransformerPipelinedPaused())
			{
				const bool bIsPaused = false;

				TArray<MeshPartition::FTransformerUnit> InitialUnits;
				InitialUnits.Add(MeshPartition::MakeTransformerUnit(PreviewSection, PreviewSection->GetPreviewMesh(), /*bRecomputeNormals=*/ false, ShouldRecomputeTangents()));

				TUniquePtr<MeshPartition::FTransformerContext> TransformerContext = LaunchTransformers(MoveTemp(InitialUnits), GetMegaMeshDefinition(), GetPreviewBuildVariant());

				if (TransformerContext.IsValid())
				{
					PreviewSectionTransformerContexts.Emplace(MoveTemp(TransformerContext));
				}

				PreviewSection->SetIsTransformerPipelinePaused(bIsPaused);
			}

			return true;
		});
	}
}

bool UMeshPartitionEditorComponent::IsModifierParticipatingInActivePreviewSectionBuild(const MeshPartition::UModifierComponent* InModifier) const
{
	for (const FPreviewSectionBuildContext& PendingPreviewBuild : PreviewSectionBuildContexts)
	{
		for (const MeshPartition::FBuildTaskHandle& BuildTask : PendingPreviewBuild.BuildTasks)
		{
			if (const TSharedPtr<MeshPartition::FBuildTask>& Task = BuildTask.GetTask())
			{
				const bool bFound = Task->GetGroup().AllModifierDescs().ContainsByPredicate([InModifierPath = FSoftObjectPath(InModifier)](const MeshPartition::FModifierDesc& Desc)
					{
						return Desc.ModifierPath == InModifierPath;
					});

				if (bFound)
				{
					return true;
				}
			}
		}
	}

	if (PendingChangedModifiers.Contains(InModifier))
	{
		return true;
	}

	return false;
}

bool UMeshPartitionEditorComponent::IsAnyPreviewSectionBuildActive() const
{
	return PreviewSectionBuildContexts.Num() != 0;
}

void UMeshPartitionEditorComponent::ResetGroupRegistry(TConstArrayView<const MeshPartition::FModifierGroup> InModifierGroups)
{
	GroupRegistry.Reset(InModifierGroups);
}

void UMeshPartitionEditorComponent::ResetGroupRegistry()
{
	GroupRegistry.Reset();
}

const MeshPartition::FModifierGroup* UMeshPartitionEditorComponent::FindGroupInRegistry(const FGuid& InGroupKey) const
{
	return GroupRegistry.FindGroup(InGroupKey);
}

bool UMeshPartitionEditorComponent::ShouldRecomputeTangents() const
{
	return bIsTransformerPipelinePaused && CVarRecomputeTangents.GetValueOnAnyThread();
}

void UMeshPartitionEditorComponent::ResetInteractiveSection()
{
	if (InteractiveSection == nullptr)
	{
		return;
	}

	InteractiveSection->ClearInteractiveModifiers();

	for (MeshPartition::APreviewSection* InteractivePreviewSection : InteractivePreviewSections)
	{
		if (InteractiveSection == nullptr)
		{
			continue;
		}

		InteractivePreviewSection->SetIsTemporarilyHiddenInEditor(!ArePreviewSectionsVisible());
	}

	InteractivePreviewSections.Empty();
}

void UMeshPartitionEditorComponent::SetInteractiveModifiers(const TArray<MeshPartition::UModifierComponent*>& InModifiers)
{
	if (!IsInteractiveModeEnabled() || (InteractiveSection == nullptr))
	{
		return;
	}

	const bool bAreAllModifierBases = Algo::AllOf(InModifiers, [](MeshPartition::UModifierComponent* Modifier) { return Modifier->IsBase(); });

	if (InModifiers.IsEmpty() || bAreAllModifierBases)
	{
		const TArray<TObjectPtr<MeshPartition::UModifierComponent>> PreviousInteractiveModifiers = InteractiveSection->GetInteractiveModifiers();
		ResetInteractiveSection();

		for (MeshPartition::UModifierComponent* PreviousInteractiveModifier : PreviousInteractiveModifiers)
		{
			if (PreviousInteractiveModifier != nullptr)
			{
				PreviousInteractiveModifier->OnChanged(PreviousInteractiveModifier->ComputeBounds(), MeshPartition::EChangeType::TransientStateChange);
			}
		}
	}
	else if (!InModifiers.IsEmpty())
	{
		TArray<MeshPartition::UModifierComponent*> SortedModifiers = InModifiers;
		TArray<MeshPartition::UModifierComponent*> CurrentInteractiveModifiers = InteractiveSection->GetInteractiveModifiers();

		SortedModifiers.Sort();
		CurrentInteractiveModifiers.Sort();

		if (SortedModifiers != CurrentInteractiveModifiers)
		{
			ResetInteractiveSection();
			const AMeshPartition* MegaMesh = GetOwner<const AMeshPartition>();
			TArray<FBox> AggregatedBounds;

			for (MeshPartition::UModifierComponent* Modifier : InModifiers)
			{
				if (!Modifier->IsBase())
				{
					AggregatedBounds.Append(Modifier->ComputeBounds());
				}
			}

			TArray<FBox> BoundsToBuild = GetIntersectingPreviewSections(AggregatedBounds, InteractivePreviewSections);

			MeshPartition::FBuilderSettings Settings;
			Settings.BuildType = MeshPartition::EBuildType::InteractiveBase;
			Settings.Transform = MegaMesh->GetTransform();
			Settings.TypePriorities = GetMegaMeshDefinition() ? GetMegaMeshDefinition()->GetModifierTypePriorities() : TArray<FName>{};
			Settings.MaxSectionComplexity = FMathd::MaxReal;
			Settings.bRecomputeNormals = true;
			Settings.bRecomputeTangents = ShouldRecomputeTangents();
			Settings.bCacheResult = true;
			Settings.bBuildSpatial = false;
			Settings.bAllowDDCRead = false;
			Settings.bAllowDDCWrite = false;
			Settings.TexcoordGenerationOptions = FChannelCollectionUVLayoutOptions::GetFromDefinition(GetMegaMeshDefinition());
			Settings.TexcoordGenerationOptions->UVQuality = EChannelUVUnwrapQuality::Preview;

			constexpr bool bInclusive = false;
			Settings.ModifierFilter = MeshPartition::AInteractiveSection::InteractiveSectionBaseFilter(InModifiers, Settings.TypePriorities);

			TSet<MeshPartition::UModifierComponent*> ModifiersToProcess;
			GetModifiersToProcessForPreviews(ModifiersToProcess, InteractivePreviewSections, BoundsToBuild);

			Settings.ModifiersToProcess = ModifiersToProcess.Array();

			const bool bContainsBases = Algo::AnyOf(Settings.ModifiersToProcess, [](MeshPartition::UModifierComponent* Modifier) { return Modifier->IsBase(); });
	
			// If this build doesn't contain any base, we won't have anything to apply modifier onto.
			if (bContainsBases)
			{
				InteractiveSection->SetInteractiveModifiers(InModifiers, MoveTemp(Settings));

				for (MeshPartition::APreviewSection* InteractivePreviewSection : InteractivePreviewSections)
				{
					InteractivePreviewSection->SetIsTemporarilyHiddenInEditor(true);
				}
			}
		}
	}
}

bool UMeshPartitionEditorComponent::IsInteractiveModeEnabled() const
{
	return CVarInteractiveModeEnabled.GetValueOnGameThread();
}

bool UMeshPartitionEditorComponent::IsInteractiveModeReady() const
{
	return IsInteractiveModeEnabled() && (InteractiveSection != nullptr) && (InteractiveSection->IsInteractiveSectionActive());
}

void UMeshPartitionEditorComponent::OnPreviewSectionSimplificationEnabledChanged(IConsoleVariable* InConsoleVariable)
{
	if (InConsoleVariable == nullptr)
	{
		return;
	}

	const bool bIsSimplificationEnabled = InConsoleVariable->GetBool();

	if (!bIsSimplificationEnabled)
	{
		ForAllPreviewSections([this](MeshPartition::APreviewSection* PreviewSection)
		{
			if ((PreviewSection != nullptr) && PreviewSection->Tags.Contains(MegaMeshEditorComponentLocals::SimplifiedPreviewSectionTag))
			{
				OnBoundsChanged( { PreviewSection->GetPreviewMeshBounds() }, MeshPartition::EChangeType::StateChange);
			}
			
			return true;
		});
	}
}

void UMeshPartitionEditorComponent::OnDefinitionModified(const FName& InMemberName, const FName& InPropertyName)
{
	const MeshPartition::UMeshPartitionDefinition* Definition = GetMegaMeshDefinition();

	if (MeshPartition::UMeshPartitionDefinition::IsMaterialPropertyName(InMemberName))
	{
		UpdateMaterial();
	}

	ForAllCurrentModifiers([Definition, &InPropertyName](MeshPartition::UModifierComponent* Modifier)
	{
		Modifier->OnMegaMeshDefinitionModified(Definition, InPropertyName);
		return true;
	});

	
	ForAllPreviewSections([Definition, &InPropertyName](MeshPartition::APreviewSection* PreviewSection)
	{
		PreviewSection->OnMeshPartitionDefinitionModified(Definition, InPropertyName);
		return true;
	});

	if (MeshPartition::UMeshPartitionDefinition::DoesPropertyNameRequireRebuild(InMemberName, InPropertyName))
	{
		ForceRebuildAllSections(MeshPartition::EChangeType::StateChange);
	}
}


UMaterialInterface* UMeshPartitionEditorComponent::GetDefinitionMaterial() const
{
	MeshPartition::UMeshPartitionDefinition* Definition = GetMegaMeshDefinition();
	if (Definition)
	{
		return Definition->GetMaterial();
	}

	return nullptr;
}
} // namespace UE::MeshPartition


#undef LOCTEXT_NAMESPACE
