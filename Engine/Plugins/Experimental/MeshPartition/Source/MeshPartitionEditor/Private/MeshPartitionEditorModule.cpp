// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorModule.h"

#include "Editor.h"
#include "Editor/UnrealEdEngine.h" // UUnrealEdEngine
#include "EditorBuildUtils.h" // FEditorBuildUtils
#include "MeshPartitionPreviewComponents.h" // UDynamicMeshPreviewComponent
#include "MeshPartitionPreviewVisualizers.h" // FMeshPreviewVisualizer
#include "Modifiers/MeshPartitionModifierVisualizer.h" // FMegaMeshModifierVisualizer
#include "UnrealEdGlobals.h" // GUnrealEd
#include "WorldPartitionMeshPartitionBuilder.h" // UWorldPartitionMeshPartitionBuilder
#include "LevelEditor.h" // FLevelEditorModule
#include "LevelEditorOutlinerSettings.h" // FLevelEditorOutlinerSettings
#include "MeshPartitionOutlinerFilter.h" // FMeshPartitionOutlinerFilter
#include "EngineUtils.h" // TActorRange
#include "WorldPartition/WorldPartitionMiniMapBuilder.h"
#include "FoliageModule.h"

#define LOCTEXT_NAMESPACE "FMeshPartitionEditorModule"

DEFINE_LOG_CATEGORY(LogMegaMeshEditor);

namespace UE::MeshPartition
{
static const FName EditorBuildType(TEXT("BuildMeshPartitionReuseByPackageHash"));
static const FName EditorBuildTypeForce(TEXT("BuildMeshPartitionForceRebuild"));

void FMeshPartitionEditorModule::StartupModule()
{
	const FText SectionLabel = LOCTEXT("MeshPartition", "Mesh Partition");

	FEditorBuildUtils::RegisterCustomBuildType(EditorBuildType,
											FCanDoEditorBuildDelegate::CreateStatic(&UWorldPartitionMeshPartitionBuilder::CanBuildMeshPartitions),
											FDoEditorBuildDelegate::CreateLambda(
												[](UWorld* InWorld, FName InBuildOption)
												{
													return UWorldPartitionMeshPartitionBuilder::BuildMeshPartitions(InWorld, EMeshPartitionReuseMode::ByPackageHash);
												}),
											/*BuildAllExtensionPoint*/NAME_None,
											/*MenuEntryLabel*/LOCTEXT("BuildMeshPartition", "Build Mesh Partition"),
											/*MenuSectionLabel*/SectionLabel,
											/*bExternalProcess=*/true);

	FEditorBuildUtils::RegisterCustomBuildType(EditorBuildTypeForce,
												FCanDoEditorBuildDelegate::CreateStatic(&UWorldPartitionMeshPartitionBuilder::CanBuildMeshPartitions),
												FDoEditorBuildDelegate::CreateLambda(
												[](UWorld* InWorld, FName InBuildOption)
												{
													return UWorldPartitionMeshPartitionBuilder::BuildMeshPartitions(InWorld, EMeshPartitionReuseMode::ForceRebuild);
												}),
												/*BuildAllExtensionPoint*/NAME_None,
												/*MenuEntryLabel*/LOCTEXT("BuildMeshPartitionForce", "Build Mesh Partition (force rebuild)"),
												/*MenuSectionLabel*/SectionLabel,
												/*bExternalProcess=*/true);

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMeshPartitionEditorModule::OnPostEngineInit);
	UWorldPartitionMiniMapBuilder::OnWarmupTick.AddRaw(this, &FMeshPartitionEditorModule::OnMinimapWarmupTick);

	// Try to register the foliage base id class ignore list if the foliage module is already loaded.
	if (IFoliageEditModuleBase* FoliageModule = IFoliageEditModuleBase::Get())
	{
		RegisterFoliageBaseIDClassIgnoreList();
	}
	// If the foliage module is not yet loaded, we need to listen for the module load callback so we
	// can register to it once it becomes loaded.
	else
	{
		FModuleManager::Get().OnModulesChanged().AddRaw(this, &FMeshPartitionEditorModule::ModulesChangedCallback);
	}
}

void FMeshPartitionEditorModule::ShutdownModule()
{
	if (IFoliageEditModuleBase* FoliageModule = IFoliageEditModuleBase::Get())
	{
		FoliageModule->UnregisterComponentBaseIDClassToIgnore(MeshPartition::UPreviewMeshComponent::StaticClass());
		FoliageModule->UnregisterComponentBaseIDClassToIgnore(MeshPartition::UStaticMeshPreviewComponent::StaticClass());
		FoliageModule->UnregisterComponentBaseIDClassToIgnore(MeshPartition::UMeshPartitionCollisionComponent::StaticClass());
	}

	FModuleManager::Get().OnModulesChanged().RemoveAll(this);
	UWorldPartitionMiniMapBuilder::OnWarmupTick.RemoveAll(this);
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	if (GUnrealEd)
	{
		for (const FName& Name : VisualizersToUnregisterOnShutdown)
		{
			GUnrealEd->UnregisterComponentVisualizer(Name);
		}
	}

	FEditorBuildUtils::UnregisterCustomBuildType(EditorBuildTypeForce);
	FEditorBuildUtils::UnregisterCustomBuildType(EditorBuildType);
}


void FMeshPartitionEditorModule::OnPostEngineInit()
{
	if (GUnrealEd)
	{
		auto RegisterVisualizer = [this](FName ComponentClassName, TSharedPtr<FComponentVisualizer> Visualizer)
		{
			GUnrealEd->RegisterComponentVisualizer(ComponentClassName, Visualizer);
			// This call should maybe be inside the RegisterComponentVisualizer call above, but since it's not,
			// we'll put it here.
			Visualizer->OnRegister();
			VisualizersToUnregisterOnShutdown.Add(ComponentClassName);

		};

		RegisterVisualizer(MeshPartition::UModifierComponent::StaticClass()->GetFName(), MakeShared<FMegaMeshModifierVisualizer>());
		RegisterVisualizer(MeshPartition::UStaticMeshPreviewComponent::StaticClass()->GetFName(), MakeShared<FMeshPreviewVisualizer>());
		RegisterVisualizer(MeshPartition::UPreviewMeshComponent::StaticClass()->GetFName(), MakeShared<FMeshPreviewVisualizer>());
	}

	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));

	TSharedRef<FFilterCategory> MeshPartitionFilters = MakeShared<FFilterCategory>(LOCTEXT("MeshPartition_OutlinerFilter_Category", "Mesh Partition"), LOCTEXT("MeshPartition_OutlinerFilter_CategoryTooltip", "Mesh Partition Filters"));
	LevelEditorModule.AddCustomFilterToOutliner(FLevelEditorOutlinerSettings::FOutlinerFilterFactory::CreateLambda([MeshPartitionFilters]() 
	{
		return MakeShared<FMeshPartitionOutlinerFilter>(MeshPartitionFilters);
	}));
	LevelEditorModule.AddCustomFilterToOutliner(FLevelEditorOutlinerSettings::FOutlinerFilterFactory::CreateLambda([MeshPartitionFilters]() 
	{
		return MakeShared<FBaseModifierFilter>(MeshPartitionFilters);
	}));
	LevelEditorModule.AddCustomFilterToOutliner(FLevelEditorOutlinerSettings::FOutlinerFilterFactory::CreateLambda([MeshPartitionFilters]() 
	{
		return MakeShared<FBuiltSectionFilter>(MeshPartitionFilters);
	}));
}

void FMeshPartitionEditorModule::ModulesChangedCallback(FName InModuleName, EModuleChangeReason InChangeReason)
{
	if ((InChangeReason == EModuleChangeReason::ModuleLoaded) && (InModuleName.ToString() == TEXT("FoliageEdit")))
	{
		RegisterFoliageBaseIDClassIgnoreList();
	}
}

void FMeshPartitionEditorModule::RegisterFoliageBaseIDClassIgnoreList()
{
	if (IFoliageEditModuleBase* FoliageModule = IFoliageEditModuleBase::Get(); ensure(FoliageModule))
	{
		// Foliage painted onto preview section components will be deleted when the preview component is deleted.
		// Ignore them as valid base id components to prevent this behavior. Foliage painted onto Preview Sections will just
		// be unlinked.
		FoliageModule->RegisterComponentBaseIDClassToIgnore(MeshPartition::UPreviewMeshComponent::StaticClass());
		FoliageModule->RegisterComponentBaseIDClassToIgnore(MeshPartition::UStaticMeshPreviewComponent::StaticClass());
		FoliageModule->RegisterComponentBaseIDClassToIgnore(MeshPartition::UMeshPartitionCollisionComponent::StaticClass());
	}
}

void FMeshPartitionEditorModule::OnMinimapWarmupTick(UWorld* World)
{
	// we must update the editor components, in synchronous mode, to ensure that preview meshes are built, so we can render them to the minimap
	if (World)
	{
		for (AMeshPartition* MeshPartition : TActorRange<AMeshPartition>(World))
		{
			if (UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent()))
			{
				EditorComponent->SetPreviewSectionBuildEnabled(true);
				EditorComponent->SetForceSynchronousPreviewSectionBuild(true);
				EditorComponent->Update();
				EditorComponent->SetForceSynchronousPreviewSectionBuild(false);
			}
		}
	}
}

IMPLEMENT_MODULE(FMeshPartitionEditorModule, MeshPartitionEditor)
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
	
