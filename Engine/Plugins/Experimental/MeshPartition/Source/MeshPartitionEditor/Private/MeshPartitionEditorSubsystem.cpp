// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorSubsystem.h"

#include "Algo/Accumulate.h"
#include "AsyncCompilationHelpers.h" // FAsyncCompilationNotification
#include "Editor/EditorEngine.h" // UEditorEngine
//#include "HAL/IConsoleManager.h"
#include "MeshPartitionEditorModule.h" // LogMegaMeshEditor
#include "Subsystems/UnrealEditorSubsystem.h" // UUnrealEditorSubsystem
#include "EngineUtils.h" // TActorIterator
#include "MeshPartitionModifierComponent.h" // MeshPartition::UModifierComponent
#include "MeshPartitionWorldUpdater.h" // FMeshPartitionWorldUpdater
#include "MeshPartitionRuntimeCellTransformer.h" // MeshPartition::URuntimeCellTransformer
#include "WorldPartition/WorldPartition.h" // UWorldPartition
#include "Misc/MessageDialog.h" // FMessageDialog
#include "Dialogs/Dialogs.h" // FSuppressableWarningDialog
#include "MeshPartitionPreviewSection.h"
#include "Components/StaticMeshComponent.h"
#include "MeshPartitionEditorComponent.h"

extern UNREALED_API UEditorEngine* GEditor;

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionEditorSubsystem)

namespace UE::MeshPartition
{
TAutoConsoleVariable<bool> CVarMeshPartitionPIECreateMissing(TEXT("MeshPartition.PIE.CreateMissingSections"),
	true,
	TEXT("Whether to create any missing (or stripped) compiled sections when playing in editor."));

TAutoConsoleVariable<int32> CVarGCThresholdMB(TEXT("MeshPartition.Preview.GCThresholdMB"),
	2ll * 1024, // Default 2GB
	TEXT("Threshold before garbage memory generated by preview sections must be garbage collected.")
);

static FAutoConsoleCommand CmdClearCache(
	TEXT("MegaMesh.Cache.Clear"),
	TEXT("Clears all data stored in the MegaMesh build caches"),
	FConsoleCommandDelegate::CreateLambda([]()
		{
			if (MeshPartition::FModifierGraphCache* Cache = UMeshPartitionEditorSubsystem::GetGraphCache())
			{
				UE_LOGF(LogMegaMeshEditor, Log, "Freed %.3lf MB from the MegaMesh Preview build cache", Cache->GetTotalMemoryUsageMB());
				Cache->Clear();
				ensure(Cache->GetTotalMemoryUsageMB() == 0.);
			}

			if (MeshPartition::FMeshBuilder* Builder = UMeshPartitionEditorSubsystem::GetMeshBuilder())
			{
				UE_LOGF(LogMegaMeshEditor, Log, "Freed %.3lf MB from the MegaMesh Builder cache", Builder->GetCacheTotalMemoryUsageMB());
				Builder->ClearCache();
				ensure(Builder->GetCacheTotalMemoryUsageMB() == 0.);
			}
		})
);

static FAutoConsoleCommand CmdCacheLogUsage(
	TEXT("MegaMesh.Cache.Usage"),
	TEXT("Logs the current memory usage of the cache in MB"),
	FConsoleCommandDelegate::CreateLambda([]()
		{
			if (MeshPartition::FModifierGraphCache* Cache = UMeshPartitionEditorSubsystem::GetGraphCache())
			{
				UE_LOGF(LogMegaMeshEditor, Log, "MegaMesh Modifier View Cache contains %.3lf MB of data", Cache->GetViewCacheMemoryUsageMB());
				UE_LOGF(LogMegaMeshEditor, Log, "MegaMesh Modifier Merged Base Cache contains %.3lf MB of data", Cache->GetMergedBaseCacheMemoryUsageMB());
			}

			if (MeshPartition::FMeshBuilder* Builder = UMeshPartitionEditorSubsystem::GetMeshBuilder())
			{
				UE_LOGF(LogMegaMeshEditor, Log, "MegaMesh Builder Cache contains %.3lf MB of data", Builder->GetCacheTotalMemoryUsageMB());
			}
		})
);

UMeshPartitionEditorSubsystem::UMeshPartitionEditorSubsystem()
{
	bIsTickable = !IsTemplate();
}

void UMeshPartitionEditorSubsystem::Tick(float InDeltaSeconds)
{
	// Updates the notifications in case we are above the time threshold to be sure to display them.
	if (PreviewSectionBuildNotification.IsValid() && (TotalPreviewSectionBuildNumber > 0))
	{
		PreviewSectionBuildNotification->Update(TotalPreviewSectionBuildNumber);
	}

	SeparateWorldBuilder.Tick();

	uint32 TotalInteractiveSectionBuildNumber = GetTotalInteractiveSectionBuildNumber();

	if (InteractiveSectionBuildNotification.IsValid() && (TotalInteractiveSectionBuildNumber > 0))
	{
		InteractiveSectionBuildNotification->Update(TotalInteractiveSectionBuildNumber);
	}

	MeshBuilder.EnforceMemoryBudget();
}

TStatId UMeshPartitionEditorSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMeshPartitionEditorSubsystem, STATGROUP_Tickables);
}

void UMeshPartitionEditorSubsystem::Initialize(FSubsystemCollectionBase& InCollection)
{
	Super::Initialize(InCollection);
	PreviewSectionBuildNotification = MakeUnique<FAsyncCompilationNotification>(GetPreviewSectionNameFormat());
	InteractiveSectionBuildNotification = MakeUnique<FAsyncCompilationNotification>(GetInteractiveSectionNameFormat());
	if (bIsTickable)
	{
		MeshPartition::IMeshPartitionCompilerInterface::RegisterMegaMeshCompiler(this);

		FEditorDelegates::StartPIE.AddUObject(this, &UMeshPartitionEditorSubsystem::OnStartPIE);
		FEditorDelegates::PostPIEStarted.AddUObject(this, &UMeshPartitionEditorSubsystem::OnPostPIEStarted);

		FEditorDelegates::EndPIE.AddUObject(this, &UMeshPartitionEditorSubsystem::EndPIE);
		FEditorDelegates::ShutdownPIE.AddUObject(this, &UMeshPartitionEditorSubsystem::ShutdownPIE);
		FEditorDelegates::CancelPIE.AddUObject(this, &UMeshPartitionEditorSubsystem::CancelPIE);

#if WITH_EDITOR
		UWorldPartition::OnPrepareEditorGameWorldForPIE.AddUObject(this, &UMeshPartitionEditorSubsystem::OnPrepareEditorGameWorldForPIE);
		UWorldPartition::OnShutdownEditorGameWorldForPIE.AddUObject(this, &UMeshPartitionEditorSubsystem::OnShutdownEditorGameWorldForPIE);
#endif // WITH_EDITOR
	}

	FCoreUObjectDelegates::GetPostGarbageCollect().AddUObject(this, &UMeshPartitionEditorSubsystem::OnPostGC);
}

void UMeshPartitionEditorSubsystem::Deinitialize()
{
	FCoreUObjectDelegates::GetPostGarbageCollect().RemoveAll(this);

	if (bIsTickable)
	{
		FEditorDelegates::StartPIE.RemoveAll(this);
		FEditorDelegates::PostPIEStarted.RemoveAll(this);

		FEditorDelegates::EndPIE.RemoveAll(this);
		FEditorDelegates::ShutdownPIE.RemoveAll(this);
		FEditorDelegates::CancelPIE.RemoveAll(this);

#if WITH_EDITOR
		UWorldPartition::OnPrepareEditorGameWorldForPIE.RemoveAll(this);
		UWorldPartition::OnShutdownEditorGameWorldForPIE.RemoveAll(this);
#endif // WITH_EDITOR

		MeshPartition::IMeshPartitionCompilerInterface::UnregisterMegaMeshCompiler(this);
	}
	InteractiveSectionBuildNotification.Reset();
	PreviewSectionBuildNotification.Reset();
	Super::Deinitialize();
}

UMeshPartitionEditorSubsystem* UMeshPartitionEditorSubsystem::Get()
{
	return GEditor ? GEditor->GetEditorSubsystem<UMeshPartitionEditorSubsystem>() : nullptr;
}

void UMeshPartitionEditorSubsystem::BuildPlaceholderCompiledSection(MeshPartition::ACompiledSection* CompiledSection)
{
	// we can't do a blocking build here, because this is called within a blocking streaming load.
	// and building requieres another blocking stgreaming load (and it's non-reentrant..)
	// so instead we put it in a queue and build it in the next Tick
	SeparateWorldBuilder.RequestCompiledSectionBuild(CompiledSection);
}

uint32 UMeshPartitionEditorSubsystem::GetTotalPreviewSectionBuildNumber() const
{
	return Algo::Accumulate(EditorComponentToPreviewSectionBuildNumber, 0, [](uint32 Result, const TPair<TWeakObjectPtr<UMeshPartitionEditorComponent>, uint32>& Pair)
	{
		return Pair.Key.IsValid() ? Result + Pair.Value : Result;
	});
}

uint32 UMeshPartitionEditorSubsystem::GetTotalInteractiveSectionBuildNumber() const
{
	return InteractiveEditorComponents.Num();
}

void UMeshPartitionEditorSubsystem::SetPreviewSectionBuildNumber(UMeshPartitionEditorComponent* InMegaMeshEditorComponent, uint32 InPreviewSectionBuildNumber)
{
	// Remove stale entries
	for (auto It = EditorComponentToPreviewSectionBuildNumber.CreateIterator(); It; ++It)
	{
		if (!It->Key.IsValid())
		{
			It.RemoveCurrent();
		}
	}

	uint32& Value = EditorComponentToPreviewSectionBuildNumber.FindOrAdd(InMegaMeshEditorComponent, 0);
	Value = InPreviewSectionBuildNumber;

	TotalPreviewSectionBuildNumber = GetTotalPreviewSectionBuildNumber();

	if (PreviewSectionBuildNotification.IsValid())
	{
		PreviewSectionBuildNotification->Update(TotalPreviewSectionBuildNumber);
	}
}

void UMeshPartitionEditorSubsystem::SetInteractiveSectionBuild(UMeshPartitionEditorComponent* InMegaMeshEditorComponent, const bool bInIsBuilding)
{
	if (bInIsBuilding)
	{
		InteractiveEditorComponents.Emplace(InMegaMeshEditorComponent);
	}
	else
	{
		InteractiveEditorComponents.Remove(InMegaMeshEditorComponent);
	}

	uint32 TotalInteractiveSectionBuildNumber = GetTotalInteractiveSectionBuildNumber();

	if (InteractiveSectionBuildNotification.IsValid())
	{
		InteractiveSectionBuildNotification->Update(TotalInteractiveSectionBuildNumber);
	}
}

FTextFormat UMeshPartitionEditorSubsystem::GetPreviewSectionNameFormat() const
{
	return NSLOCTEXT("MegaMeshEditorSubsystem", "PreviewSectionNameFormat", "{0}|plural(one=Preview Section,other=Preview Sections)");
}

FTextFormat UMeshPartitionEditorSubsystem::GetInteractiveSectionNameFormat() const
{
	return NSLOCTEXT("MegaMeshEditorSubsystem", "InteractiveSectionNameFormat", "{0}|plural(one=Interactive Section,other=Interactive Sections)");
}

int32 UMeshPartitionEditorSubsystem::CountMeshPartitions(UWorld* World)
{
	int32 MeshPartitionCount = 0;
	for (TActorIterator<AMeshPartition> It(World); It; ++It)
	{
		if (IsValid(*It))
		{
			++MeshPartitionCount;
		}
	}
	return MeshPartitionCount;
}

void UMeshPartitionEditorSubsystem::TrackGarbagePreviewSection(MeshPartition::APreviewSection* InGarbagePreviewSection)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorSubsystem::TrackGarbagePreviewSection);

	// prevent collecting memory information for static meshes which are currently compiling.
	// GetResourceSizeEx on UStaticMesh blocks until compilation is complete. In the case where we are marking a garbage preview
	// we will cancel the SM build without producing a SM object.
	InGarbagePreviewSection->ForAllPrimitiveComponents([&EstimatedGarbagePreviewSectionTotalMB = EstimatedGarbagePreviewSectionTotalMB](UPrimitiveComponent* Component)
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(Component))
			{
				if (StaticMeshComponent->GetStaticMesh() && StaticMeshComponent->IsCompiling())
				{
					return true;
				}
			}
			
			FResourceSizeEx Size(EResourceSizeMode::EstimatedTotal);
			Component->GetResourceSizeEx(Size);
	
			EstimatedGarbagePreviewSectionTotalMB += Size.GetTotalMemoryBytes() / (1024 * 1024);
			
			return true;
		});

	if (EstimatedGarbagePreviewSectionTotalMB > CVarGCThresholdMB.GetValueOnGameThread())
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
}

void UMeshPartitionEditorSubsystem::OnPostGC()
{
	EstimatedGarbagePreviewSectionTotalMB = 0;
}

void UMeshPartitionEditorSubsystem::OnPrepareEditorGameWorldForPIE(UWorldPartition* WorldPartition)
{
	UWorld* World = WorldPartition->GetWorld();
	UE_LOGF(LogMegaMeshEditor, Verbose, "OnPrepareEditorGameWorldForPIE (World: %p %ls, PIE: %d, ExistingPIE: %p)", World, *World->GetPathName(), World->IsPlayInEditor(), PIEWorldUpdater ? PIEWorldUpdater->GetPlaceholderWorld() : nullptr);

	// Check if we have any Mesh Partitions
	if (CountMeshPartitions(World) > 0)
	{
		// create a new PIE session info -- but first check if we already have one
		// NOTE: since the separate builder world is also a PIE world, we will get calls here when world partition initializes streaming in it.  We avoid doing anything in that case because there should already be a PIE World Updater present for the PIE world.
		if (!PIEWorldUpdater.IsValid())
		{
			// Check and warn if there are any in-memory changes to Mesh Partition components, as these will not show up in PIE.
			// This must happen before constructing the WorldUpdater, because unsaved MeshPartitions won't have
			// serialized descriptors and will be skipped during the update. Give the user a chance to cancel first.
			// TODO [chris.tchou] : (until we build the code to copy changes to the separate builder world)
			{
				int32 PotentiallyDirtyModifiersCount = 0;
				for (TActorIterator<AActor> It(World); It; ++It)
				{
					AActor* Actor = *It;
					TInlineComponentArray<MeshPartition::UModifierComponent*> Modifiers(Actor, false);
					if (Modifiers.Num() > 0)
					{
						if (Actor->GetPackage()->IsDirty())
						{
							PotentiallyDirtyModifiersCount += Modifiers.Num();
						}
					}
				}
				if (PotentiallyDirtyModifiersCount > 0)
				{
					UE_LOGF(LogMegaMeshEditor, Warning, "Your editor world contains %d dirty Mesh Partition modifiers.  Any changes will not be reflected in PIE until you save them.", PotentiallyDirtyModifiersCount);

					const FText WarningMessage = FText::Format(
						NSLOCTEXT("MeshPartitionEditor", "DirtyModifiers_Message", "Your editor world contains {0} unsaved Mesh Partition {0}|plural(one=modifier,other=modifiers). Any changes will not be reflected in PIE until you save them."),
						PotentiallyDirtyModifiersCount);
					const FText WarningTitle = NSLOCTEXT("MeshPartitionEditor", "DirtyModifiers_Title", "Unsaved Mesh Partition Modifiers");

					FSuppressableWarningDialog::FSetupInfo SetupInfo(WarningMessage, WarningTitle, TEXT("MeshPartition_DirtyModifiersPIEWarning"));
					SetupInfo.ConfirmText = NSLOCTEXT("MeshPartitionEditor", "DirtyModifiers_OK", "OK");
					SetupInfo.CancelText = NSLOCTEXT("MeshPartitionEditor", "DirtyModifiers_Cancel", "Cancel");
					SetupInfo.bDefaultToSuppressInTheFuture = false;

					FSuppressableWarningDialog DirtyModifiersWarning(SetupInfo);
					if (DirtyModifiersWarning.ShowModal() == FSuppressableWarningDialog::Cancel)
					{
						bCancelPIE = true;
						return;
					}
				}
			}

			SeparateWorldBuilder.ClearRequests();
			PIEPathFixer.Reset();
			PIEWorldUpdater = MakeShared<FMeshPartitionWorldUpdater>(World, FMeshPartitionWorldUpdater::EUpdateMode::ForPIE);
			PIEOriginalEditorWorldPath = FSoftObjectPath(World);

			if (!bCancelPIE)
			{
				// create placeholders INTO EDITOR WORLD, so they get duplicated into PIE world(s)
				const bool bCreatePlaceholders = CVarMeshPartitionPIECreateMissing.GetValueOnGameThread();
				if (bCreatePlaceholders)
				{
					FMeshPartitionWorldUpdater* PIEUpdater = PIEWorldUpdater.Get();
					int32 CreatedCount = PIEUpdater->CreatePlaceholderActors(World);
					UE_LOGF(LogMegaMeshEditor, Verbose, "OnPrepareEditorGameWorldForPIE -- Created %d placeholders", CreatedCount);
				}
			}
		}
	}
}

void UMeshPartitionEditorSubsystem::OnShutdownEditorGameWorldForPIE(UWorldPartition* WorldPartition)
{
	UWorld* World = WorldPartition->GetWorld();
	UE_LOGF(LogMegaMeshEditor, Verbose, "OnShutdownEditorGameWorldForPIE (World: %p %ls, PIE: %d, ExistingPIE: %p)", World, *World->GetPathName(), World->IsPlayInEditor(), PIEWorldUpdater ? PIEWorldUpdater->GetPlaceholderWorld() : nullptr);
	if (PIEWorldUpdater.IsValid())
	{
		int32 RemovedCount = PIEWorldUpdater->RemovePlaceholderActors();
		UE_LOGF(LogMegaMeshEditor, Verbose, "OnShutdownEditorGameWorldForPIE -- Removed %d placeholders", RemovedCount);
	}

	// this should be cleared already by OnPostPIEStarted, but just in case
	bCancelPIE = false;

	// NOTE: we don't clear the PIE world updater here because this is called after the first PIE world is exited,
	// however, PIE may re-enter that world later within the same PIE session, so we keep the updater until PIE ends
}

void UMeshPartitionEditorSubsystem::OnStartPIE(const bool bIsSimulating)
{
	UWorld* EditorWorld = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>()->GetEditorWorld();
	UE_LOGF(LogMegaMeshEditor, Verbose, "OnStartPIE  (%p %ls, PIE: %d)", EditorWorld, *EditorWorld->GetName(), EditorWorld->IsPlayInEditor());

	bCancelPIE = false;

	SeparateWorldBuilder.ClearRequests();
	PIEPathFixer.Reset();

	// Check if we have any Mesh Partitions
	if (CountMeshPartitions(EditorWorld) > 0)
	{
		// Double check that we have a world partition, and that it has a Mesh Partition RuntimeCellTransformer installed
		if (UWorldPartition* WorldPartition = EditorWorld->GetWorldPartition())
		{
			// if we don't have the required cell transformer, add it
			if (!WorldPartition->HasRuntimeCellTransformerOfType(MeshPartition::URuntimeCellTransformer::StaticClass()))
			{
				WorldPartition->AddRuntimeCellTransformerOfType(TSubclassOf<UWorldPartitionRuntimeCellTransformer>(MeshPartition::URuntimeCellTransformer::StaticClass()));
			}
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Error, "MegaMesh requires a world-partitioned level, please convert this level to world-partition if you wish to use MegaMesh at runtime");

			const static FText MessageTitle = FText::FromString("Mesh Partition requires World Partition!");
			FString MessageString = FString::Printf(TEXT("World Partition is not enabled for the level, and Mesh Partition will not work at runtime in this level. Conversion to World Partition is available via \"Tools > Convert Level...\"."));
			FText MessageText = FText::FromString(MessageString);
			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgCategory::Warning, EAppMsgType::OkCancel, EAppReturnType::Ok, MessageText, MessageTitle);

			if (Result == EAppReturnType::Cancel)
			{
				bCancelPIE = true;
				return;
			}
		}
	}
}

// this is called when PIE starts
void UMeshPartitionEditorSubsystem::OnPostPIEStarted(const bool bIsSimulating)
{
	if (bCancelPIE)
	{
		bCancelPIE = true;
		GEditor->RequestEndPlayMap();
		return;
	}
}

void UMeshPartitionEditorSubsystem::EndPIE(const bool bIsSimulating)
{
	bCancelPIE = false;
	if (PIEWorldUpdater.IsValid())
	{
		int32 RemovedCount = PIEWorldUpdater->RemovePlaceholderActors();
		UE_LOGF(LogMegaMeshEditor, Verbose, "EndPIE -- Removed %d placeholders", RemovedCount);
		SeparateWorldBuilder.ClearRequests();
		PIEWorldUpdater.Reset();
		PIEPathFixer.Reset();
	}
}

void UMeshPartitionEditorSubsystem::ShutdownPIE(const bool bIsSimulating)
{
	// cleanup _should_ be done in EndPIE, but just in case, let's also clean up here
	bCancelPIE = false;
	if (!ensure(!PIEWorldUpdater.IsValid()))
	{
		int32 RemovedCount = PIEWorldUpdater->RemovePlaceholderActors();
		UE_LOGF(LogMegaMeshEditor, Warning, "ShutdownPIE -- Removed %d placeholders that were not cleaned up properly", RemovedCount);
		SeparateWorldBuilder.ClearRequests();
		PIEWorldUpdater.Reset();
		PIEPathFixer.Reset();
	}
}

void UMeshPartitionEditorSubsystem::CancelPIE()
{
	// cleanup _should_ be done in EndPIE, but just in case, let's also clean up here
	bCancelPIE = false;
	if (!ensure(!PIEWorldUpdater.IsValid()))
	{
		int32 RemovedCount = PIEWorldUpdater->RemovePlaceholderActors();
		UE_LOGF(LogMegaMeshEditor, Warning, "CancelPIE -- Removed %d placeholders that were not cleaned up properly", RemovedCount);
		SeparateWorldBuilder.ClearRequests();
		PIEWorldUpdater.Reset();
		PIEPathFixer.Reset();
	}
}

TSharedPtr<FMeshPartitionWorldUpdater> UMeshPartitionEditorSubsystem::GetPIEWorldUpdaterForWorld(UWorld* World)
{
	// WorldPath may be a PIE path -- we need to fix it to get the NonPIE package name
	FSoftObjectPath WorldPath(World);
	PIEPathFixer.FixInPlace(WorldPath);
	if (WorldPath == PIEOriginalEditorWorldPath)
	{
		return PIEWorldUpdater;
	}
	else
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Failed to get PIE World Updater for PIE World '%ls' when world updater was built for World '%ls'",
			*WorldPath.ToString(), *PIEOriginalEditorWorldPath.ToString())
		return nullptr;
	}
}

} // namespace UE::MeshPartition