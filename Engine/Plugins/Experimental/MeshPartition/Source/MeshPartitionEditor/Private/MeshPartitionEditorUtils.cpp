// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorUtils.h"

#include "Algo/Find.h"
#include "Algo/ForEach.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Engine/Engine.h"
#include "Engine/StaticMesh.h"
#include "Engine/StaticMeshSourceData.h"
#include "HAL/IConsoleManager.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshPartitionCompiledSection.h"		// MeshPartition::FCompiledSectionBuildInfo
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierDescriptors.h"	// FModifierDescriptorPair
#include "MeshPartitionModifierUtils.h"			// BetterHashCombine
#include "Misc/ArchiveMD5.h"
#include "PhysicsEngine/BodySetup.h"
#include "UObject/Class.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "Editor.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "DeviceProfiles/DeviceProfileManager.h"
#include "DeviceProfiles/DeviceProfile.h"

namespace UE::MeshPartition::EditorUtils
{
	namespace
	{
		UMaterialInstanceDynamic* CreateMaterialInstance(const UMaterialInterface* InBaseMegaMeshMaterial, UObject* InOuter, const FName& InName, EObjectFlags InAdditionalObjectFlags)
		{
			UMaterialInterface* BaseMegaMeshMaterial = const_cast<UMaterialInterface*>(InBaseMegaMeshMaterial);
			if (!IsValid(BaseMegaMeshMaterial))
			{
				BaseMegaMeshMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
			}
			UMaterialInstanceDynamic* ResultMID = UMaterialInstanceDynamic::Create(BaseMegaMeshMaterial, InOuter, InName);
			ResultMID->SetFlags(InAdditionalObjectFlags);
			return	ResultMID;
		}
	}

	static FAutoConsoleCommand CVarGenerateNewModifierOpVersionGuid(
		TEXT("MegaMesh.GenerateNewModifierOpGuids"),
		TEXT("Generates a set of new random modifier op guids"),
		FConsoleCommandWithArgsDelegate::CreateStatic(
			[](const TArray<FString>& Args)
			{
				const int32 Num = Args.Num() > 0 ? FCString::Atoi(*Args[0]) : 1;
				for (int Index = 0; Index < Num; ++Index)
				{
					UE_LOGF(LogMegaMeshEditor, Log, "%ls", *FGuid::NewGuid().ToString(EGuidFormats::DigitsWithHyphensLower));
				}
			}
		)
	);

	int32 GetMegaMeshClassVersionFromClass(const UClass* InClass)
	{
		if (ensure(InClass))
		{
			check(!InClass->HasAnyClassFlags(CLASS_Abstract));
			const FString* Value = InClass->FindMetaData(TEXT("MegaMeshClassVersion"));

			int32 CodeVersionKey = 0;
			if (InClass->IsChildOf<MeshPartition::UModifierComponent>())
			{
				// #todo [roey]: Retaining the sign of the version number in the class version to keep the old behavior of negative sign meaning non-determinisitic guids.
				//				 As soon as GatherDependency path is implemented for all modifiers this hack will be removed.
				CodeVersionKey = FMath::Abs(static_cast<int32>(GetTypeHash(Cast<MeshPartition::UModifierComponent>(InClass->GetDefaultObject())->GetCodeVersionKey())));

				if (Value != nullptr && FCString::Atoi(**Value) > 0)
				{
					return CodeVersionKey;
				}
				else
				{
					return -1 * CodeVersionKey;
				}
			}
			else if (Value != nullptr)
			{
				return FCString::Atoi(**Value);
			}
		}

		return 0;
	}

	FGuid ComputePackageHash(IAssetRegistry& AssetRegistry, TArrayView<const FName> PackageNames, TArray<uint32>* OutPackageChecksums)
	{
		FArchiveMD5 AllPackageHash;

		int32 PackageHashVersion = 1;
		AllPackageHash << PackageHashVersion;

		for (FName PackageName : PackageNames)
		{
			FAssetPackageData AssetPackageData;
			if (AssetRegistry.TryGetAssetPackageData(PackageName, AssetPackageData) == UE::AssetRegistry::EExists::Exists)
			{
				FIoHash PackageHash = AssetPackageData.GetPackageSavedHash();
				UE_LOGF(LogMegaMeshEditor, VeryVerbose, "------ Package %ls -- Hash %ls", *PackageName.ToString(), *LexToString(PackageHash));
				if (OutPackageChecksums)
				{
					FIoHash::ByteArray& HashBytes = PackageHash.GetBytes();
					OutPackageChecksums->Add(
						(((uint32)HashBytes[0]) << 24) +
						(((uint32)HashBytes[1]) << 16) +
						(((uint32)HashBytes[2]) << 8) +
						(((uint32)HashBytes[3]) << 0));
				}
				AllPackageHash << PackageHash;
			}
			else
			{
				UE_LOGF(LogMegaMeshEditor, Log, "Cannot find asset registry entry for package %ls - package hash is incomplete and incremental build may build more than is necessary", *PackageName.ToString());
				if (OutPackageChecksums)
				{
					OutPackageChecksums->Add(0);
				}
			}
		}

		FGuid PackageHash = AllPackageHash.GetGuidFromHash();
		UE_LOGF(LogMegaMeshEditor, VeryVerbose, "---- PackageHash %ls", *PackageHash.ToString());

		return PackageHash;
	}

	FGuid ComputeClassHash(TArrayView<const UClass*> Classes, TArray<uint32>* OutClassChecksums)
	{
		FArchiveMD5 ClassHash;
		for (const UClass* Class : Classes)
		{
			int32 MegaMeshClassVersion = GetMegaMeshClassVersionFromClass(Class);
			if (OutClassChecksums)
			{
				OutClassChecksums->Add((uint32) MegaMeshClassVersion);
			}
			ClassHash << MegaMeshClassVersion;
		}
		return ClassHash.GetGuidFromHash();
	}

	bool PackageHashIsUpToDate(const MeshPartition::FCompiledSectionBuildInfo& BuildInfo, IAssetRegistry& AssetRegistry)
	{
		// look at the list of packages and package hashes attached to the compiled section actor desc
		FGuid ComputedPackageHash = ComputePackageHash(AssetRegistry, BuildInfo.PackageDependencies);
		return (BuildInfo.PackageHash == ComputedPackageHash);
	}

	bool ClassHashIsUpToDate(const MeshPartition::FCompiledSectionBuildInfo& BuildInfo)
	{
		TArray<const UClass*> Classes;
		Algo::Transform(BuildInfo.ClassDependencies, Classes, [](FName ClassPathName)
			{
				FSoftClassPath ClassPath(ClassPathName.ToString());
				return ClassPath.ResolveClass();
			});
		FGuid ComputedClassHash = ComputeClassHash(Classes);
		return (BuildInfo.ClassHash == ComputedClassHash);
	}

	TArray<FGuid> FindMatchingCompiledSectionsFromPackageHashes(
		TArrayView<const FCompiledSectionDescriptor> CompiledSectionDescriptorsToSearch,
		const MeshPartition::FCompiledSectionBuildInfo& BuildInfo,
		IAssetRegistry& AssetRegistry)
	{
		TArray<FGuid> MatchingGuids;

		for (const FCompiledSectionDescriptor& Desc : CompiledSectionDescriptorsToSearch)
		{
			if (Desc.Info.TargetsSameCompiledSectionAs(BuildInfo) &&			// ensures we are targeting the same compiled section
				Desc.Info.BuildVariantHash == BuildInfo.BuildVariantHash &&		// catches changes to build variant
				Desc.Info.ModifierSetHash == BuildInfo.ModifierSetHash)			// catches additions or removals from the set of modifiers (TODO: DOES NOT WORK IN LEVEL INSTANCES)
			{
				if (PackageHashIsUpToDate(Desc.Info, AssetRegistry) &&			// all used packages are unmodified on disk
					ClassHashIsUpToDate(Desc.Info))								// all used classes have the same MegaMeshClassVersion
				{
					MatchingGuids.Add(Desc.ActorDescGuid);
				}
			}
		}
		return MatchingGuids;
	}

	TArray<FGuid> FindMatchingCompiledSectionsFromModifierHash(
		TArrayView<const FCompiledSectionDescriptor> CompiledSectionDescriptorsToSearch,
		const MeshPartition::FCompiledSectionBuildInfo& BuildInfo)
	{
		TArray<FGuid> MatchingGuids;

		for (const FCompiledSectionDescriptor& Desc : CompiledSectionDescriptorsToSearch)
		{
			check(Desc.Info.MegaMeshGUID == BuildInfo.MegaMeshGUID);				// we should have filtered out mismatches by this point

			if (Desc.Info.ModifiersHash == BuildInfo.ModifiersHash &&				// catches change to any modifier hash
				Desc.Info.BuildVariantHash == BuildInfo.BuildVariantHash &&			// catches changes to build variant
				Desc.Info.TargetsSameCompiledSectionAs(BuildInfo))					// ensures we are targetting the same compiled section
				// Desc.Info.ModifierSetHash == BuildInfo.ModifierSetHash			// not necessary: is a subset of changes covered by ModifiersHash
			{
				MatchingGuids.Add(Desc.ActorDescGuid);
			}
		}
		return MatchingGuids;
	}

	bool ValidateObjectsAreUnloaded(TConstArrayView<TWeakObjectPtr<AActor>> InActorsToValidate)
	{
		TArray<AActor*> ActorsNotUnloaded;
		for (const TWeakObjectPtr<AActor>& Actor : InActorsToValidate)
		{
			if (Actor.IsValid() && IsValid(Actor.Get()))
			{
				ActorsNotUnloaded.Add(Actor.Get());
			}
		}

		if (ActorsNotUnloaded.Num() > 0)
		{
			UE_LOGF(LogMegaMeshEditor, Warning, "MegaMesh failed to Garbage Collect %d Actors (out of %d attempted)", ActorsNotUnloaded.Num(), InActorsToValidate.Num());
			for (AActor* Actor : ActorsNotUnloaded)
			{
				// figure out why it wasn't unloaded...
				FString ClassName = Actor->GetClass()->GetName();
				FString ObjectName = Actor->GetName();
				UE_LOGF(LogMegaMeshEditor, Warning, "%ls '%ls' was not unloaded by GC.  Obj Refs:", *ClassName, *Actor->GetActorNameOrLabel());
				FString Command = FString::Printf(TEXT("OBJ REFS CLASS=%s NAME=%s"), *ClassName, *ObjectName);
				GEngine->Exec(Actor->GetWorld(), *Command);
			}
			return false;
		}
		return true;
	}

	UStaticMesh* CreateStaticMesh(UObject* InOuter, const FName InStaticMeshName)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::EditorUtils::CreateStaticMesh);

		const EObjectFlags UseFlags = EObjectFlags::RF_Public | (InOuter->HasAnyFlags(EObjectFlags::RF_Transient) ? EObjectFlags::RF_Transient : EObjectFlags::RF_NoFlags);
		FName NewStaticMeshName = MakeUniqueObjectName(InOuter, UStaticMesh::StaticClass(), InStaticMeshName);
		UStaticMesh* NewStaticMesh = NewObject<UStaticMesh>(InOuter, NewStaticMeshName, UseFlags);

		if (!ensure(NewStaticMesh != nullptr))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "Cannot create static mesh.");
			return nullptr;
		}

		return NewStaticMesh;
	}

	void BuildSourceModel(FStaticMeshSourceModel& OutSourceModel, const MeshPartition::FMeshData& InBuiltMesh, bool bInRecomputeNormals, bool bInRecomputeTangents)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::EditorUtils::BuildSourceModel);

		FMeshBuildSettings& BuildSettings = OutSourceModel.BuildSettings;

		BuildSettings.bRecomputeNormals = bInRecomputeNormals;
		BuildSettings.bRecomputeTangents = bInRecomputeTangents;
		BuildSettings.bGenerateLightmapUVs = false;
		BuildSettings.DistanceFieldResolutionScale = 0.0f;
		BuildSettings.MaxLumenMeshCards = 4;

		FMeshDescription* MeshDescription = OutSourceModel.GetOrCacheMeshDescription();

		if (MeshDescription == nullptr)
		{
			return;
		}

		InBuiltMesh.ConvertToMeshDescription(*MeshDescription);

		// Force the mesh description to precalculate indexing data structures in this async task
		// so that when this function is eventually called on the GT it will not require any significant computation.
		MeshDescription->BuildIndexers();

		// Use the hash as the guid to ensure that given the same input mesh data, we always hit the cache for the static mesh/nanite build.
		OutSourceModel.CommitMeshDescription(/* bUseHashAsGuid */ true);
	}

	void FinalizeStaticMesh(const FFinalizeStaticMeshParams& InParams)
	{
		if (!ensure(InParams.StaticMesh != nullptr))
		{
			return;
		}

		UStaticMesh* StaticMesh = InParams.StaticMesh;

		StaticMesh->CreateBodySetup();

		UBodySetup* BodySetup = StaticMesh->GetBodySetup();

		if (InParams.CollisionProfile.Name != UCollisionProfile::NoCollision_ProfileName)
		{
			BodySetup->CollisionTraceFlag = ECollisionTraceFlag::CTF_UseComplexAsSimple;
		}
		else
		{
			BodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
			BodySetup->bNeverNeedsCookedCollisionData = true;
			BodySetup->bHasCookedCollisionData = false;
			BodySetup->InvalidatePhysicsData();
		}

		if (!InParams.bCanEverAffectNavigation)
		{
			StaticMesh->MarkAsNotHavingNavigationData();
		}

		TArray<FStaticMaterial> StaticMaterials;
		FStaticMaterial NewMaterial;

		NewMaterial.MaterialInterface = (InParams.Material != nullptr) ? InParams.Material : UMaterial::GetDefaultMaterial(MD_Surface);
		NewMaterial.MaterialSlotName = TEXT("MegaMeshStaticMeshMaterial");
		NewMaterial.ImportedMaterialSlotName = NewMaterial.MaterialSlotName;
		NewMaterial.UVChannelData = FMeshUVChannelInfo(1.f);		// this avoids an ensure in  UStaticMesh::GetUVChannelData

		StaticMaterials.Add(NewMaterial);
		StaticMesh->SetStaticMaterials(StaticMaterials);

		FMeshDescription* MeshDescription = StaticMesh->GetMeshDescription(0);
		int32 NumPolygonGroups = (MeshDescription != nullptr) ? MeshDescription->PolygonGroups().Num() : 0;
		int32 MaxNumSections = FMath::Max(0, NumPolygonGroups);
		int32 CurNumValidSections = 1;
		
		while (CurNumValidSections < MaxNumSections)
		{
			StaticMesh->GetStaticMaterials().Add(FStaticMaterial());
			++CurNumValidSections;
		}

		StaticMesh->GetNaniteSettings().bEnabled = InParams.bUseNanite;
		if (InParams.bUseNanite)
		{
			StaticMesh->GetNaniteSettings().GenerateFallback = InParams.NaniteFallbackMode;
			StaticMesh->GetNaniteSettings().FallbackTarget = InParams.NaniteFallbackTarget;
			StaticMesh->GetNaniteSettings().FallbackPercentTriangles = InParams.NaniteFallbackPercentTriangles;
			StaticMesh->GetNaniteSettings().FallbackRelativeError = InParams.NaniteFallbackRelativeError;
		}

		StaticMesh->bSupportRayTracing = true;
		StaticMesh->bGenerateMeshDistanceField = false;

		if (InParams.bSetupSections)
		{
			FMeshSectionInfoMap& SectionInfoMap = StaticMesh->GetSectionInfoMap();

			for (int32 LODIndex = 0; LODIndex < FMath::Max(InParams.NumLODs, 1); LODIndex++)
			{
				for (int32 MaterialIndex = 0; MaterialIndex < StaticMesh->GetStaticMaterials().Num(); MaterialIndex++)
				{
					SectionInfoMap.Set(LODIndex, MaterialIndex, FMeshSectionInfo(MaterialIndex));
				}
			}
		}

		StaticMesh->GetOriginalSectionInfoMap().CopyFrom(StaticMesh->GetSectionInfoMap());

		[[maybe_unused]] bool MarkedDirty = StaticMesh->MarkPackageDirty();
		StaticMesh->PostEditChange();
	}

	void FPIEPathFixer::Reset()
	{
		PIEToNonPIEPackageName.Reset();
	}

	void FPIEPathFixer::FixInPlace(FSoftObjectPath& InOutPath)
	{
		FName OldPackageName = InOutPath.GetAssetPath().GetPackageName();
		if (FName* NonPIEPackageName = PIEToNonPIEPackageName.Find(OldPackageName))
		{
			InOutPath.RemapPackage(OldPackageName, *NonPIEPackageName);
		}
		else
		{
			// slow path - remove PIE prefix via string ops, then cache the mapping
			FString PathString = InOutPath.ToString();
			FSoftObjectPath StrippedWorldPath(UWorld::RemovePIEPrefix(PathString));
			FName NewPackageName = StrippedWorldPath.GetAssetPath().GetPackageName();
			PIEToNonPIEPackageName.Add(OldPackageName, NewPackageName);
			InOutPath.RemapPackage(OldPackageName, NewPackageName);
		}
	}

	const FString* GetDeviceTypeFromDeviceProfile(UDeviceProfile* Profile)
	{
		while (Profile)
		{
			if (!Profile->DeviceType.IsEmpty())
			{
				return &Profile->DeviceType;
			}
			Profile = Profile->GetParentProfile();
		}
		return nullptr;
	}

	ITargetPlatform* FindTargetPlatformStartingWith(const FString& InPlatformNamePrefix, bool bSearchPlatformName, bool bSearchIniPlatformName, EBuildTargetType PreferredBuildTargetType)
	{
		// see if there is a preview platform available
		if (InPlatformNamePrefix.IsEmpty())
		{
			return nullptr;
		}

		ITargetPlatform* PlatformMatch = nullptr;
		ITargetPlatform* IniPlatformMatch = nullptr;

		// search all target platforms
		ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
		TArray<ITargetPlatform*> Platforms = TPM.GetTargetPlatforms();
		for (ITargetPlatform* Platform : Platforms)
		{
			FString PlatformName = Platform->PlatformName();							// "WindowsClient"
			FString IniPlatformName = Platform->IniPlatformName();						// "Windows"
			FString CookingDeviceProfileName = Platform->CookingDeviceProfileName();

			bool bIsClientOnly = Platform->IsClientOnly();
			bool bIsServerOnly = Platform->IsServerOnly();
			EBuildTargetType BuildTargetType = Platform->GetRuntimePlatformType();

			if (bSearchPlatformName && PlatformName.StartsWith(InPlatformNamePrefix))
			{
				if (PlatformMatch == nullptr || PlatformMatch->GetRuntimePlatformType() != PreferredBuildTargetType)
				{
					PlatformMatch = Platform;
				}
			}

			if (bSearchIniPlatformName && IniPlatformName.StartsWith(InPlatformNamePrefix))
			{
				if (IniPlatformMatch == nullptr || IniPlatformMatch->GetRuntimePlatformType() != PreferredBuildTargetType)
				{
					IniPlatformMatch = Platform;
				}
			}

			UE_LOGF(LogCore, Warning, "Platform: %ls %ls %ls [%d, %d, %d] [%ls, %ls]", *PlatformName, *IniPlatformName, *CookingDeviceProfileName, EnumToUnderlyingType(BuildTargetType), bIsClientOnly, bIsServerOnly,
				PlatformMatch ? *PlatformMatch->PlatformName() : TEXT("null"),
				IniPlatformMatch ? *IniPlatformMatch->PlatformName() : TEXT("null"));
		}

		// return the non-null match
		ITargetPlatform* OverallMatch = nullptr;
		if (PlatformMatch == nullptr)
		{
			OverallMatch = IniPlatformMatch;
		}
		else if (IniPlatformMatch == nullptr)
		{
			OverallMatch = PlatformMatch;
		}
		else
		{
			// prefer the preferred build target type
			if (PlatformMatch->GetRuntimePlatformType() == PreferredBuildTargetType)
			{
				OverallMatch = PlatformMatch;
			}
			else if (IniPlatformMatch->GetRuntimePlatformType() == PreferredBuildTargetType)
			{
				OverallMatch = IniPlatformMatch;
			}
			else
			{
				// prefer the platform match (more specific)
				OverallMatch = PlatformMatch;
			}
		}

		UE_LOGF(LogCore, Warning, "Search for '%ls' returned Platform: %ls", *InPlatformNamePrefix, OverallMatch ? *OverallMatch->PlatformName() : TEXT("null"));

		return OverallMatch;
	}

	ITargetPlatform* GetPreviewPlatform(FName& OutPreviewPlatformName)
	{
		if (!GEditor)
		{
			return nullptr;
		}

		FName PreviewPlatformName;
		GEditor->GetPreviewPlatformName(PreviewPlatformName);
		if (PreviewPlatformName.IsNone())
		{
			return nullptr;
		}

		const bool bSearchPlatformName = true;
		const bool bSearchIniPlatformName = true;

		// try matching against preview device profile device type (i.e. "Windows")
		if (UDeviceProfile* PreviewDeviceProfile = UDeviceProfileManager::Get().GetPreviewDeviceProfile())
		{
			if (const FString* DeviceType = GetDeviceTypeFromDeviceProfile(PreviewDeviceProfile))
			{
				if (ITargetPlatform* DeviceTypeMatchTargetPlatform = FindTargetPlatformStartingWith(*DeviceType, bSearchPlatformName, bSearchIniPlatformName, EBuildTargetType::Client))
				{
					OutPreviewPlatformName = PreviewPlatformName;
					return DeviceTypeMatchTargetPlatform;
				}
			}
		}

		// try matching against preview platform name
		if (ITargetPlatform* PreviewMatchTargetPlatform = FindTargetPlatformStartingWith(PreviewPlatformName.ToString(), bSearchPlatformName, bSearchIniPlatformName, EBuildTargetType::Client))
		{
			OutPreviewPlatformName = PreviewPlatformName;
			return PreviewMatchTargetPlatform;
		}

		return nullptr;
	}

	TArray<ITargetPlatform*> GetTargetPlatforms()
	{
		TArray<ITargetPlatform*> Targets;

		FName PreviewPlatformName;
		if (ITargetPlatform* PreviewTargetPlatform = GetPreviewPlatform(PreviewPlatformName))
		{
			// use preview platform if there is one
			Targets.Emplace(PreviewTargetPlatform);
		}
		else
		{
			// otherwise use active target platforms
			Targets = GetTargetPlatformManager()->GetActiveTargetPlatforms();
		}

		return MoveTemp(Targets);
	}

	UMaterialInstanceDynamic* GetOrCreateMaterialInstance(UMaterialInstanceDynamic* InMID, const UMaterialInterface* InBaseMegaMeshMaterial, UObject* InOuter, const FName& InName, EObjectFlags InAdditionalObjectFlags)
	{
		const UMaterialInterface* BaseMegaMeshMaterial = InBaseMegaMeshMaterial;

		if (IsValid(InMID) && IsValid(BaseMegaMeshMaterial) && (InMID->Parent == BaseMegaMeshMaterial))
		{
			return InMID;
		}

		return CreateMaterialInstance(BaseMegaMeshMaterial, InOuter, InName, InAdditionalObjectFlags);
	}
}