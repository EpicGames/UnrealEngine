// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorWorldSubsystem.h"
#include "EngineUtils.h"
#include "Subsystems/UnrealEditorSubsystem.h"	// UUnrealEditorSubsystem
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionWorldUpdater.h"
#include "Editor.h"								// GEditor

namespace UE::MeshPartition
{
static FAutoConsoleCommand ReportSectionStatus(
	TEXT("MeshPartition.ReportSectionStatus"),
	TEXT("Report the current mesh partition compiled section status for the current editor world"),
	FConsoleCommandDelegate::CreateStatic(&UMeshPartitionEditorWorldSubsystem::ReportSectionStatus));

void UMeshPartitionEditorWorldSubsystem::ReportSectionStatus()
{
	UWorld* EditorWorld = nullptr;
	if (GEditor)
	{
		if (UUnrealEditorSubsystem* EditorSubsystem = GEditor->GetEditorSubsystem<UUnrealEditorSubsystem>())
		{
			EditorWorld = EditorSubsystem->GetEditorWorld();
		}
	}
	if (EditorWorld == nullptr)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Dumping Compiled Section Status -- No Editor World Found");
		return;
	}

	UE_LOGF(LogMegaMeshEditor, Warning, "Dumping Compiled Section Status for World: %ls (%ls)", *EditorWorld->GetName(), *EditorWorld->GetPathName());
	FMeshPartitionWorldUpdater MeshPartitionWorldUpdater(EditorWorld, FMeshPartitionWorldUpdater::EUpdateMode::ForCompile);
	MeshPartitionWorldUpdater.ReportSectionStatus(EditorWorld);
}

TStatId UMeshPartitionEditorWorldSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMeshPartitionEditorWorldSubsystem, STATGROUP_Tickables);
}

void UMeshPartitionEditorWorldSubsystem::Tick(float InDeltaTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshPartitionEditorWorldSubsystem::Tick);
	Super::Tick(InDeltaTime);
	
	for (AMeshPartition* MeshPartition : TActorRange<AMeshPartition>(GetWorld()))
	{
		if (UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent()))
		{
			EditorComponent->Update();
		}
	}
}
} // namespace UE::MeshPartition