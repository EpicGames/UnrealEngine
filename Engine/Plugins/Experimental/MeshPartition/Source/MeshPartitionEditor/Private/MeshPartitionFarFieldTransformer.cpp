// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionFarFieldTransformer.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceConstant.h"
#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionEditorUtils.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionPreviewSection.h"

namespace UE::MeshPartition
{

namespace MeshPartitionFarFieldTransformerLocals
{
	
	
} // namespace MeshPartitionFarFieldTransformerLocals

bool FFarFieldTransformer::Execute(MeshPartition::FTransformerContext& InTransformerContext) const
{
	UE::Tasks::TTask<TArray<UStaticMesh*>> CreateStaticMeshesTask = UE::Tasks::Launch(TEXT("FFarFieldTransformer::CreateStaticMeshes"), [this,
		&InTransformerContext
		]() mutable
		{
			return CreateStaticMeshes(InTransformerContext);
		},
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri
	);

	TArray<UE::Tasks::FTask> BuildFarFieldMeshTasks;

	BuildFarFieldMeshTasks.SetNum(InTransformerContext.TransformerUnits.Num());

	for (int32 Index = 0; Index < InTransformerContext.TransformerUnits.Num(); ++Index)
	{
		BuildFarFieldMeshTasks[Index] = UE::Tasks::Launch(TEXT("FFarFieldTransformer::BuilFarFielddMesh"), [this,
			&InTransformerContext,
			&CreateStaticMeshesTask,
			Index]() mutable
			{
				BuildStaticMesh(InTransformerContext, CreateStaticMeshesTask, Index);
			}, UE::Tasks::Prerequisites(CreateStaticMeshesTask));
	}

	UE::Tasks::FTask FinalizeFarFieldMeshes = UE::Tasks::Launch(TEXT("FFarFieldTransformer::FinalizeStaticMeshes"), [this,
		&CreateStaticMeshesTask,
		&InTransformerContext]() mutable
		{
			FinalizeStaticMeshes(InTransformerContext, CreateStaticMeshesTask);
		},
		UE::Tasks::Prerequisites(BuildFarFieldMeshTasks),
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri
	);

	FinalizeFarFieldMeshes.Wait();

	return true;
}

void FFarFieldTransformer::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	InDependencies += FarFieldMeshEdgeLength;
}

TArray<UStaticMesh*> FFarFieldTransformer::CreateStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext) const
{
	check(IsInGameThread());
		   	
	TArray<UStaticMesh*> Results;
		   	
	if (InTransformerContext.bWasCancelled)
	{
		return Results;
	}
		   	
	for (const MeshPartition::FTransformerUnit& TransformerUnit : InTransformerContext.TransformerUnits)
	{
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if (!ensure(TransformerUnit.MeshData.IsValid()) || !ensure(TransformerUnit.MeshData->VertexCount() != 0) || !ensure(Section != nullptr))
		{
			// Still add nullptr to preserve 1-1 num between TransformerUnits and StaticMeshes
			Results.Add(nullptr);
			continue;
		}

		if (UStaticMesh* StaticMesh = MeshPartition::EditorUtils::CreateStaticMesh(Section, TEXT("FarFieldStaticMesh")))
		{
			StaticMesh->SetInternalFlags(EInternalObjectFlags::Async);
			Results.Add(StaticMesh);
		}
	}
		   	
	return Results;
}

void FFarFieldTransformer::BuildStaticMesh(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask, const int32 InTransformerUnitIndex) const
{
	if (InTransformerContext.bWasCancelled)
	{
		return;
	}
			
	UStaticMesh* StaticMesh = nullptr;
	{
		TArray<UStaticMesh*>& StaticMeshes = InCreateStaticMeshesTask.GetResult();
					
		if (InTransformerUnitIndex >= StaticMeshes.Num())
		{
			return;
		}
					
		StaticMesh = StaticMeshes[InTransformerUnitIndex];
	}
			
	// static mesh can be null if the built section had no data, we would not have created the static mesh for it.
	if (StaticMesh == nullptr)
	{
		return;
	}
		 	
	const MeshPartition::FTransformerUnit& TransformerUnit = InTransformerContext.TransformerUnits[InTransformerUnitIndex];

	if (!ensure(TransformerUnit.MeshData != nullptr))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get PreviewMesh.");
		return;
	}
			
	constexpr bool bTransferAttributes = true;
	const bool bTransferNormals = TransformerUnit.bShouldRecomputeNormals == false;
	const MeshPartition::FMeshData SimplifiedMesh = MeshPartition::BuildHelpers::SimplifyMesh(*TransformerUnit.MeshData, FarFieldMeshEdgeLength, bTransferAttributes, bTransferNormals);

	if (!ensure(IsValid(StaticMesh)))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get StaticMesh.");
		return;
	}
		 	
	StaticMesh->SetNumSourceModels(1);
	StaticMesh->CreateMeshDescription(0);
	FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(0);
		 	
	MeshPartition::EditorUtils::BuildSourceModel(SourceModel, SimplifiedMesh, TransformerUnit.bShouldRecomputeNormals, TransformerUnit.bShouldRecomputeTangents);
}

void FFarFieldTransformer::FinalizeStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask) const
{
	check(IsInGameThread());
		   
	TArray<UStaticMesh*>& FarFieldMeshes = InCreateStaticMeshesTask.GetResult();

	for (int32 Index = 0; Index < InTransformerContext.TransformerUnits.Num(); ++Index)
	{
		const MeshPartition::FTransformerUnit& TransformerUnit = InTransformerContext.TransformerUnits[Index];
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if ((Section == nullptr) || !ensure(Index < FarFieldMeshes.Num()))
		{
			continue;
		}

		UStaticMesh* FarFieldMesh = FarFieldMeshes[Index];

		if (FarFieldMesh != nullptr)
		{
			FarFieldMesh->ClearInternalFlags(EInternalObjectFlags::Async);
			ForEachObjectWithOuter(FarFieldMesh, [](UObject* Object)
								   {
									   Object->ClearInternalFlags(EInternalObjectFlags::Async);
								   });
		}

		FMeshDescription* MeshDescription = (FarFieldMesh != nullptr) ? FarFieldMesh->GetMeshDescription(0) : nullptr;

		if (!InTransformerContext.bWasCancelled && (MeshDescription != nullptr) && (MeshDescription->Vertices().Num() != 0))
		{
			MeshPartition::EditorUtils::FFinalizeStaticMeshParams Params;

			Params.StaticMesh = FarFieldMesh;
			Params.CollisionProfile = UCollisionProfile::NoCollision_ProfileName;
			Params.NumLODs = 0;
			Params.bCanEverAffectNavigation = false;
			Params.bUseNanite = false;
			Params.bSetupSections = false;

			if (MeshPartition::APreviewSection* PreviewSection = Cast<MeshPartition::APreviewSection>(Section))
			{
				Params.Material = PreviewSection->GetMaterialInstance();
				MeshPartition::EditorUtils::FinalizeStaticMesh(Params);

				PreviewSection->SetFarFieldMesh(FarFieldMesh);
			}
			else if (MeshPartition::ACompiledSection* CompiledSection = Cast<MeshPartition::ACompiledSection>(Section))
			{
				Params.Material = CompiledSection->GetMaterialInstance();
				MeshPartition::EditorUtils::FinalizeStaticMesh(Params);
				
				CompiledSection->SetFarFieldMesh(FarFieldMesh);
			}
			else
			{
				checkf(false, TEXT("This should never happen. Transformers should only be applied on Preview or Compiled sections at the moment."));
			}
		}
	}
}

} // namespace UE::MeshPartition
