// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionGroupRegistry.h"

namespace UE::MeshPartition
{

FGuid FModifierGroupRegistry::GetGroupRegistryKey(const MeshPartition::FModifierGroup& InGroup)
{
	return InGroup.ComputeBaseModifierSetHash();
}

FGuid FModifierGroupRegistry::StageGroup(const MeshPartition::FModifierGroup& InGroup)
{
	FGuid GroupKey = GetGroupRegistryKey(InGroup);

	StagedGroups.Add(GroupKey, InGroup);

	return GroupKey;
}

bool FModifierGroupRegistry::CommitStagedGroup(const FGuid& InGroupKey)
{
	if (!ensure(StagedGroups.Contains(InGroupKey)))
	{
		return false;
	}

	MeshPartition::FModifierGroup Group = MoveTemp(StagedGroups.FindChecked(InGroupKey));
	StagedGroups.Remove(InGroupKey);

	ensure(GetGroupRegistryKey(Group) == InGroupKey);

	Groups.Emplace(InGroupKey, MoveTemp(Group));

	return true;
}

bool FModifierGroupRegistry::RevertStagedGroup(const FGuid& InGroupKey)
{
	return StagedGroups.Remove(InGroupKey) != 0;
}

void FModifierGroupRegistry::RemoveGroup(const FGuid& InBaseSetHash)
{
	Groups.Remove(InBaseSetHash);
}

void FModifierGroupRegistry::Reset()
{
	Groups.Reset();
	StagedGroups.Reset();
}

void FModifierGroupRegistry::Reset(TConstArrayView<const MeshPartition::FModifierGroup> InModifierGroups)
{
	Reset();

	for (const MeshPartition::FModifierGroup& ModifierGroup : InModifierGroups)
	{
		Groups.Emplace(GetGroupRegistryKey(ModifierGroup), ModifierGroup);
	}
}

const MeshPartition::FModifierGroup* FModifierGroupRegistry::FindGroup(const FGuid& InKey) const
{
	return Groups.Find(InKey);
}

const MeshPartition::FModifierGroup* FModifierGroupRegistry::FindStagedGroup(const FGuid& InKey) const
{
	return StagedGroups.Find(InKey);
}

TArray<MeshPartition::FModifierGroup> FModifierGroupRegistry::FindGroupsContainingModifiers(TConstArrayView<MeshPartition::FModifierDesc> InModifiers) const
{
	TSet<FSoftObjectPath> StagedBases;

	TArray<MeshPartition::FModifierGroup> Results;

	for (const TPair<FGuid, MeshPartition::FModifierGroup>& StagedGroup : StagedGroups)
	{
		for (const MeshPartition::FModifierDesc& Modifier : InModifiers)
		{
			if (StagedGroup.Value.ModifierDescs().Contains(Modifier))
			{
				// Add all the bases from this group to the list of staged bases.
				// this will allow us to check if any of the committed groups contain a matching base
				// and if so, exclude them from the list
				for (const MeshPartition::FModifierDesc& StagedBase : StagedGroup.Value.BaseDescs())
				{
					StagedBases.Add(StagedBase.ModifierPath);
				}

				Results.Add(StagedGroup.Value);
				// don't need to do any more work on this group, it was already added.
				break;
			}
		}
	}

	for (const TPair<FGuid, MeshPartition::FModifierGroup>& CommittedGroup : Groups)
	{
		bool bMatchesStaged = false;
		
		for (const MeshPartition::FModifierDesc& BaseDesc : CommittedGroup.Value.BaseDescs())
		{
			if (StagedBases.Contains(BaseDesc.ModifierPath))
			{
				bMatchesStaged = true;
				break;
			}
		}
		
		if (bMatchesStaged)
		{
			continue;
		}

		for (const MeshPartition::FModifierDesc& Modifier : InModifiers)
		{
			auto Pred = [&ModifierPath = Modifier.ModifierPath](const MeshPartition::FModifierDesc& Desc)
			{
				return ModifierPath == Desc.ModifierPath;
			};

			if (CommittedGroup.Value.ModifierDescs().ContainsByPredicate(Pred))
			{
				Results.Add(CommittedGroup.Value);
				// Don't need to do any more work on this group, it was already added.
				break;
			}
		}
	}

	return Results;
}

TArray<MeshPartition::FModifierGroup> FModifierGroupRegistry::FindGroupsIntersectingBounds(TConstArrayView<const FBox> InBounds) const
{
	TArray<MeshPartition::FModifierGroup> Results;
	TSet<FSoftObjectPath> StagedBases;

	for (const TPair<FGuid, MeshPartition::FModifierGroup>& StagedGroupPair : StagedGroups)
	{
		const MeshPartition::FModifierGroup& Group = StagedGroupPair.Value;
		const FBox GroupBounds = Group.ComputeBaseBounds();

		for (const FBox& BoundsToCheck : InBounds)
		{
			if (BoundsToCheck.Intersect(GroupBounds))
			{
				// Add all the bases from this group to the list of staged bases.
				// this will allow us to check if any of the committed groups contain a matching base
				// and if so, exclude them from the list
				for (const MeshPartition::FModifierDesc& StagedBase : Group.BaseDescs())
				{
					StagedBases.Add(StagedBase.ModifierPath);
				}

				Results.Add(Group);
				break;
			}
		}
	}

	for (const TPair<FGuid, MeshPartition::FModifierGroup>& CommittedGroup : Groups)
	{
		const MeshPartition::FModifierGroup& Group = CommittedGroup.Value;
		bool bMatchesStaged = false;
		
		for (const MeshPartition::FModifierDesc& BaseDesc : Group.BaseDescs())
		{
			if (StagedBases.Contains(BaseDesc.ModifierPath))
			{
				bMatchesStaged = true;
				break;
			}
		}
		
		if (bMatchesStaged)
		{
			continue;
		}

		const FBox GroupBounds = Group.ComputeBaseBounds();

		for (const FBox& BoundsToCheck : InBounds)
		{
			if (BoundsToCheck.Intersect(GroupBounds))
			{
				Results.Add(Group);
				break;
			}
		}
	}

	return Results;
}

}
