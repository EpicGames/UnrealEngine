// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionHeightmapImporter.h"

#include "IImageWrapper.h"
#include "IImageWrapperModule.h"
#include "LocationVolume.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "PackageSourceControlHelper.h"
#include "SourceControlHelpers.h"
#include "ActorFactories/ActorFactory.h"
#include "Builders/CubeBuilder.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "Engine/World.h"
#include "Generators/MeshShapeGenerator.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "Modules/ModuleManager.h"
#include "UObject/SavePackage.h"
#include "WorldPartition/WorldPartition.h"
#include "MeshPartitionEditorUtils.h"

#define LOCTEXT_NAMESPACE "MegaMeshHeightmapImport"

namespace UE::MeshPartition
{
ALocationVolume* CreateLocationVolume(UWorld *const World, AMeshPartition *const MegaMesh, const FString& Label, const FBox& Bounds)
{
	checkf(World, TEXT("World must not be nullptr."));

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::CreateLocationVolume);

	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Name = *Label;
	SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;

	ALocationVolume* LocationVolume = World->SpawnActor<ALocationVolume>(Bounds.GetCenter(), {}, SpawnParameters);
	if (LocationVolume)
	{
		LocationVolume->SetActorLabel(Label);
		LocationVolume->GetRootComponent()->SetMobility(EComponentMobility::Movable);
		LocationVolume->AttachToActor(MegaMesh, FAttachmentTransformRules::KeepWorldTransform);
		LocationVolume->SetActorScale3D(Bounds.GetSize());
		LocationVolume->Tags.Add(FName("MeshTerrainLocationVolume"));

		UCubeBuilder* Builder = NewObject<UCubeBuilder>();
		Builder->X = 1.0f;
		Builder->Y = 1.0f;
		Builder->Z = 1.0f;
		UActorFactory::CreateBrushForVolumeActor(LocationVolume, Builder);
	}

	return LocationVolume;
}

FHeightmapImporter::FHeightmapImporter(const FHeightmapImportParams& InParams)
	: Params(InParams)
{
	// Disable save and unload and creation of location volumes if the world is not a world partitioned world.
	Params.bSaveAndUnload &= Params.World->GetWorldPartition() != nullptr;
	Params.LocationVolumesResolution = Params.World->GetWorldPartition() != nullptr ? Params.LocationVolumesResolution : FInt32Point(0);
}

TArray<FSectionInfo> FHeightmapImporter::GetSectionInfos(const FHeightmapImportParams& Params)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::GetSectionInfos);

	const int32 NumSections = Params.SectionsResolution.X * Params.SectionsResolution.Y;

	TArray<FSectionInfo> SectionInfos;
	SectionInfos.Reserve(NumSections);

	const FInt32Point MaxSectionResolution = FMath::DivideAndRoundUp(Params.MeshResolution, Params.SectionsResolution);
	ensure(MaxSectionResolution.X > 0 && MaxSectionResolution.Y > 0);

	FInt32Point SectionStart = {0};
	FInt32Point RemainingMeshResolution = Params.MeshResolution;
	FInt32Point SectionResolution = {0};

	for (int32 SectionY = 0; SectionY < Params.SectionsResolution.Y; ++SectionY)
	{
		SectionResolution.Y = FMath::Min(MaxSectionResolution.Y, RemainingMeshResolution.Y);
		RemainingMeshResolution.Y -= SectionResolution.Y;
		RemainingMeshResolution.X = Params.MeshResolution.X;

		SectionStart.X = 0;

		for (int32 SectionX = 0; SectionX < Params.SectionsResolution.X; ++SectionX)
		{
			SectionResolution.X = FMath::Min(MaxSectionResolution.X, RemainingMeshResolution.X);
			RemainingMeshResolution.X -= SectionResolution.X;

			const FInt32Point SectionEnd = SectionStart + SectionResolution;

			const FVector2d MinUV = {static_cast<double>(SectionStart.X) / Params.MeshResolution.X, static_cast<double>(SectionStart.Y) / Params.MeshResolution.Y};
			const FVector2d MaxUV = {static_cast<double>(SectionEnd.X) / Params.MeshResolution.X, static_cast<double>(SectionEnd.Y) / Params.MeshResolution.Y};

			SectionInfos.Add({{SectionX, SectionY}, SectionResolution, MinUV, MaxUV});
			SectionStart.X += SectionResolution.X;
		}
		SectionStart.Y += SectionResolution.Y;
	}

#if UE_BUILD_DEBUG
	int64 NumQuads = 0;
	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		const FSectionInfo& SectionInfo = SectionInfos[SectionIndex];
		NumQuads += SectionInfo.Resolution.X * SectionInfo.Resolution.Y;
	}
	ensure(NumQuads == Params.MeshResolution.X * Params.MeshResolution.Y);
#endif

	return SectionInfos;
}

bool FHeightmapImporter::BeginLoadHeightmapFile()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::BeginLoadHeightmapFile);

	TArray64<uint8> TempData;
	if (!FFileHelper::LoadFileToArray(TempData, *Params.HeightmapFilename))
	{
		ErrorText = LOCTEXT("HeightmapFileLoadFail_FileNotFound", "Failed to load heightmap from file. Could not find the heightmap file.");
		return false;
	}

	IImageWrapperModule& ImageWrapperModule = FModuleManager::LoadModuleChecked<IImageWrapperModule>("ImageWrapper");
	TSharedPtr<IImageWrapper> ImageWrapper = ImageWrapperModule.CreateImageWrapper(EImageFormat::PNG);

	if (!ImageWrapper->SetCompressed(TempData.GetData(), TempData.Num()))
	{
		ErrorText = LOCTEXT("HeightmapFileLoadFail_BadPNG", "Heightmap data is in not a correctly formatted PNG file");
		return false;
	}

	HeightmapResolution.X = ImageWrapper->GetWidth();
	HeightmapResolution.Y = ImageWrapper->GetHeight();

	LoadHeightmapTask = Tasks::Launch(TEXT("Decompress Heightmap Data"),
		[ImageWrapper, HeightmapResolution = HeightmapResolution]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UHeightmapImportTool::ImportHeightmap);

			TArray64<uint16> Result;
			TArray64<uint8> RawData;
			if (ImageWrapper->GetBitDepth() == 16)
			{
				if (ensure(ImageWrapper->GetRaw(ERGBFormat::Gray, 16, RawData)))
				{
					Result.SetNumUninitialized(HeightmapResolution.X * HeightmapResolution.Y);
					FMemory::Memcpy(Result.GetData(), RawData.GetData(), HeightmapResolution.X * HeightmapResolution.Y * 2);
				}
			}
			else if (ImageWrapper->GetBitDepth() == 8)
			{
				if (ensure(ImageWrapper->GetRaw(ERGBFormat::Gray, 8, RawData)))
				{
					Result.Reserve(HeightmapResolution.X * HeightmapResolution.Y);
					Algo::Transform(RawData, Result, [](uint8 Value) { return static_cast<uint16>(Value * 0x101); }); // Expand to 16-bit
				}
			}
			else
			{
				ensure(false);
			}
			return MoveTemp(Result);
		});

	return true;
}

bool FHeightmapImporter::Import(FPackageSourceControlHelper* InSourceControlHelper)
{
	if (Params.HeightmapFilename.IsEmpty())
	{
		ErrorText = LOCTEXT("MissingHeightmapFile", "No heightmap file provided!");
		return false;
	}
	if (!Params.HeightmapFilename.EndsWith(".png"))
	{
		ErrorText = LOCTEXT("HeightmapNotPNGFormat", "Heightmap import only supports .png files!");
		return false;
	}

	if (Params.SectionsResolution.X <= 0 || Params.SectionsResolution.Y <= 0)
	{
		ErrorText = LOCTEXT("SectionsResolutionInvalid", "The number of sections in XY dimension needs to be greater than zero!");
		return false;
	}

	if (Params.SectionsResolution.X * Params.SectionsResolution.Y > 4096)
	{
		ErrorText = LOCTEXT("SectionsResolutionTooBig", "The total number of sections is more than the maximum 4096!");
		return false;
	}

	if (Params.SectionsResolution.X > Params.MeshResolution.X || Params.SectionsResolution.Y > Params.MeshResolution.Y)
	{
		ErrorText = LOCTEXT("SectionsResolutionLargerThanMeshResolution", "The section resolution must not be larger than the mesh resolution!");
		return false;
	}

	if (Params.LocationVolumesResolution.X < 0 || Params.LocationVolumesResolution.Y < 0)
	{
		ErrorText = LOCTEXT("LocationVolumesResolutionInvalid", "The number of location volumes in XY dimension needs to not be negative!");
		return false;
	}

	if (Params.LocationVolumesResolution.X * Params.LocationVolumesResolution.Y > 256)
	{
		ErrorText = LOCTEXT("LocationVolumesResolutionTooBig", "The total number of location volumes needs to be not greater than 256!");
		return false;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::Import);

	if (!BeginLoadHeightmapFile())
	{
		return false;
	}

	const int32 NumSections = Params.SectionsResolution.X * Params.SectionsResolution.Y;
	const int32 NumLocationVolumes = Params.LocationVolumesResolution.X * Params.LocationVolumesResolution.Y;

	FScopedSlowTask SlowTask(NumSections + NumLocationVolumes, LOCTEXT("HeightmapImport_SlowTask", "Importing heightmap..."));
	SlowTask.MakeDialog(true);

	UMeshPartitionEditorComponent* MegaMeshEditorComponent = nullptr;
	// Spawn and save the MegaMesh actor
	{
		MegaMesh = Params.World->SpawnActor<AMeshPartition>(AMeshPartition::StaticClass(), FTransform::Identity);
		MegaMeshEditorComponent = NewObject<UMeshPartitionEditorComponent>(MegaMesh, UMeshPartitionEditorComponent::StaticClass(), TEXT("MegaMeshEditorComponent"), RF_Transactional);
		MegaMesh->SetMeshPartitionComponent(MegaMeshEditorComponent);

		if (Params.bSaveAndUnload)
		{
			SaveActorPackages( { MegaMesh } );
		}
	}

	// Upperbound on how many bytes each section will require to avoid allocating too much memory at once
	constexpr int64 EstimatedBytesPerVertex = 350;
	constexpr float MemoryMargin = 0.1f;
	const int64 EstimatedBytesPerSection = EstimatedBytesPerVertex * Params.MeshResolution.X * Params.MeshResolution.Y / NumSections;
	const int64 AvailablePhysicalMemory = FPlatformMemory::GetStats().AvailablePhysical * (1. - MemoryMargin);
	const int64 MaxTasksInFlight = FMath::Max<int64>(AvailablePhysicalMemory / EstimatedBytesPerSection, 1);

	MegaMeshMeshProviders.Reserve(NumSections);
	MeshTasks.Reserve(MaxTasksInFlight);
	SavedPackages.Reserve(SavedPackages.Num() + NumSections);

	const TArray<FSectionInfo> SectionInfos = GetSectionInfos(Params);

	// center the entire generated mega mesh around the origin
	const FVector2d MeshWorldOffset = FVector2d{-Params.MeshSize} / 2.0;

	// disable non-dirty tracking (which normally will try to Pin newly created non-dirty actors)
	// - if we are saving and unloading, we don't want them to pin
	UWorldPartition* WorldPartition = Params.World->GetWorldPartition();
	UWorldPartition::FDisableNonDirtyActorTrackingScope DisableNonDirtyActorTrackingScope(WorldPartition, Params.bSaveAndUnload);

	// Spawn all the base modifiers ahead of time so we can operate directly on their internal FDynamicMesh to avoid copies.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::Import_SpawnBaseModifiers);

		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			SlowTask.TickProgress();

			const FSectionInfo& SectionInfo = SectionInfos[SectionIndex];

			const FVector2d SectionCenterUV = (SectionInfo.MinUV + SectionInfo.MaxUV) * 0.5;
			const FVector2d SectionCenter = MeshWorldOffset + SectionCenterUV * FVector2d(Params.MeshSize);

			const FTransform Transform = FTransform(FVector(SectionCenter.X, SectionCenter.Y, 0.0));

			// if we are unloading, don't bother registering
			const bool bRegisterModifier = !Params.bSaveAndUnload;
			AActor* BaseModifier = MegaMeshEditorComponent->SpawnBaseModifier({}, {}, Transform, bRegisterModifier);

			const FString ActorLabel = FString::Format(TEXT("Section_X{0}-Y{1}"), {SectionInfo.IndexXY.X, SectionInfo.IndexXY.Y});
			BaseModifier->SetActorLabel(ActorLabel); // NOTE: this causes re-registration as a side effect

			MeshPartition::UMeshProviderModifier* MeshProviderComponent = BaseModifier->FindComponentByClass<MeshPartition::UMeshProviderModifier>();
			MegaMeshMeshProviders.Add(MeshProviderComponent);

			// tell the preview section builder to ignore this while we build it -- we will clear at the end
			MeshProviderComponent->SetIgnoreChanged(true);
		}
	}

	if (NumLocationVolumes > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::Import_SpawnLocationVolumes);

		const FVector3d VolumeScale = {
			Params.MeshSize.X / Params.LocationVolumesResolution.X, Params.MeshSize.Y / Params.LocationVolumesResolution.Y, Params.MeshSize.Z
		};

		for (int32 VolumeY = 0; VolumeY < Params.LocationVolumesResolution.Y; ++VolumeY)
		{
			for (int32 VolumeX = 0; VolumeX < Params.LocationVolumesResolution.X; ++VolumeX)
			{
				SlowTask.TickProgress();

				const FVector3d BoundsMin = { MeshWorldOffset.X + VolumeScale.X * VolumeX, MeshWorldOffset.Y + VolumeScale.Y * VolumeY, 0.0};
				const FVector3d BoundsMax = { MeshWorldOffset.X + VolumeScale.X * (VolumeX + 1), MeshWorldOffset.Y + VolumeScale.Y * (VolumeY + 1), VolumeScale.Z};

				const FString& Label = FString::Format(TEXT("LocationVolume_X{0}-Y{1}"), {VolumeX, VolumeY});

				CreateLocationVolume(Params.World, MegaMesh, Label, {BoundsMin, BoundsMax});
			}
		}
	}

	if (SlowTask.ShouldCancel())
	{
		CancelAndWait();
		return false;
	}

	// Get a single pixel value from the heightmap.
	auto GetHeightmapValue = [](const TArray64<uint16>& Heightmap, const FInt32Point Resolution, int32 X, int32 Y) -> double
	{
		const uint64 HeightmapIndex = X + Y * Resolution.X;
		const double Height = static_cast<double>(Heightmap[HeightmapIndex]) / static_cast<double>(UINT16_MAX);
		return Height;
	};

	// Get an interpolated value from the heightmap using UV coordinates.
	auto GetHeight = [&GetHeightmapValue](const TArray64<uint16>& Heightmap, const FInt32Point Resolution, const FVector2f UV) -> double
	{
		const FVector2d XY = {UV.X * Resolution.X, UV.Y * Resolution.Y};

		const int32 X0 = FMath::Clamp<int32>(static_cast<int32>(XY.X - 0.5), 0, Resolution.X - 1);
		const int32 Y0 = FMath::Clamp<int32>(static_cast<int32>(XY.Y - 0.5), 0, Resolution.Y - 1);
		const int32 X1 = FMath::Clamp<int32>(X0 + 1, 0, Resolution.X - 1);
		const int32 Y1 = FMath::Clamp<int32>(Y0 + 1, 0, Resolution.Y - 1);

		const FVector2d D = {XY.X - (X0 + 0.5), XY.Y - (Y0 + 0.5)};

		const double H00 = GetHeightmapValue(Heightmap, Resolution, X0, Y0);
		const double H10 = GetHeightmapValue(Heightmap, Resolution, X1, Y0);
		const double H01 = GetHeightmapValue(Heightmap, Resolution, X0, Y1);
		const double H11 = GetHeightmapValue(Heightmap, Resolution, X1, Y1);

		const double H0010 = H00 * (1.0 - D.X) + H10 * D.X;
		const double H1011 = H01 * (1.0 - D.X) + H11 * D.X;

		const double Height = H0010 * (1.0 - D.Y) + H1011 * D.Y;

		return Height;
	};

	// When this many base modifier actors have finished being built, we save them all simultaneously and then unload them.
	constexpr uint32 NumActorsBeforeSave = 32;

	TArray<TWeakObjectPtr<AActor>> ActorsToSave;
	ActorsToSave.Reserve(NumActorsBeforeSave);

	Meshes.SetNum(NumSections);

	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		if (SlowTask.ShouldCancel())
		{
			CancelAndWait();
			return false;
		}

		const FSectionInfo& SectionInfo = SectionInfos[SectionIndex];

		Tasks::FTask BuildSectionMeshTask = Tasks::Launch(TEXT("Build Section Mesh"),
			[&bIsCancelled = bIsCancelled, &LoadHeightmapTask = LoadHeightmapTask, HeightmapResolution = HeightmapResolution,
				Mesh = &Meshes[SectionIndex], &GetHeight, &SectionInfo, MeshSize = Params.MeshSize]
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::Import_BuildSectionMesh);

				if (bIsCancelled)
				{
					return;
				}

				Mesh->EnableVertexUVs(FVector2f(0, 0));
				Mesh->EnableAttributes();
				Geometry::FDynamicMeshUVOverlay* UVOverlay = Mesh->Attributes()->PrimaryUV();

				const FVector2d SectionScale = (SectionInfo.MaxUV - SectionInfo.MinUV) * FVector2d{MeshSize};

				const FVector2d V00 = {-SectionScale.X / 2.f, -SectionScale.Y / 2.f};
				const FVector2d V01 = {SectionScale.X / 2.f, -SectionScale.Y / 2.f};
				const FVector2d V11 = {SectionScale.X / 2.f, SectionScale.Y / 2.f};
				const FVector2d V10 = {-SectionScale.X / 2.f, SectionScale.Y / 2.f};

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::Import_BuildSectionMesh_AppendVertices);

					const FVector2f UV00(SectionInfo.MinUV);
					const FVector2f UV01(SectionInfo.MinUV.X, SectionInfo.MaxUV.Y);
					const FVector2f UV11(SectionInfo.MaxUV);
					const FVector2f UV10(SectionInfo.MaxUV.X, SectionInfo.MinUV.Y);

					const TArray64<uint16>& HeightmapData = LoadHeightmapTask.GetResult();

					Mesh->BeginUnsafeVerticesInsert();
					int32 VertexId = 0;
					for (int32 IndexY = 0; IndexY <= SectionInfo.Resolution.Y; ++IndexY)
					{
						const double Yt = static_cast<double>(IndexY) / static_cast<double>(SectionInfo.Resolution.Y);
						for (int32 IndexX = 0; IndexX <= SectionInfo.Resolution.X; ++IndexX)
						{
							const double Xt = static_cast<double>(IndexX) / static_cast<double>(SectionInfo.Resolution.X);

							const FVector2d VertexXY = Geometry::FMeshShapeGenerator::BilinearInterp(V00, V01, V11, V10, Xt, Yt);
							const FVector2f UV = Geometry::FMeshShapeGenerator::BilinearInterp(UV00, UV10, UV11, UV01, Xt, Yt);
							const double Height = GetHeight(HeightmapData, HeightmapResolution, UV);

							Geometry::FVertexInfo VertexInfo({VertexXY.X, VertexXY.Y, Height * MeshSize.Z});
							Mesh->InsertVertex(VertexId, VertexInfo, true);
							UVOverlay->AppendElement(UV);
							Mesh->SetVertexUV(VertexId++, UV);
						}
					}
					Mesh->EndUnsafeVerticesInsert();
				}

				{
					TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::Import_BuildSectionMesh_AppendTriangles);
					Mesh->BeginUnsafeTrianglesInsert();
					int32 TriangleId = 0;
					for (int32 IndexY = 0; IndexY < SectionInfo.Resolution.Y; ++IndexY)
					{
						for (int32 IndexX = 0; IndexX < SectionInfo.Resolution.X; ++IndexX)
						{
							const int32 Quad00 = IndexY * (SectionInfo.Resolution.X + 1) + IndexX;
							const int32 Quad10 = (IndexY + 1) * (SectionInfo.Resolution.X + 1) + IndexX;
							const int32 Quad01 = Quad00 + 1;
							const int32 Quad11 = Quad10 + 1;
							const Geometry::FIndex3i Triangle0(Quad00, Quad11, Quad01);
							const Geometry::FIndex3i Triangle1(Quad00, Quad10, Quad11);

							Mesh->InsertTriangle(TriangleId, Triangle0, 0, true);
							UVOverlay->SetTriangle(TriangleId++, Geometry::FIndex3i(Quad00, Quad11, Quad01));

							Mesh->InsertTriangle(TriangleId, Triangle1, 0, true);
							UVOverlay->SetTriangle(TriangleId++, Geometry::FIndex3i(Quad00, Quad10, Quad11));
						}
					}
					Mesh->EndUnsafeTrianglesInsert();
				}

				Mesh->EnableAttributes();
				Geometry::FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->GetNormalLayer(0);
				Geometry::FMeshNormals::InitializeOverlayToPerVertexNormals(NormalOverlay);
			}, LoadHeightmapTask);

		MeshTasks.Add({SectionIndex, BuildSectionMeshTask});

		const bool bHasMoreWork = SectionIndex != NumSections - 1;
		// If we've reached the max concurrency, start processing results to keep memory consumption down:
		while ((bHasMoreWork && MeshTasks.Num() > MaxTasksInFlight) || (!bHasMoreWork && MeshTasks.Num() > 0))
		{
			// process any completed mesh tasks
			for (auto It = MeshTasks.CreateIterator(); It; ++It)
			{
				SlowTask.TickProgress();
				if (SlowTask.ShouldCancel())
				{
					CancelAndWait();
					return false;
				}

				if (!It->Value.IsCompleted())
				{
					continue;
				}

				const int32 ProviderIndex = It->Key;
				MegaMeshMeshProviders[ProviderIndex]->SetMesh(MoveTemp(Meshes[ProviderIndex]));
				ActorsToSave.Add(MegaMeshMeshProviders[ProviderIndex]->GetOwner());

				// remove from the mesh tasks list
				It.RemoveCurrentSwap();

				// Stop popping completed tasks and launch more when we go below this percentage of MaxTasksInFlight:
				constexpr float TaskFillPercent = 0.75;
				if (bHasMoreWork && MeshTasks.Num() < MaxTasksInFlight * TaskFillPercent)
				{
					break;
				}
			}
		}

		if (!bHasMoreWork || ActorsToSave.Num() >= NumActorsBeforeSave)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::Import_SaveGeneratedSections);
			SlowTask.EnterProgressFrame(ActorsToSave.Num());

			if (Params.bSaveAndUnload)
			{
				SaveActorPackages(ActorsToSave);
				UnloadActors(ActorsToSave);
			}

			ActorsToSave.Reset();
		}
	}

	if (InSourceControlHelper && Params.bSaveAndUnload)
	{
		[[maybe_unused]] bool bAddedToSourceControl = InSourceControlHelper->AddToSourceControl(SavedPackages);
	}

	// If save and unload is disabled, the ignore changed flag must be reset and bounds must be updated to trigger preview mesh generation:
	if (!Params.bSaveAndUnload)
	{
		TArray<FBox> Bounds;
		for (MeshPartition::UMeshProviderModifier* MeshProvider : MegaMeshMeshProviders)
		{
			MeshProvider->SetIgnoreChanged(false);
			Bounds.Append(MeshProvider->ComputeBounds());
		}
		MegaMeshEditorComponent->OnBoundsChanged(Bounds, EChangeType::StateChange);
	}

	return true;
}

void FHeightmapImporter::SaveActorPackages(const TArray<TWeakObjectPtr<AActor>>& InActorsToSave)
{
	if (!ensure(Params.bSaveAndUnload) || InActorsToSave.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::SaveActorPackages);

	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;

	TArray<FPackageSaveInfo> PackageSaveInfos;
	TArray<FSavePackageResultStruct> SaveResults;
	PackageSaveInfos.Reserve(InActorsToSave.Num());
	SaveResults.Reserve(InActorsToSave.Num());

	for (const TWeakObjectPtr<AActor>& Actor : InActorsToSave)
	{
		UPackage* PackageToSave = Actor->GetPackage();
		FPackageSaveInfo SaveInfo;
		SaveInfo.Asset = nullptr;
		SaveInfo.Package = PackageToSave;
		SaveInfo.Filename = SourceControlHelpers::PackageFilename(PackageToSave);
		PackageSaveInfos.Add(SaveInfo);
	}

	UPackage::SaveConcurrent(PackageSaveInfos, SaveArgs, SaveResults);

	for (const TWeakObjectPtr<AActor>& Actor : InActorsToSave)
	{
		SavedPackages.Add(SourceControlHelpers::PackageFilename(Actor->GetPackage()));
	}
}

void FHeightmapImporter::UnloadActors(const TArray<TWeakObjectPtr<AActor>>& InActorsToUnload) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::UnloadActors);

	UWorldPartition* WorldPartition = Params.World->GetWorldPartition();

	// just make sure the object has no standalone flags, then it should get garbage collected
	for (const TWeakObjectPtr<AActor>& ActorWeakPtr : InActorsToUnload)
	{
		AActor* Actor = ActorWeakPtr.Get();
		if (Actor == nullptr)
		{
			continue; // already unloaded
		}

		// unpin if necessary
		if (!ensure(!WorldPartition->IsActorPinned(Actor->GetActorGuid())))
		{
			WorldPartition->UnpinActors( { Actor->GetActorGuid() });
		}

		ForEachObjectWithPackage(Actor->GetPackage(), [](UObject* Object)
		{
			Object->ClearFlags(RF_Standalone);
			return true;
		});
	}

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);

	// Double check that they were actually unloaded
	EditorUtils::ValidateObjectsAreUnloaded(InActorsToUnload);
}

void FHeightmapImporter::CancelAndWait()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FHeightmapImporter::CancelAndWait);

	bIsCancelled = true;

	// Outstanding tasks may try to reference the mesh provider components directly, so we have to wait here until all outstanding tasks join before we can clean up any uobjects:
	for (const TPair<int, Tasks::FTask>& TaskInFlight : MeshTasks)
	{
		[[maybe_unused]] bool bCompleted = TaskInFlight.Value.Wait();
	}

	// Destroy any mesh providers they haven't already been saved
	{
		// Using TWeakObjectPtr so that we can double check that Actors were unloaded after GC
		TArray<TWeakObjectPtr<AActor>> ActorsToUnload;
		ActorsToUnload.Reserve(MegaMeshMeshProviders.Num());
		
		Algo::TransformIf(MegaMeshMeshProviders, ActorsToUnload,
			[] (const MeshPartition::UMeshProviderModifier* MeshProvider)
			{
				return MeshProvider != nullptr;
			},
			[] (const MeshPartition::UMeshProviderModifier* MeshProvider)
			{
				return MeshProvider->GetOwner();
			});

		UnloadActors(ActorsToUnload);
	}

	// Destroy the parent mega mesh actor after all loaded sections are destroyed and GC'd
	{
		// It's important here to destroy -> save -> GC in order for WP to pick up that this is now an empty package and remove the actor desc:
		// (note this is different from what UnloadActors does)

		Params.World->DestroyActor(MegaMesh);

		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		UPackage::Save(MegaMesh->GetPackage(), nullptr, *USourceControlHelpers::PackageFilename(MegaMesh->GetPackage()), SaveArgs);

		ForEachObjectWithPackage(MegaMesh->GetPackage(), [](UObject* Object)
		{
			Object->ClearFlags(RF_Standalone);
			return true;
		});

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
	}

	// Finally, delete all the saved OFPA actor files directly off the disk so we don't need to load them in the editor which would be a very slow operation
	for (FString SavedPackage : SavedPackages)
	{
		IFileManager::Get().Delete(*SavedPackage, false, true);
	}

	SavedPackages.Empty();
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE