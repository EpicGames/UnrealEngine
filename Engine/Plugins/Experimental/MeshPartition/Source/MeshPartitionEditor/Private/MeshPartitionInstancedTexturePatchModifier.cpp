// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionInstancedTexturePatchModifier.h"
#include "Modifiers/Ops/MeshPartitionTexturePatchOp.h"

#include "Curves/CurveFloat.h"

namespace UE::MeshPartition
{
	
UInstancedTexturePatchModifier::UInstancedTexturePatchModifier()
{
}

void UInstancedTexturePatchModifier::SetDisabledByCode(bool bInDisabledByCode)
{
	if (bDisabledByCode == bInDisabledByCode)
	{
		return;
	}
	bDisabledByCode = bInDisabledByCode;

	// ComputeBounds() will give us empty bounds once we're disabled.
	OnChanged(ComputeBounds(), bInDisabledByCode ? EChangeType::TransientStateChange : EChangeType::StateChange);
}

void UInstancedTexturePatchModifier::ResetForReuse()
{
	ClearInstances();
}

bool UInstancedTexturePatchModifier::IsUsed() const
{
	return NumInstances() != 0;
}

TArray<FBox> UInstancedTexturePatchModifier::ComputeBounds() const
{
	TArray<FBox> BoundingBoxes;
	for (int32 Index = 0; Index < Instances.Num(); ++Index)
	{
		BoundingBoxes.Emplace(GetInstanceWorldspaceBounds(Index));
	}
	return BoundingBoxes;
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UInstancedTexturePatchModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	if (!HasScalarField() || Instances.Num() == 0)
	{
		return nullptr;
	}

	TSharedPtr<FTexturePatchModifierOp> Op = AllocateBackgroundOp<FTexturePatchModifierOp>(GetFName());

	FTransform BaseTransform = GetComponentTransform();
	if (!bApplyComponentZScale)
	{
		FVector3d Scale = BaseTransform.GetScale3D();
		Scale.Z = 1.0;
		BaseTransform.SetScale3D(Scale);
	}
	
	Op->GlobalBounds.SetNum(Instances.Num());
	Op->PatchTransforms.SetNum(Instances.Num());
	for (int32 InstanceID = 0; InstanceID < Instances.Num(); ++InstanceID)
	{
		Op->GlobalBounds[InstanceID] = GetInstanceWorldspaceBounds(InstanceID);
		Op->PatchTransforms[InstanceID] = BaseTransform * Instances[InstanceID];
	}

	Op->UnscaledPatchCoverage = UnscaledPatchCoverage;
	Op->VerticalPatchExtentUp = VerticalPatchExtentUp;
	Op->VerticalPatchExtentDown = VerticalPatchExtentDown;
	Op->TessellationMode = TessellationMode;
	Op->TessellationErrorMode = TessellationErrorMode;
	Op->TessellationError = TessellationError;
	Op->MinimumEdgeLength = MinimumEdgeLength;
	Op->MaximumEdgeLength = MaximumEdgeLength;
	Op->FeatureSensitivity = TessellationFeatureSensitivity;
	Op->bMeshRegularization = bTessellationMeshRegularization;
	Op->DetailAdjustmentChannelName = DetailAdjustmentChannelName;

	// Helper to avoid duplicating copied curve data across weight/height entries.
	// TODO: Could have a cache to share across modifiers, but might not be worth it
	TMap<UCurveFloat*, TSharedPtr<const FRichCurve>> CopiedCurves;
	auto CopyCurve = [&CopiedCurves](UCurveFloat* CurveIn) -> TSharedPtr<const FRichCurve>
	{
		if (!CurveIn)
		{
			return nullptr;
		}

		if (const TSharedPtr<const FRichCurve>* Existing = CopiedCurves.Find(CurveIn))
		{
			return *Existing;
		}

		TSharedPtr<const FRichCurve> CopiedCurve = MakeShared<FRichCurve>(CurveIn->FloatCurve);
		CopiedCurves.Add(CurveIn, CopiedCurve);
		
		return CopiedCurve;
	};

	auto UpdateOpTextureEntry = [this, &CopyCurve](const UTexturePatchEntry& Source, FTexturePatchModifierOp::FTextureEntry& Dest)
	{
		Dest.BlendMode = Source.BlendMode;
		Dest.SoftnessParameter = Source.SoftnessParameter;
		Dest.AlphaMode = Source.AlphaMode;
		Dest.SelfMaskTolerance = Source.SelfMaskTolerance;

		if (Source.bUseValueCurve)
		{
			Dest.ValueCurve = CopyCurve(Source.ValueCurve);
			Dest.ValueCurveOffset = Source.ValueCurveOffset;
			Dest.ValueCurveScale = Source.ValueCurveScale;
		}

		Dest.FalloffMode = Source.FalloffMode;
		Dest.FalloffDistance = Source.FalloffDistance;
		Dest.CornerRadius = Source.CornerRadius;
		if (Source.FalloffMode == MeshPartition::ETexturePatchFalloffMode::CustomCurve)
		{
			Dest.FalloffCurve = CopyCurve(Source.FalloffCurve);
		}
	};

	if (HeightChannel && HeightChannelField)
	{
		Op->HeightChannel.Emplace(HeightChannelField.ToSharedRef(), FilteredDisplacementMap.ToSharedRef());
		UpdateOpTextureEntry(*HeightChannel, *Op->HeightChannel);
		Op->HeightChannel->ZeroInEncoding = HeightChannel->ZeroInEncoding;
		Op->HeightChannel->EncodingScale = HeightChannel->EncodingScale;
		Op->HeightChannel->HeightScaleWeightChannel = HeightChannel->HeightScaleWeightChannel.GetName();
	}
	for (int32 Index = 0, Num = FMath::Min(WeightChannels.Num(), WeightChannelFields.Num()); Index < Num; ++Index)
	{
		const TObjectPtr<MeshPartition::UTexturePatchWeightEntry>& WeightChannelEntry = WeightChannels[Index];
		const TSharedPtr<const FChannelFieldData>& WeightChannelField = WeightChannelFields[Index];

		if (!WeightChannelEntry || !WeightChannelField)
		{
			continue;
		}
		FTexturePatchModifierOp::FWeightEntry& Entry = Op->WeightChannels.Emplace_GetRef(WeightChannelFields[Index].ToSharedRef());
		UpdateOpTextureEntry(*WeightChannelEntry, Entry);
		Entry.WeightChannelName = WeightChannelEntry->WeightChannelName;
		Entry.bApplyHeightMinMaxBlend = WeightChannelEntry->bApplyHeightMinMaxBlend;
		Entry.HeightMinMaxBlendDistance = WeightChannelEntry->HeightMinMaxBlendDistance;
		Entry.ZeroInEncoding = WeightChannelEntry->ZeroInEncoding;
		Entry.EncodingScale = WeightChannelEntry->EncodingScale;
	}

	return Op;
}

FGuid UInstancedTexturePatchModifier::GetCodeVersionKey() const
{
	return FTexturePatchModifierOp::GetCodeVersionKey();
}

bool UInstancedTexturePatchModifier::IsTemporarilyDisabledInEditor() const
{
	return Super::IsTemporarilyDisabledInEditor() || bDisabledByCode;
}

void UInstancedTexturePatchModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
}

void UInstancedTexturePatchModifier::AddInstances(const TArray<FVector>& InNewInstances)
{
	Instances.Reserve(Instances.Num() + InNewInstances.Num());

	TArray<FBox> NewBounds;
	for (const FVector& InstanceLocation : InNewInstances)
	{
		const int InstanceID = Instances.Emplace(InstanceLocation);
		NewBounds.Emplace(GetInstanceWorldspaceBounds(InstanceID));
	}
	OnChanged(NewBounds, EChangeType::StateChange);
}

void UInstancedTexturePatchModifier::ClearInstances()
{
	Instances.Empty();
}

FTransform UInstancedTexturePatchModifier::GetInstanceWorldspaceTransform(int InInstanceID) const
{
	const bool bValidInstanceID = Instances.IsValidIndex(InInstanceID);
	
	if (!ensure(bValidInstanceID))
	{
		return FTransform::Identity;
	}

	return GetComponentTransform() * Instances[InInstanceID];
}
FBox UInstancedTexturePatchModifier::GetInstanceWorldspaceBounds(int InInstanceID) const
{
	if (!ensure(Instances.IsValidIndex(InInstanceID)))
	{
		return {};
	}

	FVector2d Coverage = GetUnscaledCoverage();

	const float Padding = 0.2f;

	Geometry::FAxisAlignedBox3d LocalBounds(
		FVector3d(-Coverage / 2. * (1. + Padding), -VerticalPatchExtentDown),
		FVector3d( Coverage / 2. * (1. + Padding),  VerticalPatchExtentUp));

	FTransform PatchToWorld = GetComponentTransform() * Instances[InInstanceID];
	if (!bApplyComponentZScale)
	{
		FVector3d Scale = PatchToWorld.GetScale3D();
		Scale.Z = 1.0;
		PatchToWorld.SetScale3D(Scale);
	}

	FBox GlobalBounds;
	for (int i = 0; i < 8; ++i)
	{
		GlobalBounds += PatchToWorld.TransformPosition(LocalBounds.GetCorner(i));
	}

	return { GlobalBounds };
}


void UInstancedTexturePatchModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	Super::GatherDependencies(Dependencies);

	Dependencies += Instances;
}

}