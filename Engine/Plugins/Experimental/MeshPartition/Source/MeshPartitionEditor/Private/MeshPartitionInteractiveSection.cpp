// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPartitionInteractiveSection.h"

#include "MeshPartition.h"
#include "Engine/World.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionPreviewComponents.h"
#include "MeshPartitionModifierComponent.h"

static TAutoConsoleVariable<float> CVarInteractiveSimplifierEdgeLength(TEXT("MegaMesh.InteractiveMode.SimplifierEdgeLength"),
													    			   20,
													    			   TEXT("Edge length constraint passed down to the mesh simplifier for interactive mode."));

static TAutoConsoleVariable<int32> CVarInteractiveSimplifierMinVertexNumber(TEXT("MegaMesh.InteractiveMode.SimplifierMinVertexNumber"),
																			1000000,
																			TEXT("If vertex count is below this number, the mesh will not be simplified when entering interactive mode."));

namespace UE::MeshPartition
{
AInteractiveSection::AInteractiveSection()
{
#if WITH_EDITORONLY_DATA
	bLockLocation = true;
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = true;
	PreviewMeshComponent = CreateDefaultSubobject<UPreviewMeshComponent>(TEXT("PreviewMeshComponent"));
	RootComponent = PreviewMeshComponent;
	
	PreviewMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	State = EInteractiveSectionState::Initialized;
}

void AInteractiveSection::Tick(float InDeltaSeconds)
{
	Super::Tick(InDeltaSeconds);

	Update();
}

void AInteractiveSection::Update()
{
	switch (State)
	{
		case EInteractiveSectionState::BuildingBase:
			UpdateBuildingBase();
			break;
		case EInteractiveSectionState::PreparingBase:
			UpdatePreparingBase();
			break;
		case EInteractiveSectionState::BuildingPreviewMesh:
			UpdateBuildingPreviewMesh();
			break;
		default:
			break;
	}
}

void AInteractiveSection::SetParent(AMeshPartition* InMegaMesh)
{
	Modify(true);
	DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
	Parent = InMegaMesh;
	
	// Note: If Parent is valid, it should always have a valid TransientSectionAttachAnchor (as assigned in Parent's constructor)
	if (Parent.IsValid() && ensure(Parent->GetTransientSectionAttachAnchor()))
	{
		AttachToComponent(Parent->GetTransientSectionAttachAnchor(), FAttachmentTransformRules::SnapToTargetIncludingScale);
	}
}

UMeshPartitionEditorComponent* AInteractiveSection::GetMegaMeshEditorComponent() const
{
	return Cast<UMeshPartitionEditorComponent>(Parent->GetMeshPartitionComponent());
}

void AInteractiveSection::SetInteractiveModifiers(const TArray<MeshPartition::UModifierComponent*>& InModifiers, MeshPartition::FBuilderSettings&& InSettings)
{
	ensure(State == EInteractiveSectionState::Initialized);

	BuildSettings = MoveTemp(InSettings);
	BuildSettings.SimplifierOptions.Emplace( MeshPartition::FBuilderSettings::FSimplifierOptions{
		CVarInteractiveSimplifierEdgeLength.GetValueOnGameThread(),
		CVarInteractiveSimplifierMinVertexNumber.GetValueOnGameThread()
	});

	TRACE_BOOKMARK(TEXT("Start MegaMesh Interactive base build"));
	BaseBuildTasks = UE::MeshPartition::Build::LaunchBuilds(BuildSettings);

	check(BaseBuildTasks.Num() == 1);

	BuildSettings.ModifiersToProcess.RemoveAll([](const MeshPartition::UModifierComponent* Modifier)
	{
	return !Modifier->IsBase();
	});

	for (MeshPartition::UModifierComponent* Modifier : InModifiers)
	{
		if (Modifier->IsBase())
		{
			continue;
		}

		TArray<MeshPartition::UModifierComponent*> InteractiveProxies = Modifier->GetInteractiveProxies();
	
		if (InteractiveProxies.IsEmpty())
		{
			BuildSettings.ModifiersToProcess.Emplace(Modifier);
		}
		else
		{
			BuildSettings.ModifiersToProcess.Append(InteractiveProxies);
		}

		constexpr bool bIsInteractive = true;
		Modifier->SetIsInteractive(bIsInteractive);
		InteractiveModifiers.Emplace(Modifier);
	}

	// Reset build settings which only apply to the interactive base:
	BuildSettings.ModifierFilter = nullptr;
	BuildSettings.BuildType = MeshPartition::EBuildType::InteractiveModifier;
	BuildSettings.SimplifierOptions.Reset();
	BuildSettings.bCacheResult = false;

	UMeshPartitionEditorSubsystem* Subsystem = UMeshPartitionEditorSubsystem::Get();
	UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(Parent->GetMeshPartitionComponent());

	if (ensure(Subsystem && EditorComponent))
	{
		constexpr bool bInIsBuilding = true;
		Subsystem->SetInteractiveSectionBuild(EditorComponent, bInIsBuilding);
	}

	State = EInteractiveSectionState::BuildingBase;
}

void AInteractiveSection::ClearInteractiveModifiers()
{
	for (MeshPartition::UModifierComponent* InteractiveModifier : InteractiveModifiers)
	{
		if (InteractiveModifier != nullptr)
		{
			constexpr bool bIsInteractive = false;
			InteractiveModifier->SetIsInteractive(bIsInteractive);
		}
	}
	
	InteractiveModifiers.Empty();

	for (MeshPartition::FBuildTaskHandle& BaseBuildTask : BaseBuildTasks)
	{
		BaseBuildTask.Cancel();
	}

	BaseBuildTasks.Empty();

	for (MeshPartition::FBuildTaskHandle& PreviewBuildTask : PreviewBuildTasks)
	{
		PreviewBuildTask.Cancel();
	}

	PreviewBuildTasks.Empty();

	SetIsTemporarilyHiddenInEditor(true);
	
	State = EInteractiveSectionState::Initialized;
}

void AInteractiveSection::OnModifierChanged()
{
	if (State != EInteractiveSectionState::BasePrepared)
	{
		// Tick may be disabled when property updates are interactive. Letting us a chance to complete async jobs here.
		Update();
		bPendingBuildIsOutdated = true;
		return;
	}

	if (CopyTask.IsValid() && !CopyTask.IsCompleted())
	{
		return;
	}

	CopyTask = Tasks::Launch(TEXT("AInteractiveSection_CopyTask"),
								[&CopySource = GetCurrentCopySource(), &CopyDestination = GetCurrentCopyDestination()]()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(AInteractiveSection::Copy_Task);
		CopyDestination = CopySource;
	});

	BuildSettings.BaseMesh = &GetCurrentSourceBase();
	PreviewBuildTasks = UE::MeshPartition::Build::LaunchBuilds(BuildSettings);
	bPendingBuildIsOutdated = false;

	check(PreviewBuildTasks.Num() == 1);

	RotateBaseMeshes();
	State = EInteractiveSectionState::BuildingPreviewMesh;
}

FModifierFilterFunc AInteractiveSection::InteractiveSectionBaseFilter(const TArray<MeshPartition::UModifierComponent*>& InModifiers, TConstArrayView<FName> InTypePriorities)
{
	struct FInteractiveSectionBaseFilter
	{
		FInteractiveSectionBaseFilter(const TArray<MeshPartition::UModifierComponent*>& InModifiers, TConstArrayView<FName> InTypePriorities)
		{
			for (MeshPartition::UModifierComponent* Modifier : InModifiers)
			{
				if (Modifier->IsBase())
				{
					continue;
				}
					
				TArray<MeshPartition::UModifierComponent*> ModifiersToEvaluate = { Modifier };
				ModifiersToEvaluate.Append(Modifier->GetInteractiveProxies());
				
				for (MeshPartition::UModifierComponent* ModifierToEvaluate : ModifiersToEvaluate)
				{
					uint32 ModifierPriority = UE::MeshPartition::FilterHelpers::FindLayerPriorityIndexFromName(InTypePriorities, ModifierToEvaluate->GetType());
				
					if (ModifierPriority >= MaxPriorityIndex)
					{
						MaxPriorityIndex = ModifierPriority;
					}
				
					ModifierInstances.Emplace(*ModifierToEvaluate);
				}
			}
		}

		// Returns true for modifiers which should be kept
		bool operator()(const MeshPartition::FBuilderSettings& Settings, const MeshPartition::FModifierDesc& OtherDescriptor)
		{
			uint32 OtherPriority = UE::MeshPartition::FilterHelpers::FindLayerPriorityIndexFromName(Settings.TypePriorities, OtherDescriptor.Type);

			if (OtherDescriptor.IsBase())
			{
				return true;
			}

			// Anything above the max priority shouldn't be kept.
			if (OtherPriority > MaxPriorityIndex)
			{
				return false;
			}

			// All modifiers part of ModifierInstances are the interactive one and shouldn't be part of the "interactive base".
			for (const MeshPartition::FModifierDesc& ModifierInstance : ModifierInstances)
			{
				if (ModifierInstance.ModifierPath == OtherDescriptor.ModifierPath)
				{
					return false;
				}
			}

			return true;
		}

		TArray<MeshPartition::FModifierDesc> ModifierInstances;
		uint32 MaxPriorityIndex = 0;
	};
	
	return FInteractiveSectionBaseFilter(InModifiers, InTypePriorities);
}

void AInteractiveSection::AddModifier(MeshPartition::UModifierComponent* InModifier)
{
	InteractiveModifiers.Emplace(InModifier);
	BuildSettings.ModifiersToProcess.Emplace(InModifier);
	OnModifierChanged();
}

void AInteractiveSection::RemoveModifier(MeshPartition::UModifierComponent* InModifier)
{
	InteractiveModifiers.Remove(InModifier);
	BuildSettings.ModifiersToProcess.Remove(InModifier);
	OnModifierChanged();
}

void AInteractiveSection::UpdateBuildingBase()
{
	/* Handling this here since waiting for task completion in ClearInteractiveModifier would be costly.
	Instead we let the task complete on its own and don't mind about it until we really would need it here. */
	if (BaseSetupTask.IsValid() && !BaseSetupTask.IsCompleted())
	{
		return;
	}

	if (BaseBuildTasks.IsEmpty())
	{
		return;
	}
	
	MeshPartition::FBuildTaskHandle& BaseBuildTask = BaseBuildTasks[0];
	
	if (BaseBuildTask.GetTask().IsValid() && BaseBuildTask.IsCompleted() && !BaseBuildTask.IsCancelled())
	{
		BaseSetupTask = Tasks::Launch(TEXT("AInteractiveSection_BaseSetupTask"),
		[&BaseMeshes = BaseMeshes,
		SourceMeshPtr = BaseBuildTask.GetTask()->GetMesh()]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(AInteractiveSection::BaseSetupTask);

			BaseMeshes[0] = *SourceMeshPtr;

			BaseMeshes[1] = BaseMeshes[0];
			BaseMeshes[2] = BaseMeshes[0];
		});
	
		BaseBuildTasks.Empty();
		State = EInteractiveSectionState::PreparingBase;
	}
}

void AInteractiveSection::UpdatePreparingBase()
{
	if (BaseSetupTask.IsValid() && BaseSetupTask.IsCompleted())
	{
		PreviewMeshComponent->SetMeshData(MoveTemp(GetCurrentCopyDestination()));

		PreviewMeshComponent->SetMaterial(0, MaterialInstance);
		SetIsTemporarilyHiddenInEditor(false);

		BaseSetupTask = Tasks::FTask();

		UMeshPartitionEditorSubsystem* Subsystem = UMeshPartitionEditorSubsystem::Get();
		UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(Parent->GetMeshPartitionComponent());

		if (ensure(Subsystem && EditorComponent))
		{
			constexpr bool bInIsBuilding = false;
			Subsystem->SetInteractiveSectionBuild(EditorComponent, bInIsBuilding);
		}

		State = EInteractiveSectionState::BasePrepared;
		OnModifierChanged();
	}
}

void AInteractiveSection::UpdateBuildingPreviewMesh()
{
	const bool bIsPreviewMeshReady = !PreviewBuildTasks.IsEmpty() && PreviewBuildTasks[0].GetTask().IsValid() && PreviewBuildTasks[0].IsCompleted() && !PreviewBuildTasks[0].IsCancelled();
	const bool bIsCopyTaskDone = CopyTask.IsValid() && CopyTask.IsCompleted();

	if (bIsPreviewMeshReady && bIsCopyTaskDone)
	{
		PreviewMeshComponent->SetMeshData(MoveTemp(*PreviewBuildTasks[0].GetTask()->GetMutableMesh()));
		PreviewBuildTasks.Empty();
		CopyTask = Tasks::FTask();
		State = EInteractiveSectionState::BasePrepared;

		if (bPendingBuildIsOutdated)
		{
			OnModifierChanged();
		}
	}
}
} // namespace UE::MeshPartition