// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorComponent.h"

#include "MeshPartition.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionDefinition.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "MeshPartitionChannelCollection.h"

#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialFunctionInstance.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialFunctionMaterialLayerBlend.h"
#include "Materials/MaterialExpressionFunctionInput.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "Materials/MaterialExpressionConstant.h"
#include "Materials/MaterialExpressionConstant2Vector.h"
#include "Materials/MaterialExpressionConstant3Vector.h"
#include "Materials/MaterialExpressionConstant4Vector.h"
#include "Materials/MaterialExpressionTextureObject.h"
#include "Materials/MaterialExpressionSetMaterialAttributes.h"
#include "Materials/MaterialExpressionBlendMaterialAttributes.h"
#include "Materials/MaterialExpressionMaterialAttributeLayers.h"
#include "Materials/MaterialExpressionLayerStack.h"
#include "MeshPartitionMaterialExpressionUtils.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AssetRegistry/AssetRegistryModule.h"

#include "MeshPartitionEditorModule.h" // LogMegaMeshEditor

namespace UE::MeshPartition
{
template<class MaterialExpressionType>
MaterialExpressionType* NewExpression(UMaterial* InMaterial)
{
	MaterialExpressionType* Expression = NewObject<MaterialExpressionType>(InMaterial);
	InMaterial->GetExpressionCollection().AddExpression(Expression);

	Expression->Material = InMaterial;

	Expression->UpdateMaterialExpressionGuid(true, true);

	// use for params
	Expression->UpdateParameterGuid(true, true);
	if (Expression->HasAParameterName())
	{
		Expression->ValidateParameterName(false);
	}

	InMaterial->AddExpressionParameter(Expression, InMaterial->EditorParameters);

	Expression->MarkPackageDirty();

	return Expression;
}

UMaterialExpressionMaterialFunctionCall* AddMaterialFunction(bool IsBlend, int InIndex, UMaterialFunctionInterface* InAddedFunction, UMaterial* InParentMaterial)
{
	if (!InAddedFunction)
		return nullptr;

	UMaterialExpressionMaterialFunctionCall* Expression = NewExpression<UMaterialExpressionMaterialFunctionCall>(InParentMaterial);

	Expression->MaterialFunction = InAddedFunction;
	Expression->FunctionParameterInfo.Association = (IsBlend ? EMaterialParameterAssociation::BlendParameter : EMaterialParameterAssociation::LayerParameter);
	Expression->FunctionParameterInfo.Index = InIndex;

	InAddedFunction->GetInputsAndOutputs(Expression->FunctionInputs, Expression->FunctionOutputs);
	for (FFunctionExpressionOutput& FunctionOutput : Expression->FunctionOutputs)
	{
		Expression->Outputs.Add(FunctionOutput.Output);
	}

	Expression->UpdateFromFunctionResource();

	return Expression;
}


FFunctionExpressionInput* GetMFCInput(UMaterialExpressionMaterialFunctionCall* InFunctionCall, int32 InIndex)
{
	if (InFunctionCall && (InFunctionCall->FunctionInputs.Num() > InIndex))
	{
		return 	&InFunctionCall->FunctionInputs[InIndex];
	}
	return nullptr;
}


UMaterialFunctionInterface* CreateDefaultBlendFunc()
{
	const FString DefaultBlend = TEXT("/MeshPartition/Materials/MFB_DefaultBlend.MFB_DefaultBlend");
	// @TODO SG: We will use this asset once the LAyerStack features are available
	//const FString DefaultBlend = TEXT("/MeshPartition/Materials/MLB_DefaultBlend.MLB_DefaultBlend");
	static TObjectPtr<UMaterialFunctionInterface> DefaultBlendFunction = FindObject<UMaterialFunction>(nullptr, *(DefaultBlend));
	if (!DefaultBlendFunction)
	{
		DefaultBlendFunction = LoadObject<UMaterialFunction>(nullptr, *(DefaultBlend));
	}

	return DefaultBlendFunction;
}

namespace MegaMeshEditorComponentLocals
{
	UMaterialInstanceDynamic* CreateMaterialInstance(const UMaterialInterface* InBaseMegaMeshMaterial, UObject* InOuter, const FName& InName, EObjectFlags InAdditionalObjectFlags)
	{
		UMaterialInterface* BaseMegaMeshMaterial = const_cast<UMaterialInterface*>(InBaseMegaMeshMaterial);
		if (!IsValid(BaseMegaMeshMaterial))
		{
			BaseMegaMeshMaterial = UMaterial::GetDefaultMaterial(MD_Surface);
		}
		UMaterialInstanceDynamic* ResultMID = UMaterialInstanceDynamic::Create(BaseMegaMeshMaterial, InOuter, InName);
		ResultMID->SetFlags(InAdditionalObjectFlags);
		return	ResultMID;
	}

	UMaterialInstanceDynamic* GetOrCreateMaterialInstance(UMaterialInstanceDynamic* InMID, const UMaterialInterface* InBaseMegaMeshMaterial, UObject* InOuter, const FName& InName, EObjectFlags InAdditionalObjectFlags)
	{
		const UMaterialInterface* BaseMegaMeshMaterial = InBaseMegaMeshMaterial;

		if (IsValid(InMID) && IsValid(BaseMegaMeshMaterial) && (InMID->Parent == BaseMegaMeshMaterial))
		{
			return InMID;
		}

		return CreateMaterialInstance(BaseMegaMeshMaterial, InOuter, InName, InAdditionalObjectFlags);
	}

	UMaterialInterface* CreateDefinitionRuntimeMaterial(UObject* InOuter, MeshPartition::UMeshPartitionDefinition* InDefinition)
	{
		const FString DefaultInspectorMaterialPath = TEXT("/MeshPartition/Materials/M_MeshPartitionInspector.M_MeshPartitionInspector");
	
		static TObjectPtr<UMaterial> DefaultInspectorMaterial = FindObject<UMaterial>(nullptr, *DefaultInspectorMaterialPath);
		if (!DefaultInspectorMaterial)
		{
			DefaultInspectorMaterial = LoadObject<UMaterial>(nullptr, *DefaultInspectorMaterialPath);
		}

		UMaterialInstanceConstant* RuntimeMaterialInstance = NewObject<UMaterialInstanceConstant>(InOuter, TEXT("MeshPartitionRuntimeMaterialMainInstance"), RF_Public);
		RuntimeMaterialInstance->Parent = DefaultInspectorMaterial;
		RuntimeMaterialInstance->EnumerationObjects.Add((UObject*)InDefinition);

		RuntimeMaterialInstance->UpdateCachedData();
		
		RuntimeMaterialInstance->PreEditChange(NULL);
		RuntimeMaterialInstance->PostEditChange();

		// Assign the newly created material to the Definition
		// This will mark the Definition dirty and require to save it.
		return RuntimeMaterialInstance;


	}
}
} // namespace UE::MeshPartition
