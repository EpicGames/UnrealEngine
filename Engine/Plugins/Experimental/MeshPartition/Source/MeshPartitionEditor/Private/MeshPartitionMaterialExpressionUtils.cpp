// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMaterialExpressionUtils.h"

#include "Engine/Texture.h"
#include "Engine/TextureCollection.h"
#include "MaterialCompiler.h"
#include "Materials/MaterialInterface.h"
#include "Materials/Material.h"
#include "Materials/MaterialExpressionCustom.h"
#include "UObject/ConstructorHelpers.h"

#include "MeshPartitionDefinition.h"
#include "MeshPartitionChannel.h"
#include "MeshPartitionChannelCollection.h"

namespace UE::MeshPartition
{

// **************************************************
// UMaterialExpressionMeshPartitionResource
// **************************************************

UMaterialExpressionMeshPartitionResource::UMaterialExpressionMeshPartitionResource(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	static ConstructorHelpers::FObjectFinder<UTexture2DArray> DefaultTextureArray = TEXT("/MeshPartition/Textures/Void2DArray.Void2DArray");
	// Set up a default texture.
	// A valid texture collection is required for this expression to compile
	TextureParameter = DefaultTextureArray.Object;

#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("Channel Texture")));
#endif
}

UObject* UMaterialExpressionMeshPartitionResource::GetReferencedTexture() const
{
	return TextureParameter.IsValid() ? TextureParameter.Get() : nullptr;
}

#if WITH_EDITOR
int32 UMaterialExpressionMeshPartitionResource::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	{
		UTexture* TextureToUse = TextureParameter.Get();

		if (!TextureToUse)
		{
			return Compiler->Errorf(TEXT("Missing MeshPartition Texture."));
		}

		// Return the TextureObject (used by other expressions like samplers)
		return Compiler->TextureParameter(ChannelTextureParameterName, TextureToUse, EMaterialSamplerType::SAMPLERTYPE_Masks);
	}
}

void UMaterialExpressionMeshPartitionResource::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MeshPartition Shared Resource Param"));
	OutCaptions.Add(TEXT("MeshPartitionResource"));
}
#endif

FGuid& UMaterialExpressionMeshPartitionResource::GetParameterExpressionId()
{
	// The resource guid is constant and shared for all the mesh partition material expression instances
	static FGuid SharedGUID(0x4D656761, 0x4D657368, 0x20466F72, 0x65766572);
	return SharedGUID;
}

namespace MaterialExpression
{
#if WITH_EDITOR

	int32 CompileChannelTexcoordUV(class FMaterialCompiler* Compiler)
	{
		const int32 CHANNEL_TEXCOORD_INDEX = 0;
		return Compiler->TextureCoordinate(CHANNEL_TEXCOORD_INDEX, false, false);
	}

	int32 CompilePickChannelTexture(class FMaterialCompiler* Compiler, int32 InTextureCollection_IDX)
	{
		// The Channel Texture is bound per section
		// so this function is just a sanity check passtrough
		if (InTextureCollection_IDX >= 0)
		{
			return InTextureCollection_IDX;
		}
		else
		{
			return Compiler->Error(TEXT("Invalid default texture collection. Please assign a valid texture collection containing at least one texture."));
		}
	}
	
	int32 CompileUnpackChannelTextureSlot(class FMaterialCompiler* Compiler, int32 InChannelIndex_IDX)
	{	
		// Adjust the CustomPrimitiveData type based on the packing table num words: FChannelPacking::TableNumWords
		// default single packing for now 
		// we may have to adjust this per platform, to be determined
		EMaterialValueType ChannelTableCustomPrimitiveDataType = MCT_Float4;
	
		// Fetch the full table packed as float[N] 
		int32 ChannelTable_IDX = Compiler->CustomPrimitiveData(PrimitiveDataIndex, ChannelTableCustomPrimitiveDataType);

		// Which 32bits word in the table is ChannelIndex / WordSlotCount
		int32 ChannelTableWordIndex_IDX = Compiler->Floor(Compiler->Mul(InChannelIndex_IDX, Compiler->Constant(1.0f / FChannelPacking::WordNumSlots)));

		// Pick the 32bits word in the table
		int32 ChannelTableWord_IDX = Compiler->VectorComponent(ChannelTable_IDX, ChannelTableWordIndex_IDX);
		
		// Within that word, which slot is the reminder
		int32 ChannelTableWordSlot_IDX = Compiler->Sub(InChannelIndex_IDX, Compiler->Mul(ChannelTableWordIndex_IDX, Compiler->Constant(FChannelPacking::WordNumSlots)));

		// Unpack the channel texture slot value from the word
		int32 ChannelTextureSlot_IDX = Compiler->BitfieldExtract(ChannelTableWord_IDX,
																 Compiler->Mul(ChannelTableWordSlot_IDX, Compiler->Constant(FChannelPacking::SlotNumBits)),
																 Compiler->Constant(FChannelPacking::SlotNumBits));

		return ChannelTextureSlot_IDX;
	}

	int32 CompileFetchChannel(class FMaterialCompiler* Compiler, int32 InTextureCollection_IDX, int32 InChannelIndex_IDX)
	{
		if (InChannelIndex_IDX >= 0)
		{
			// From channel Index required, lookup in the ChannelTable the concrete slice index where to look for it
			int32 ChannelTextureSlot_IDX = CompileUnpackChannelTextureSlot(Compiler, InChannelIndex_IDX);

			// Fetch texel at the correct slice and 2d uv
			int32 TextureFromCollectionCodeIndex = CompilePickChannelTexture(Compiler, InTextureCollection_IDX);
			int32 UV_IDX = CompileChannelTexcoordUV(Compiler);
			int32 UVSlice_IDX = Compiler->AppendVector(UV_IDX, ChannelTextureSlot_IDX);
			int32 UV_ConstSlice_IDX = Compiler->AppendVector(UV_IDX, Compiler->Constant(0)); // pass a pure uv and constant slice for derivative evaluation to avoid nanite uv seams issues
			int32 FetchedValueChannel_IDX = Compiler->TextureSample(TextureFromCollectionCodeIndex, UVSlice_IDX, SAMPLERTYPE_Masks, Compiler->DDX(UV_ConstSlice_IDX), Compiler->DDY(UV_ConstSlice_IDX), TMVM_Derivative, SSM_FromTextureAsset);

			// If the channel texture slot returned is -1, it means the channel is empty in the section, exit returning 0
			// else return the fetched channel value
			return  Compiler->If(
						ChannelTextureSlot_IDX, // A is uint ChannelTextureSlot
						Compiler->Constant(FChannelPacking::SlotInvalid), // B is comparison invalid channel slot 
						Compiler->Constant(0.0f), // A > B => return 0
						Compiler->Constant(0.0f), // A == B => return 0 
						FetchedValueChannel_IDX, // A < B => return fetched value!
						Compiler->Constant(0.00001f));
		}
		else
		{
			return Compiler->Constant(0.0f);
		}
	}

#endif
}


// **************************************************
// UMaterialExpressionMeshPartitionTexcoord
// **************************************************

UMaterialExpressionMeshPartitionTexcoord::UMaterialExpressionMeshPartitionTexcoord(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	bShowOutputNameOnPin = true;

	Outputs.Reset();
	Outputs.Add(FExpressionOutput(TEXT("uv")));
	Outputs.Add(FExpressionOutput(TEXT("uv [unit]")));
#endif
}

#if WITH_EDITOR
int32 UMaterialExpressionMeshPartitionTexcoord::Compile(class FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// the channel texcoord uv
	int32 UV_IDX = MaterialExpression::CompileChannelTexcoordUV(Compiler);
	if (OutputIndex == 0)
	{
		return UV_IDX;
	}
	else
	{
		int32 UnitPerUV = Compiler->CustomPrimitiveData(4, EMaterialValueType::MCT_Float2);
		return Compiler->Mul(UV_IDX, UnitPerUV);
	}
}

void UMaterialExpressionMeshPartitionTexcoord::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Add(TEXT("MeshPartitionTexCoord"));
}
#endif

// **************************************************
// UMaterialExpressionMeshPartitionChannelSample
// **************************************************

UMaterialExpressionMeshPartitionChannelSample::UMaterialExpressionMeshPartitionChannelSample(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR
int32 UMaterialExpressionMeshPartitionChannelSample::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Make sure the texture is connected
	if (!ChannelTextureInput.IsConnected())
	{
		return Compiler->Errorf(TEXT("Texture input not connected."));
	}

	// Compile the texture object from the input
	int32 TextureCollection_IDX = ChannelTextureInput.Compile(Compiler);

	int32 ChannelIndex = FindChannelIndex(Channel);
	if (ChannelIndex == INDEX_NONE)
	{
		return Compiler->Errorf(TEXT("Invalid channel, not found in the definition."));
	}
	
	return MaterialExpression::CompileFetchChannel(Compiler, TextureCollection_IDX, Compiler->Constant(ChannelIndex));
}

EMaterialValueType UMaterialExpressionMeshPartitionChannelSample::GetInputValueType(int32 InputIndex)
{ 
	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionMeshPartitionChannelSample::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Float1;
}

void UMaterialExpressionMeshPartitionChannelSample::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Emplace(TEXT("MeshPartitionChannelSample"));
}
#endif

TArray<FName> UMaterialExpressionMeshPartitionChannelSample::GetDefinitionChannels() const
{
	const UMeshPartitionDefinition* Definition = MeshPartitionDefinition.Get();
	if (Definition)
	{
		return Definition->GetChannelMap().GetChannels();
	}

	return TArray<FName>();
}

int32 UMaterialExpressionMeshPartitionChannelSample::FindChannelIndex(FName InChannel) const
{
	const UMeshPartitionDefinition* Definition = MeshPartitionDefinition.Get();
	if (Definition)
	{
		return Definition->GetChannelMap().FindChannel(InChannel);
	}
	return INDEX_NONE;
}

// **************************************************
// UMaterialExpressionMeshPartitionChannelSampleIndex
// **************************************************

UMaterialExpressionMeshPartitionChannelSampleIndex::UMaterialExpressionMeshPartitionChannelSampleIndex(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}


#if WITH_EDITOR
int32 UMaterialExpressionMeshPartitionChannelSampleIndex::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Make sure the texture is connected
	if (!ChannelTextureInput.IsConnected())
	{
		return Compiler->Errorf(TEXT("Texture input not connected."));
	}

	// Compile the texture object from the input
	int32 TextureCollection_IDX = ChannelTextureInput.Compile(Compiler);

	int32 Arg1 = ChannelIndex.GetTracedInput().Expression ? ChannelIndex.Compile(Compiler) : Compiler->Constant(ConstChannelIndex);

	int32 FetchedValueIndex = MaterialExpression::CompileFetchChannel(Compiler, TextureCollection_IDX, Arg1);

	return FetchedValueIndex;
}

EMaterialValueType UMaterialExpressionMeshPartitionChannelSampleIndex::GetInputValueType(int32 InputIndex)
{
	return InputIndex == 1 ? MCT_UInt1 : MCT_Unknown;
}

EMaterialValueType UMaterialExpressionMeshPartitionChannelSampleIndex::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Float1;
}

void UMaterialExpressionMeshPartitionChannelSampleIndex::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Emplace(TEXT("MeshPartitionChannelSampleIndex"));
}
#endif

// **************************************************
// UMaterialExpressionMeshPartitionInspector
// **************************************************

UMaterialExpressionMeshPartitionInspector::UMaterialExpressionMeshPartitionInspector(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

int32 UMaterialExpressionMeshPartitionInspector::Compile(FMaterialCompiler* Compiler, int32 OutputIndex)
{
	// Make sure the texture is connected
	if (!ChannelTextureInput.IsConnected())
	{
		return Compiler->Errorf(TEXT("Texture input not connected."));
	}

	// Compile the texture object from the input
	int32 TextureCollection_IDX = ChannelTextureInput.Compile(Compiler);
	  
	int32 NumChannels = MeshPartition::FChannelPacking::MaxNumberPackedChannels;

	TArray<FLinearColor> ChannelColors;
	for (int32 i = 0; i < NumChannels; ++i)
	{
		ChannelColors.Add(FLinearColor::MakeRandomColor());
	}

	int32 Value_Idx = Compiler->Constant3(0, 0, 0);
	for (int32 ChannelIndex = 0; ChannelIndex < NumChannels; ++ChannelIndex)
	{
		int32 FetchedValue_Idx = MaterialExpression::CompileFetchChannel(Compiler, TextureCollection_IDX, Compiler->Constant(ChannelIndex));

		const FLinearColor& Color = ChannelColors[ChannelIndex];
		int32 ChannelColor_Idx = Compiler->Constant3(Color.R, Color.G, Color.B);

		Value_Idx = Compiler->Lerp(Value_Idx, Compiler->Constant3(Color.R, Color.G, Color.B), FetchedValue_Idx);
	}

	return Value_Idx;
}

EMaterialValueType UMaterialExpressionMeshPartitionInspector::GetInputValueType(int32 InputIndex)
{
	//return InputIndex == 1 ? MCT_UInt1 : MCT_Unknown;
	return MCT_Unknown;
}

EMaterialValueType UMaterialExpressionMeshPartitionInspector::GetOutputValueType(int32 OutputIndex)
{
	return MCT_Float3;
}

void UMaterialExpressionMeshPartitionInspector::GetCaption(TArray<FString>& OutCaptions) const
{
	OutCaptions.Emplace(TEXT("MeshPartition Channels Inspector"));
}


#endif

} // namespace UE::MeshPartition