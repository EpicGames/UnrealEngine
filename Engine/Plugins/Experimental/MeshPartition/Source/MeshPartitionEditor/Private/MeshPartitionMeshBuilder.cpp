// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMeshBuilder.h"

#include "BoxTypes.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionModifierTaskGraph.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionModifierGraphCache.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionChannelCollection.h"
#include "VEUV/VEUVTypes.h"
#include "Algo/AnyOf.h"
#include "Async/ParallelFor.h"
#include "UObject/Package.h"
#include "ProfilingDebugging/CookStats.h"
#include "Spatial/MeshAABBTree3.h"
#include "Operations/MeshClusterSimplifier.h"

#include "DerivedDataCache.h"
#include "DerivedDataCacheInterface.h"
#include "DerivedDataRequestOwner.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/MemoryReader.h"

static TAutoConsoleVariable<bool> CVarBuilderCacheEnabled(
	TEXT("MegaMesh.Cache.Enabled.Builder"),
	true,
	TEXT("Enables the MegaMesh Builder cache")
);

static TAutoConsoleVariable<float> CVarBuilderCacheMemoryBudgetMB(
	TEXT("MegaMesh.Cache.BudgetMB.Builder"),
	8.f * 1024,
	TEXT("Memory budget for data in cache (MB)."));

static TAutoConsoleVariable<float> CVarBuilderCacheMemoryCleanupRatio(
	TEXT("MegaMesh.Cache.MemoryCleanupRatio.Builder"),
	0.7f,
	TEXT("Target cache size ratio after triggering a cleanup (between 0 and 1.)."));

namespace UE::MeshPartition::FilterHelpers
{
	constexpr uint32 UngroupedLayerIndex = TNumericLimits<uint32>::Max();

	uint32 FindLayerPriorityIndexFromName(TConstArrayView<FName> InLayerStack, const FName& InLayerName)
	{
		const int64 FoundIndex = InLayerStack.Find(InLayerName);

		const uint32 LayerIndex = FoundIndex != INDEX_NONE ? static_cast<uint32>(FoundIndex) : UngroupedLayerIndex;

		return LayerIndex;
	}

	MeshPartition::FModifierFilterFunc FilterModifiersByLastLayerToBuild(const FName& InLastLayerToBuild, bool bInInclusive)
	{
		// Local copy so the lambda captures this by value instead of the ref passed to the function which will go out of scope
		const FName LastLayerToBuild = InLastLayerToBuild;
		return [LastLayerToBuild, bInInclusive](const MeshPartition::FBuilderSettings& Settings, const MeshPartition::FModifierDesc& Descriptor)
		{
			const uint32 LayerToBuildIndex = FindLayerPriorityIndexFromName(Settings.TypePriorities, LastLayerToBuild);

			const uint32 ModifierLayerPriorityIndex = FindLayerPriorityIndexFromName(Settings.TypePriorities, Descriptor.Type);

			const bool bShouldKeep = Descriptor.IsBase() || (ModifierLayerPriorityIndex < LayerToBuildIndex) || (bInInclusive && ModifierLayerPriorityIndex == LayerToBuildIndex);

			return bShouldKeep;
		};
	}

	MeshPartition::FModifierFilterFunc FilterModifiersUntilSubpriorityWithinLayer(const FName& InLastLayerToBuild, double InLastSubPriorityToBuild, bool bInInclusive)
	{
		return [InLastLayerToBuild, InLastSubPriorityToBuild, bInInclusive](const MeshPartition::FBuilderSettings& Settings, const MeshPartition::FModifierDesc& Descriptor)
		{
			const uint32 LayerToBuildIndex = FindLayerPriorityIndexFromName(Settings.TypePriorities, InLastLayerToBuild);

			const uint32 ModifierLayerPriorityIndex = FindLayerPriorityIndexFromName(Settings.TypePriorities, Descriptor.Type);

			const bool bShouldKeep = Descriptor.IsBase() 
				|| (ModifierLayerPriorityIndex < LayerToBuildIndex) 
				|| (ModifierLayerPriorityIndex == LayerToBuildIndex 
					&& (Descriptor.Priority < InLastSubPriorityToBuild 
						|| (bInInclusive && Descriptor.Priority == InLastSubPriorityToBuild)));

			return bShouldKeep;
		};
	}

	MeshPartition::FModifierFilterFunc FilterModifiersByIndexToBuild(const uint32& InIndexToBuild, bool bInInclusive)
	{
		// Local copy so the lambda captures this by value instead of the ref passed to the function which will go out of scope
		const uint32 IndexToBuild = InIndexToBuild;
		return [IndexToBuild, bInInclusive](const MeshPartition::FBuilderSettings& Settings, const MeshPartition::FModifierDesc& Descriptor)
			{
				const uint32 ModifierLayerPriorityIndex = FindLayerPriorityIndexFromName(Settings.TypePriorities, Descriptor.Type);

				const bool bShouldKeep = Descriptor.IsBase() || (ModifierLayerPriorityIndex < IndexToBuild) || (bInInclusive && ModifierLayerPriorityIndex == IndexToBuild);

				return bShouldKeep;
			};
	}

	MeshPartition::FModifierFilterFunc FilterOnlyBaseModifiers()
	{
		return [](const MeshPartition::FBuilderSettings& Settings, const MeshPartition::FModifierDesc& Descriptor)
		{
			return Descriptor.IsBase();
		};
	}

	MeshPartition::FModifierFilterFunc FilterModifiersByLastModifierToBuild(const MeshPartition::FModifierDesc& InModifierDescriptor, bool bInInclusive)
	{
		struct FLastModifierFilter
		{
			FLastModifierFilter(const MeshPartition::FModifierDesc& InModifierDescriptor, bool bInInclusive)
				: LastModifierDescriptor(InModifierDescriptor)
				, bInclusive(bInInclusive)
			{
			}

			// Returns true for modifiers which should be kept
			bool operator()(const MeshPartition::FBuilderSettings& Settings, const MeshPartition::FModifierDesc& OtherDescriptor)
			{
				if (OtherDescriptor.ModifierPath == LastModifierDescriptor.ModifierPath)
				{
					return bInclusive;
				}

				const bool bOtherDescriptorSortsBeforeLast = MeshPartition::FModifierGroup::ShouldApplyModifierBefore(Settings.TypePriorities, OtherDescriptor, LastModifierDescriptor);
				return bOtherDescriptorSortsBeforeLast;
			}

			MeshPartition::FModifierDesc LastModifierDescriptor;
			bool bInclusive;
		};
	
		return FLastModifierFilter(InModifierDescriptor, bInInclusive);
	}
}

#if ENABLE_COOK_STATS
namespace UE::MeshPartition::BuildStats
{
	static FCookStats::FDDCResourceUsageStats UsageStats;
	static FCookStatsManager::FAutoRegisterCallback RegisterCookStats([](FCookStatsManager::AddStatFuncRef AddStat)
	{
		UsageStats.LogStats(AddStat, TEXT("MegaMesh.Usage"), TEXT(""));
	});
}
#endif

namespace UE::MeshPartition::GridHelpers
{
	FVector ComputeLocalAnchor(const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld)
	{
		return InGridSettings.WorldOriginOffset - InLocalToWorld.GetTranslation();
	}

	FGridDimensions ComputeGridDimensions(const FBox& InBounds, const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld)
	{
		check(InGridSettings.CellSize > 0);

		const double CellSize = static_cast<double>(InGridSettings.CellSize);
		const FVector LocalAnchor = ComputeLocalAnchor(InGridSettings, InLocalToWorld);

		// Anchor-shifted floor snap: cells lie on multiples of CellSize starting from LocalAnchor, not world zero.
		// LocalAnchor expresses the WP grid origin in mesh-local space, so MeshPartition cell numbering matches WP
		// runtime cell coords 1-to-1 for translation-only actor transforms. With WorldOriginOffset=Zero and
		// InLocalToWorld=Identity, LocalAnchor=Zero, preserving the legacy world-zero-anchored snap.
		FVector SnappedMin(FMath::FloorToDouble((InBounds.Min.X - LocalAnchor.X) / CellSize) * CellSize + LocalAnchor.X,
						   FMath::FloorToDouble((InBounds.Min.Y - LocalAnchor.Y) / CellSize) * CellSize + LocalAnchor.Y,
						   FMath::FloorToDouble((InBounds.Min.Z - LocalAnchor.Z) / CellSize) * CellSize + LocalAnchor.Z);

		FIntVector OriginCoord(FMath::FloorToInt32((InBounds.Min.X - LocalAnchor.X) / CellSize),
							   FMath::FloorToInt32((InBounds.Min.Y - LocalAnchor.Y) / CellSize),
							   FMath::FloorToInt32((InBounds.Min.Z - LocalAnchor.Z) / CellSize));

		const FBox SnappedBounds(SnappedMin, InBounds.Max);
		const FVector Extents = SnappedBounds.GetExtent();

		const int32 NumX = FMath::Max(1, FMath::DivideAndRoundUp(Extents.X * 2.0, CellSize));
		const int32 NumY = FMath::Max(1, FMath::DivideAndRoundUp(Extents.Y * 2.0, CellSize));
		int32 NumZ = FMath::Max(1, FMath::DivideAndRoundUp(Extents.Z * 2.0, CellSize));

		FVector CellExtent(CellSize);

		// 2D grid: collapse Z into a single column. NumZ=1; OriginCoord.Z=0 is a
		// sentinel (no world-position Z mapping in 2D, so SectionBox.Z is NOT
		// reconstructible from GridCellCoord alone -- consumers match on the (X,Y,0)
		// key, never reconstruct geometry from it). SnappedMin.Z keeps the 3D-path
		// floor(Min.Z/CellSize)*CellSize value so the section's Z lower bound stays
		// grid-aligned and stable against FP jitter in InBounds.Min.Z across builds.
		if (InGridSettings.bIs2D)
		{
			NumZ = 1;
			OriginCoord.Z = 0;

			// Floor handles legit flat-Z meshes (Max.Z == Min.Z -> centroids at Min.Z fit the
			// inclusive Contains check) and is a best-effort fallback for malformed inputs.
			// Truly-invalid bounds (Max.Z < Min.Z) collapse SectionBox to a thin slab and
			// silently drop centroids above it -- ensureMsgf surfaces the upstream bug; the
			// floor remains the shipping-build fallback.
			ensureMsgf(InBounds.Max.Z >= InBounds.Min.Z,
					   TEXT("ComputeGridDimensions 2D: invalid Z bounds (Min=%g, Max=%g) -- upstream FBox is malformed; SectionBox will collapse and triangles may drop"),
					   InBounds.Min.Z, InBounds.Max.Z);
			CellExtent.Z = FMath::Max(KINDA_SMALL_NUMBER, InBounds.Max.Z - SnappedMin.Z);
		}

		const int64 TotalCells64 = static_cast<int64>(NumX) * static_cast<int64>(NumY) * static_cast<int64>(NumZ);
		check(TotalCells64 > 0 && TotalCells64 <= MAX_int32);

		return { SnappedMin, OriginCoord, { NumX, NumY, NumZ }, CellExtent, static_cast<int32>(TotalCells64) };
	}

	TMap<FIntVector, MeshPartition::FMeshData> BuildGridCellMeshes(const MeshPartition::FMeshData& InMesh, const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld)
	{
		const FGridDimensions Grid = ComputeGridDimensions(FBox(InMesh.GetBounds()), InGridSettings, InLocalToWorld);

		constexpr bool bFilterEmptyMeshes = false;
		TArray<MeshPartition::FMeshData> FlatCells = BuildHelpers::BuildSections(InMesh, Grid, bFilterEmptyMeshes);

		TMap<FIntVector, MeshPartition::FMeshData> Result;
		Result.Reserve(Grid.TotalCells);

		for (int32 FlatIndex = 0; FlatIndex < FlatCells.Num(); ++FlatIndex)
		{
			if (FlatCells[FlatIndex].VertexCount() == 0)
			{
				continue;
			}

			const int32 LocalX = FlatIndex % Grid.CellNumber.X;
			const int32 LocalY = (FlatIndex / Grid.CellNumber.X) % Grid.CellNumber.Y;
			const int32 LocalZ = FlatIndex / (Grid.CellNumber.X * Grid.CellNumber.Y);
			const FIntVector Coord = Grid.OriginCoord + FIntVector(LocalX, LocalY, LocalZ);

			Result.Add(Coord, MoveTemp(FlatCells[FlatIndex]));
		}

		return Result;
	}
}

namespace UE::MeshPartition::BuildHelpers
{
	namespace Private
	{
		void TransferVertexAttributes(TConstArrayView<int32> InTargetToSourceVertexIDs, const MeshPartition::FMeshData& InSourceMesh, MeshPartition::FMeshData& OutTargetMesh, const bool bTransferNormals)
		{
			OutTargetMesh.SetNumSourceUVChannels(InSourceMesh.GetNumSourceUVChannels());

			ParallelFor(InTargetToSourceVertexIDs.Num(), [&](int VertexID)
			{
				int SourceVertexID = InTargetToSourceVertexIDs[VertexID];

				OutTargetMesh.SetChannelUV(VertexID, InSourceMesh.GetChannelUV(SourceVertexID));

				for (int32 UVChannelIndex = 0; UVChannelIndex < InSourceMesh.GetNumSourceUVChannels(); ++UVChannelIndex)
				{
					const FVector2f VertexUVs = InSourceMesh.GetVertexUV(SourceVertexID, UVChannelIndex);
					OutTargetMesh.SetVertexUV(VertexID, VertexUVs, UVChannelIndex);
				}

				if (bTransferNormals)
				{
					FVector3f VertexNormal = InSourceMesh.GetVertexNormal(SourceVertexID);
					OutTargetMesh.SetVertexNormal(VertexID, VertexNormal);
				}
			});

			for (const FName& SourceLayerName : InSourceMesh.GetWeightLayerNames())
			{
				const TArray<float>& SourceValues = InSourceMesh.GetWeightLayerValues(SourceLayerName);
				TArray<float> ResultValues;
				TArray<int> VertexIDs;

				OutTargetMesh.InitializeWeightLayer(SourceLayerName);
				ResultValues.SetNumZeroed(InTargetToSourceVertexIDs.Num());
				VertexIDs.SetNumZeroed(InTargetToSourceVertexIDs.Num());
						
				ParallelFor(ResultValues.Num(), [&](int VertexID)
				{
					ResultValues[VertexID] = SourceValues[InTargetToSourceVertexIDs[VertexID]];
					VertexIDs[VertexID] = VertexID;
				});

				OutTargetMesh.SetWeightLayerValues(SourceLayerName, VertexIDs, ResultValues);
			}
		}

		void TransferTriangleAttributes(TConstArrayView<int32> InTargetToSourceTriangleIDs, const MeshPartition::FMeshData& InSourceMesh, MeshPartition::FMeshData& OutTargetMesh)
		{
			ParallelFor(InTargetToSourceTriangleIDs.Num(), [&InSourceMesh, &OutTargetMesh, &InTargetToSourceTriangleIDs](int TriangleID)
			{
				const int BaseID = InSourceMesh.GetBaseID(InTargetToSourceTriangleIDs[TriangleID]);
				OutTargetMesh.SetBaseID(TriangleID, BaseID);
			});
		}

		void FilterOutOfBoundsTriangles(MeshPartition::FMeshData& InOutMesh, TConstArrayView<Geometry::FOrientedBox3d> InFilterBounds, const FTransform& InTransform, MeshPartition::EFilterBoundsMode InFilterMode)
		{
			TArray<TArray<int>> TrianglesToRemove;

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::FilterOutOfBoundsTriangles_Collect);

				ParallelForWithTaskContext(TrianglesToRemove, InOutMesh.MaxTriangleID(),
				[&](TArray<int>& OutTrianglesToRemove, int TriangleID)
				{
					if (!InOutMesh.IsTriangle(TriangleID))
					{
						return;
					}

					UE::Geometry::FIndex3i Triangle = InOutMesh.GetTriangle(TriangleID);

					for (int Index = 0; Index < 3; ++Index)
					{
						const FVector3d Vertex = InTransform.TransformPosition(InOutMesh.GetVertex(Triangle[Index]));
						for (const Geometry::FOrientedBox3d& Bounds : InFilterBounds)
						{
							if ((InFilterMode == MeshPartition::EFilterBoundsMode::Inclusive && !Bounds.Contains(Vertex)) ||
								(InFilterMode == MeshPartition::EFilterBoundsMode::Exclusive && Bounds.Contains(Vertex)))
							{
								OutTrianglesToRemove.Add(TriangleID);
								return;
							}
						}
					}
				});
			}

			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::FilterOutOfBoundsTriangles_Remove);

				for (TArray<int> Triangles : TrianglesToRemove)
				{
					for (int TriangleID : Triangles)
					{
						InOutMesh.RemoveTriangle(TriangleID, true);
					}
				}
			}
		}


		FSharedBuffer TryGetBuiltMegaMeshFromDDC(const FBlake3Hash& InBuildKey)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(TryGetBuiltMegaMeshFromDDC);

			using namespace UE::DerivedData;
			FCacheKey CacheKey;
			CacheKey.Bucket = FCacheBucket(TEXT("MegaMesh"));
			CacheKey.Hash = InBuildKey;

			FCacheGetRequest Request = { UE::FSharedString(TEXT("MegaMeshMeshData")), CacheKey, ECachePolicy::Default };

			FRequestOwner RequestOwner(EPriority::Blocking);

			FSharedBuffer SharedBuffer;
			bool bDDCHit = false;

			GetCache().Get(MakeArrayView(&Request, 1), RequestOwner,
						   [&SharedBuffer, &bDDCHit](FCacheGetResponse&& Response) mutable
						   {
							   if (Response.Status == EStatus::Ok)
							   {
								   static const FValueId MeshDataId = FValueId::FromName("BuiltMegaMeshData");
								   const FCompressedBuffer& CompressedBuffer = Response.Record.GetValue(MeshDataId).GetData();
								   SharedBuffer = CompressedBuffer.Decompress();
								   UE_LOGF(LogMegaMeshEditor, Log, "MegaMeshBuild DDC Hit %ls", *LexToString(Response.Record.GetKey().Hash));
								   bDDCHit = true;
							   }
						   });

			RequestOwner.Wait();

			return SharedBuffer;
		}

		int64 SaveBuiltMegaMeshToDDC(const FBlake3Hash& InBuildKey, const MeshPartition::FMeshData& InMeshData)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SaveBuiltMegaMeshToDDC);

			using namespace UE::DerivedData;

			FCacheKey CacheKey;
			CacheKey.Bucket = FCacheBucket(TEXT("MegaMesh"));
			CacheKey.Hash = InBuildKey;

			FLargeMemoryWriter Ar(0, true);

			// Const cast is safe here because we are using a writer archive.
			const_cast<MeshPartition::FMeshData&>(InMeshData).Serialize(Ar);

			static const FValueId MeshDataId = FValueId::FromName("BuiltMegaMeshData");
			FValue MeshDataValue = FValue::Compress(FSharedBuffer::MakeView(Ar.GetData(), Ar.TotalSize()));

			FCacheRecordBuilder RecordBuilder(CacheKey);
			RecordBuilder.AddValue(MeshDataId, MeshDataValue);

			FRequestOwner RequestOwner(UE::DerivedData::EPriority::Normal);
			const FCachePutRequest PutRequest = { UE::FSharedString(TEXT("MegaMeshMeshData")), RecordBuilder.Build(), ECachePolicy::Default };
			GetCache().Put(MakeArrayView(&PutRequest, 1), RequestOwner);

			// Keep the put request alive when the RequestOwner goes out of scope to ensure it continues to process the request asynchronously
			RequestOwner.KeepAlive();

			UE_LOGF(LogMegaMeshEditor, Log, "MegaMeshBuild DDC Put %ls", *LexToString(CacheKey.Hash));
			return Ar.TotalSize();
		}

		bool ShouldAllowDDCForBuildGroup(const MeshPartition::FBuilderSettings& InSettings, const UE::MeshPartition::FModifierGroup& InGroup)
		{
			// function can only be called from the GT since it accesses modifier component uobject data directly
			check(IsInGameThread());

			bool bShouldAllowDDCForBuildGroup = true;

			// Disable writing transient changes to DDC as much as possible.
			// The package of the modifier being dirty is used to determine if the modifier is in a "pending" state and is likely to change again soon.
			InGroup.ForEachModifier([&bShouldAllowDDCForBuildGroup](MeshPartition::UModifierComponent* Modifier)
			{
				if (Modifier->GetPackage() && Modifier->GetPackage()->IsDirty())
				{
					UE_LOGF(LogMegaMeshEditor, Verbose, "DDC for MegaMesh Build disabled. Reason: Disabled by modifier (%ls) in dirty package: %ls", *Modifier->GetPathName(), *GetNameSafe(Modifier->GetPackage()));
					bShouldAllowDDCForBuildGroup = false;
					return false;
				}
					
				if (Modifier->IsTemporarilyDisabledInEditor())
				{
					UE_LOGF(LogMegaMeshEditor, Verbose, "DDC for MegaMesh Build disabled. Reason: Disabled by temporarily disabled modifier: %ls", *Modifier->GetPathName());
					bShouldAllowDDCForBuildGroup = false;
					return false;
				}

				return true;
			});

			return bShouldAllowDDCForBuildGroup;
		}

		bool ShouldAllowDDCWriteForBuildGroup(const MeshPartition::FBuilderSettings& InSettings, const UE::MeshPartition::FModifierGroup& InGroup)
		{
			if (!InSettings.bAllowDDCWrite)
			{
				return false;
			}

			// Individual modifier ops may explicitly disable ddc for builds of which they are a part.
			for (UE::MeshPartition::FModifierIndex ModifierIndex : InGroup.ModifierIndices())
			{
				TSharedPtr<const UE::MeshPartition::IModifierBackgroundOp> ModifierOp = InGroup.GetModifierOp(ModifierIndex);
				if (ModifierOp && ModifierOp->DisableDDCWrite())
				{
					const MeshPartition::FModifierDesc& ModifierDesc = InGroup.GetModifierDesc(ModifierIndex);
					UE_LOGF(LogMegaMeshEditor, Log, "DDC Write for MegaMesh Build disabled. Reason: Disabled by Op. Modifier (%ls), Op (%ls)", *ModifierDesc.ModifierPath.ToString(), *ModifierOp->GetOperationName().ToString());
					return false;
				}
			}

			return true;
		}
	} // namespace Private

	FBlake3Hash ComputeModifierGroupBuildKey(const MeshPartition::FBuilderSettings& InSettings, const UE::MeshPartition::FModifierGroup& InGroup)
	{
		FBlake3 Hasher;

		auto UpdateHashPODType = []<typename Type> (FBlake3& Hasher, const Type& InData)
		{
			Hasher.Update(reinterpret_cast<const uint8*>(&InData), sizeof(Type));
		};

		auto UpdateHashFString = [](FBlake3& Hasher, const FString& InStr)
		{
			FStringView Name(InStr);
			Hasher.Update(Name.GetData(), Name.Len() * sizeof(TCHAR));
		};

		static FGuid VersionKey(TEXT("1bd4c1d9-f8c3-4b96-92ee-80763cb396f3"));
		UpdateHashPODType(Hasher, VersionKey);

		// Hash the current mesh data type version into the key to ensure swapping mesh types hits different DDC entries.
		UpdateHashPODType(Hasher, MeshPartition::FMeshData::GetVersionKey());

		UpdateHashPODType(Hasher, InSettings.Transform);
		for (const Geometry::FOrientedBox3d& FilterBox : InSettings.FilterBounds)
		{
			UpdateHashPODType(Hasher, FilterBox);
		}
		UpdateHashPODType(Hasher, InSettings.FilterBoundsMode);

		if (InSettings.SimplifierOptions.IsSet())
		{
			UpdateHashPODType(Hasher, InSettings.SimplifierOptions.GetValue());
		}

		if (InSettings.TexcoordGenerationOptions.IsSet())
		{
			const FChannelCollectionUVLayoutOptions& TexcoordOptions = InSettings.TexcoordGenerationOptions.GetValue();
			UpdateHashPODType(Hasher, TexcoordOptions);

			// Hash the VEUV algorithm-version GUID when this build uses VolumeEncoded so that algo/config changes invalidate cached outputs
			if (TexcoordOptions.UVLayoutMethod == EChannelCollectionUVLayoutMethod::VolumeEncoded)
			{
				UpdateHashPODType(Hasher, FVEUVConfig::AlgorithmVersionGuid);
			}
		}
		if (InSettings.ChannelRenderSettings)
		{
			UpdateHashPODType(Hasher, InSettings.ChannelRenderSettings->TexelSize);
			for (FName ChannelName : InSettings.ChannelRenderSettings->ChannelMap.GetChannels())
			{
				UpdateHashFString(Hasher, ChannelName.ToString());
			}
		}
	
		UpdateHashPODType(Hasher, InSettings.bRecomputeTangents);
		UpdateHashPODType(Hasher, InSettings.bRecomputeNormals);

		UpdateHashPODType(Hasher, InSettings.bRecomputeNormals);
	
		FBlake3Hash GroupHash = InGroup.ComputeGroupBuildHash();
		Hasher.Update(GroupHash.GetBytes(), sizeof(FBlake3Hash::ByteArray));

		return Hasher.Finalize();
	}

	TArray<MeshPartition::FModifierDesc> InitializeModifierDescriptors(const MeshPartition::FBuilderSettings& InSettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(InitializeModifierDescriptors);

		TArray<MeshPartition::FModifierDesc> Descriptors;

		Algo::TransformIf(InSettings.ModifiersToProcess, Descriptors, [](MeshPartition::UModifierComponent* Modifier)
		{
			return Modifier != nullptr;
		},
		[](MeshPartition::UModifierComponent* Modifier) -> MeshPartition::FModifierDesc
		{
			return MeshPartition::FModifierDesc(*Modifier);
		});

		if (InSettings.ModifierFilter.IsSet())
		{
			for (auto It = Descriptors.CreateIterator(); It; ++It)
			{
				const bool bShouldKeep = InSettings.ModifierFilter(InSettings, *It);
				if (!bShouldKeep)
				{
					It.RemoveCurrent();
				}
			}
		}

		UE::MeshPartition::SortModifierDescriptors(InSettings.TypePriorities, Descriptors);

		return Descriptors;
	}

	TArray<MeshPartition::FMeshData> BuildSections(const MeshPartition::FMeshData& InMesh, const GridHelpers::FGridDimensions& InGridDimensions, const bool bInFilterEmptyMeshes)
	{
		//todo(luc.eygasier): think about reusing the spatial structure in the builder result.
		MeshPartition::FMeshABBTree3 Spatial;
		constexpr bool bAutoBuild = true;

		Spatial.SetMesh(&InMesh, bAutoBuild);

		const int32 NumberOfMeshSectionsX = InGridDimensions.CellNumber.X;
		const int32 NumberOfMeshSectionsY = InGridDimensions.CellNumber.Y;
		const int32 TotalNumberOfSections = InGridDimensions.TotalCells;

		TArray<MeshPartition::FMeshData> Result;

		Result.SetNum(TotalNumberOfSections);

		TArray<std::atomic<int32>> TriangleOwners;
		TriangleOwners.SetNum(InMesh.MaxTriangleID());

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MarkTriangles)

			ParallelFor(TotalNumberOfSections, [&](int32 SectionIndex)
			{
				const int32 SectionIndexX = SectionIndex % NumberOfMeshSectionsX;
				const int32 SectionIndexY = SectionIndex / NumberOfMeshSectionsX % NumberOfMeshSectionsY;
				const int32 SectionIndexZ = SectionIndex / (NumberOfMeshSectionsX * NumberOfMeshSectionsY);

				const FVector SectionPos = InGridDimensions.SnappedMin + FVector(SectionIndexX, SectionIndexY, SectionIndexZ) * InGridDimensions.CellExtent;

				const UE::Geometry::FAxisAlignedBox3d SectionBox(SectionPos, SectionPos + InGridDimensions.CellExtent);
				const UE::Geometry::FAxisAlignedBox3d ExpandedSectionBox(SectionPos - KINDA_SMALL_NUMBER, SectionPos + InGridDimensions.CellExtent + KINDA_SMALL_NUMBER);

				MeshPartition::FMeshABBTree3::FTreeTraversal Traversal;

				int SelectAllDepth = TNumericLimits<int>::Max();
				int CurrentDepth = -1;

				{
					Traversal.NextBoxF = [&ExpandedSectionBox, &CurrentDepth, &SelectAllDepth](const UE::Geometry::FAxisAlignedBox3d& Box, int Depth)
					{
						CurrentDepth = Depth;

						if (Depth > SelectAllDepth)
						{
							return true;
						}

						SelectAllDepth = TNumericLimits<int>::Max();

						if (ExpandedSectionBox.Contains(Box))
						{
							SelectAllDepth = Depth;
							return true;
						}

						return ExpandedSectionBox.Intersects(Box);
					};

					Traversal.NextTriangleF = [&TriangleOwners, &SectionBox, &InMesh, &CurrentDepth, &SelectAllDepth, SectionIndex = SectionIndex] (int TriangleID)
					{
						auto MarkTriangle = [&](int TriangleID)
						{
							int32 PreviousOwner = 0;
							while (!TriangleOwners[TriangleID].compare_exchange_weak(PreviousOwner, SectionIndex + 1, std::memory_order_relaxed))
							{
								// don't try to compare and swap if the previous owner is a higher index.
								// This is how we deterministically resolve overlaps
								if (PreviousOwner > (SectionIndex + 1))
								{
									return;
								}
							}
						};

						// This TriangleID is entirely contained in the selection rectangle so we can skip intersection testing
						if (CurrentDepth >= SelectAllDepth)
						{
							MarkTriangle(TriangleID);
							return;
						}

						FVector3d VertexA;
						FVector3d VertexB;
						FVector3d VertexC;

						InMesh.GetTriVertices(TriangleID, VertexA, VertexB, VertexC);

						FVector3d Centroid = (VertexA + VertexB + VertexC) / 3.0;

						if (SectionBox.Contains(Centroid))
						{
							MarkTriangle(TriangleID);
						}
					};

					Spatial.DoTraversal(Traversal);
				}
			});
		}

		{
			TRACE_CPUPROFILER_EVENT_SCOPE(BuildMeshes)

			ParallelFor(TotalNumberOfSections, [&](int32 SectionIndex)
			{
				TArray<int> Triangles;
				for (int32 TriangleIndex = 0; TriangleIndex < TriangleOwners.Num(); ++TriangleIndex)
				{
					if (TriangleOwners[TriangleIndex].load(std::memory_order_relaxed) == (SectionIndex + 1))
					{
						Triangles.Add(TriangleIndex);
					}
				}

				TMap<int, int> SourceToNewVertexIDs;
				TArray<int> NewToSourceVertexIDs;
				TArray<int> NewToSourceTriangleIDs;

				MeshPartition::FMeshData& ResultMesh = Result[SectionIndex];

				for (int SourceTriangleID : Triangles)
				{
					UE::Geometry::FIndex3i Triangle = InMesh.GetTriangle(SourceTriangleID);
					UE::Geometry::FIndex3i NewTriangle;
			
					for (int Index = 0; Index < 3; ++Index)
					{
						const int VertexID = Triangle[Index];
						int NewVertexID = 0;
						int* NewVertexIDPtr = SourceToNewVertexIDs.Find(VertexID);

						if (NewVertexIDPtr == nullptr)
						{
							NewVertexID = ResultMesh.AppendVertex(InMesh.GetVertex(VertexID));
							SourceToNewVertexIDs.Emplace(VertexID, NewVertexID);

							int NewVertexIDIndex = NewToSourceVertexIDs.Emplace(VertexID);
							ensure(NewVertexID == NewVertexIDIndex);
						}
						else
						{
							NewVertexID = *NewVertexIDPtr;
						}

						NewTriangle[Index] = NewVertexID;
					}

					int NewTriangleID = ResultMesh.AppendTriangle(NewTriangle);
					int NewTriangleIDIndex = NewToSourceTriangleIDs.Emplace(SourceTriangleID);
					ensure(NewTriangleID == NewTriangleIDIndex);
				}

				constexpr bool bTransferNormals = true;
				BuildHelpers::Private::TransferVertexAttributes(NewToSourceVertexIDs, InMesh, ResultMesh, bTransferNormals);
				BuildHelpers::Private::TransferTriangleAttributes(NewToSourceTriangleIDs, InMesh, ResultMesh);
			});
		}

		if (bInFilterEmptyMeshes)
		{
			Result.RemoveAll([](const MeshPartition::FMeshData& Mesh) { return Mesh.VertexCount() == 0; });
		}
		return MoveTemp(Result);
	}

	MeshPartition::FMeshData SimplifyMesh(const MeshPartition::FMeshData& InSourceMesh, const float InEdgeLength, const bool bInTransferAttributes, const bool bTransferNormals)
	{
		UE::Geometry::TMeshWrapperAdapterd<const MeshPartition::FMeshData> InputAdapter(&InSourceMesh);
		MeshPartition::FMeshData SimplifiedMesh;
		UE::Geometry::MeshClusterSimplify::FResultMeshAdapter ResultAdapter;
		ResultAdapter.Init(&SimplifiedMesh);

		ResultAdapter.TransferPerVertexAttributes = [&InSourceMesh, &SimplifiedMesh, bTransferNormals](TConstArrayView<int32> ResultToSourceVertexIDs)
		{
			BuildHelpers::Private::TransferVertexAttributes(ResultToSourceVertexIDs, InSourceMesh, SimplifiedMesh, bTransferNormals);
		};

		ResultAdapter.TransferPerTriangleAttributes = [&InSourceMesh, &SimplifiedMesh](TConstArrayView<int32> ResultToSourceTriangleID)
		{
			BuildHelpers::Private::TransferTriangleAttributes(ResultToSourceTriangleID, InSourceMesh, SimplifiedMesh);
		};

		UE::Geometry::MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
		SimplifyOptions.TargetEdgeLength = InEdgeLength;
		SimplifyOptions.bTransferAttributes = bInTransferAttributes;

		UE::Geometry::MeshClusterSimplify::Simplify(InputAdapter, ResultAdapter, SimplifyOptions);
		return MoveTemp(SimplifiedMesh);
	}

	MeshPartition::FMeshData MergeBaseModifiers(const UE::MeshPartition::FModifierGroup& InGroup, const MeshPartition::FBuilderSettings& InSettings, const FGuid& InBaseCacheKey)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::MergeBaseModifiers);

		MeshPartition::FMeshData MergedMesh;
		MeshPartition::FModifierGraphCache* ModifierCache = UMeshPartitionEditorSubsystem::GetGraphCache();
		const bool bHasCachedMesh = ModifierCache->GetCachedMergedBaseMesh(InBaseCacheKey, MergedMesh);

		if (!bHasCachedMesh)
		{
			TArray<TPair<int, int>> MergeVertices;
			int32 NextBaseID = 0;
			for (UE::MeshPartition::FModifierIndex Base: InGroup.BaseIndices())
			{
				const int32 BaseID = NextBaseID++;

				const TSharedPtr<const FDynamicMesh3> ModifierMesh = InGroup.GetModifierOp(Base)->GetMesh();
				const FTransform RelativeTransform = InGroup.GetModifierOp(Base)->GetMeshTransform();

				TSet<int> AppendBoundaryVertices;
				MergedMesh.AppendDynamicMesh(*ModifierMesh, RelativeTransform, BaseID, &AppendBoundaryVertices);
				for (int VID : AppendBoundaryVertices)
				{
					MergeVertices.Emplace(VID, BaseID);
				}
			}

			MergedMesh.WeldCoincidentVertices(MergeVertices);

			if (ShouldApplyBaseSimplificationForGroup(InGroup, InSettings))
			{
				constexpr bool bTransferAttributes = true;
				constexpr bool bTransferNormals = false;
				MergedMesh = SimplifyMesh(MergedMesh, InSettings.SimplifierOptions->TargetEdgeLength, bTransferAttributes, bTransferNormals);
			}

			// It is always worth caching the base mesh, since it will always be either equivalent to a copy or more commonly more expensive due to the cost of merging
			if (MeshPartition::FModifierGraphCache::IsCachingEnabled())
			{
				ModifierCache->CacheMergedBaseMesh(InBaseCacheKey, MergedMesh);
			}
		}

		return MergedMesh;
	}

	bool ShouldApplyBaseSimplificationForGroup(const UE::MeshPartition::FModifierGroup& InGroup, const MeshPartition::FBuilderSettings& InSettings)
	{
		return (InSettings.BuildType == MeshPartition::EBuildType::SimplifiedPreviewSection) &&
				InSettings.SimplifierOptions.IsSet() &&
			   (InGroup.ComputeBaseComplexity() >= static_cast<double>(InSettings.SimplifierOptions->MinVertexCount));
	}
	FGuid ComputeBaseCacheKey(const UE::MeshPartition::FModifierGroup& InGroup, const MeshPartition::FBuilderSettings& InSettings)
	{
		FGuid Result = InGroup.ComputeBaseCacheKey();

		if (InSettings.BuildType == MeshPartition::EBuildType::InteractiveBase)
		{
			static FGuid InteractiveBaseGuid = FGuid::NewDeterministicGuid(TEXT("InteractiveBase"));
			Result = FGuid::Combine(Result, InteractiveBaseGuid);
		}
		else if (InSettings.BuildType == MeshPartition::EBuildType::InteractiveModifier)
		{
			static FGuid InteractiveModifierGuid = FGuid::NewDeterministicGuid(TEXT("InteractiveModifier"));
			Result = FGuid::Combine(Result, InteractiveModifierGuid);
		}
		else if (ShouldApplyBaseSimplificationForGroup(InGroup, InSettings))
		{
			static FGuid SimplifiedPreviewSectionGuid = FGuid::NewDeterministicGuid(TEXT("SimplifiedPreviewSection"));
			Result = FGuid::Combine(Result, SimplifiedPreviewSectionGuid);

			if (InSettings.SimplifierOptions.IsSet())
			{
				FGuid SimplifierOptionKey = FGuid::NewGuidFromHashBytes(&InSettings.SimplifierOptions.GetValue(), sizeof (MeshPartition::FBuilderSettings::FSimplifierOptions));
				Result = FGuid::Combine(Result, SimplifierOptionKey);
			}
		}

		return Result;
	}

	Tasks::FTask ProcessModifierGroup(
		const MeshPartition::FBuilderSettings& InSettings,
		UE::MeshPartition::FModifierGroup InGroup,
		TSharedPtr<MeshPartition::FModifierTaskGraph> InTaskGraph)
	{
#if DO_GUARD_SLOW
		ensure(InGroup.ValidateIsSorted(InSettings.TypePriorities));
#endif // DO_GUARD_SLOW
		
		MeshPartition::FMeshData BaseMesh;
		FGuid BaseGroupCacheKey = ComputeBaseCacheKey(InGroup, InSettings);
		const bool bIsBaseMeshSet = InSettings.BaseMesh.IsSet();
		const bool bUseCache = !bIsBaseMeshSet;
		
		if (!bIsBaseMeshSet)
		{
			BaseMesh = MergeBaseModifiers(InGroup, InSettings, BaseGroupCacheKey);
		}

		return InTaskGraph->Execute(bIsBaseMeshSet ? MoveTemp(*InSettings.BaseMesh.GetValue()) : MoveTemp(BaseMesh), BaseGroupCacheKey, MoveTemp(InGroup), InSettings.Transform, InSettings.TypePriorities, bUseCache);
	}
}

namespace UE::MeshPartition::Build
{
	void Wait(TConstArrayView<MeshPartition::FBuildTaskHandle> InTaskHandles)
	{
		for (const MeshPartition::FBuildTaskHandle& TaskHandle : InTaskHandles)
		{
			TaskHandle.Wait();
		}
	}

	bool AreAllTasksComplete(TConstArrayView<MeshPartition::FBuildTaskHandle> InTaskHandles)
	{
		for (const MeshPartition::FBuildTaskHandle& TaskHandle : InTaskHandles)
		{
			if (!TaskHandle.IsCompleted())
			{
				return false;
			}
		}

		return true;
	}

	TArray<MeshPartition::FBuildTaskHandle> LaunchBuilds(const MeshPartition::FBuilderSettings& InSettings)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::LaunchBuilds);

		using namespace BuildHelpers;
		using namespace BuildHelpers::Private;

		TArray<MeshPartition::FModifierDesc> Descriptors = InitializeModifierDescriptors(InSettings);

		TArray<UE::MeshPartition::FModifierGroup> ModifierGroups = UE::MeshPartition::BuildModifierGroups(Descriptors, InSettings);

		for (UE::MeshPartition::FModifierGroup& ModifierGroup : ModifierGroups)
		{
			ModifierGroup.SetBuildType(InSettings.BuildType);
			ModifierGroup.ProgressToState(UE::MeshPartition::FModifierGroup::EState::ModifiersResolved);
		}

		TArray<MeshPartition::FBuildTaskHandle> Handles;
		Handles.Reserve(ModifierGroups.Num());

		MeshPartition::FMeshBuilder* MeshBuilder = UMeshPartitionEditorSubsystem::GetMeshBuilder();
		check(MeshBuilder);

		for (int32 GroupIndex = 0; GroupIndex < ModifierGroups.Num(); ++GroupIndex)
		{
			// Check if the build should utilize DDC before removing any disabled modifiers.
			// We don't want to try to use DDC for builds which contain temporarily disabled modifiers
			const bool bAllowDDC = (InSettings.BuildType != MeshPartition::EBuildType::InteractiveBase) &&
								   (InSettings.BuildType != MeshPartition::EBuildType::InteractiveModifier) &&
								   (InSettings.BuildType != MeshPartition::EBuildType::SimplifiedPreviewSection) &&
								   ShouldAllowDDCForBuildGroup(InSettings, ModifierGroups[GroupIndex]);

			ModifierGroups[GroupIndex].RemoveDisabledModifiers();

			// Remove all disabled modifiers.
			// This is explicitly done after all the groupings are finalized to avoid disabling having any effect on the groups.
			if (ModifierGroups[GroupIndex].BaseDescs().Num() == 0)
			{
				continue;
			}

			// #todo: I don't think we really need background ops on the GT here.
			// It may be simpler if we should only build them right before passing off to the async thread.
			ModifierGroups[GroupIndex].ProgressToState(UE::MeshPartition::FModifierGroup::EState::BackgroundOpsCreated);

			Handles.Add(MeshBuilder->Build_Internal(InSettings, MoveTemp(ModifierGroups[GroupIndex]), bAllowDDC));
		}

		return Handles;
	}
}

namespace UE::MeshPartition
{
FBuildTaskHandle::FBuildTaskHandle(const TSharedPtr<MeshPartition::FBuildTask>& InBuildTask)
{
	check(InBuildTask);
	Task = InBuildTask;
	Task->Handles++;
}

FBuildTaskHandle::~FBuildTaskHandle()
{
	Release(Task, bIsCancelled);
}

FBuildTaskHandle::FBuildTaskHandle(FBuildTaskHandle&& InHandle)
{
	Task = InHandle.Task;
	bIsCancelled = InHandle.bIsCancelled;
	InHandle.Task = nullptr;
	bIsCancelled = false;
}

FBuildTaskHandle::FBuildTaskHandle(const FBuildTaskHandle& InHandle)
{
	Task = InHandle.Task;
	if(Task)
	{
		Task->Handles++;
	}
	bIsCancelled = InHandle.bIsCancelled;
}

FBuildTaskHandle& FBuildTaskHandle::operator=(const FBuildTaskHandle& InHandle)
{
	bool bWasCancelled = bIsCancelled;
	TSharedPtr<MeshPartition::FBuildTask> PreviousTask = Task;

	Task = InHandle.Task;
	if (Task)
	{
		Task->Handles++;
	}
	bIsCancelled = InHandle.bIsCancelled;

	Release(PreviousTask, bWasCancelled);

	return *this;
}

FBuildTaskHandle& FBuildTaskHandle::operator=(FBuildTaskHandle&& InHandle)
{
	const bool bWasCancelled = bIsCancelled;
	const TSharedPtr<MeshPartition::FBuildTask> PreviousTask = Task;

	Task = InHandle.Task;
	bIsCancelled = InHandle.bIsCancelled;
	InHandle.Task = nullptr;
	bIsCancelled = false;

	Release(PreviousTask, bWasCancelled);

	return *this;
}

void FBuildTaskHandle::Release(const TSharedPtr<MeshPartition::FBuildTask>& InTask, bool bCancel)
{
	if (InTask)
	{
		InTask->Handles--;
		if (bCancel)
		{
			UMeshPartitionEditorSubsystem::Get()->GetMeshBuilder()->Cancel(InTask);
		}
	}
}

bool FBuildTaskHandle::Wait() const
{
	if (Task)
	{
		return Task->Wait();
	}

	return true;
}

bool FBuildTaskHandle::IsCompleted() const
{
	return !Task || Task->BuildTask.IsCompleted();
}

void FBuildTaskHandle::Cancel()
{
	bIsCancelled = true;
	Release(Task, bIsCancelled);
}

MeshPartition::FBuildTaskHandle FBuildTask::CreateHandle()
{
	return MeshPartition::FBuildTaskHandle(AsShared());
}

MeshPartition::FBuildPerfStats FBuildTask::GetBuildPerfStats() const
{
	return TaskGraph->GetBuildPerfStats();
};

TSharedPtr<const MeshPartition::FMeshData> FBuildTask::GetMesh() const
{
	Wait();
	return Mesh;
}

TSharedPtr<MeshPartition::FMeshData> FBuildTask::GetMutableMesh()
{
	check(!bIsCached);
	Wait();
	return Mesh;
}

TSharedPtr<const MeshPartition::FMeshABBTree3> FBuildTask::GetSpatial() const
{
	check(bBuildSpatial);
	Wait();
	return Spatial;
}

TSharedPtr<const MeshPartition::FSectionChannels> FBuildTask::GetSectionChannels() const
{
	Wait();
	return SectionChannels;
}

bool FBuildTask::Wait() const
{
	UE::Tasks::FTask WaitTask = UE::Tasks::Launch(
		TEXT("WaitOnGameThread_Task"), []() {},
		UE::Tasks::Prerequisites(BuildTask),
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);

	return WaitTask.Wait();
}

namespace
{
	constexpr uint64 BuilderCacheMaxNumElements = 65536;
}

FMeshBuilder::FMeshBuilder()
	: Cache(BuilderCacheMaxNumElements)
{
}

double FMeshBuilder::GetCacheTotalMemoryUsageMB() const
{
	uint64 TotalMemoryInBytes = 0;
	{
		UE::TUniqueLock Lock(CacheMutex);
		for (const auto& CacheEntry : Cache)
		{
			TotalMemoryInBytes += CacheEntry->GetByteCount();
		}
	}
	return TotalMemoryInBytes / (1024.0 * 1024.0);
}

void FMeshBuilder::EnforceMemoryBudget()
{
	// Enforce budget every 2 seconds
	if (FPlatformTime::Seconds() - LastEnforceMemoryBudget < 2.0)
	{
		return;
	}

	const double MemoryBudgetInBytes = CVarBuilderCacheEnabled.GetValueOnAnyThread() ? CVarBuilderCacheMemoryBudgetMB.GetValueOnAnyThread() * 1024 * 1024 : 0.0;
	
	double TotalMemoryInBytes = GetCacheTotalMemoryUsageMB() * 1024 * 1024;
	
	if (TotalMemoryInBytes < MemoryBudgetInBytes)
	{
		return;
	}

	UE::TUniqueLock Lock(CacheMutex);
	const float MemoryCleanupRatio = FMath::Clamp(CVarBuilderCacheMemoryCleanupRatio.GetValueOnAnyThread(), 0.0f, 1.0f);
	const uint64 TargetCacheMemoryUsage = static_cast<uint64>(MemoryCleanupRatio * MemoryBudgetInBytes);

	if (TargetCacheMemoryUsage == 0)
	{
		Cache.Empty(BuilderCacheMaxNumElements);
	}
	else
	{
		while (TotalMemoryInBytes > TargetCacheMemoryUsage)
		{
			TSharedPtr<MeshPartition::FBuildTask> RemovedEntry = Cache.RemoveLeastRecent();
			TotalMemoryInBytes -= RemovedEntry->GetByteCount();
		}
	}

	LastEnforceMemoryBudget = FPlatformTime::Seconds();
}

void FMeshBuilder::ClearCache()
{
	UE::TUniqueLock Lock(CacheMutex);
	Cache.Empty(BuilderCacheMaxNumElements);
}

void FMeshBuilder::Cancel(const TSharedPtr<MeshPartition::FBuildTask>& InBuildTask)
{
	// Handle count could change after this test but since we can only create handles by launching a build or copying/moving existing handles it is pretty safe.
	if (InBuildTask && InBuildTask->Handles == 0)
	{
		UE::TUniqueLock Lock(CacheMutex);
		Cache.Remove(InBuildTask->Key);
		InBuildTask->bIsCancelled = true;
		InBuildTask->TaskGraph->Cancel();
	}
}

bool FMeshBuilder::QueryCache(const MeshPartition::FBuilderSettings& InSettings, FBlake3Hash InKey, MeshPartition::FBuildTaskHandle& OutHandle)
{
	check(InSettings.bCacheResult && CVarBuilderCacheEnabled.GetValueOnAnyThread());

	UE::TUniqueLock Lock(CacheMutex);
	if (TSharedPtr<MeshPartition::FBuildTask>* FoundTask = Cache.FindAndTouch(InKey))
	{
		MeshPartition::FBuildTaskHandle Handle = (*FoundTask)->CreateHandle();

		const bool bIsValidMeshEntry = !Handle.IsCancelled() && (!Handle.IsCompleted() || Handle.GetTask()->GetMesh().IsValid());
		const bool bIsValidSpatial = bIsValidMeshEntry && Handle.GetTask()->bBuildSpatial && (!Handle.IsCompleted() || Handle.GetTask()->GetSpatial().IsValid());

		if (bIsValidMeshEntry && bIsValidSpatial == InSettings.bBuildSpatial)
		{
			OutHandle = Handle;
			return true;
		}

		TSharedPtr<MeshPartition::FBuildTask> ExistingTask;
		if (InSettings.bBuildSpatial && !bIsValidSpatial && bIsValidMeshEntry)
		{
			ExistingTask = *FoundTask;
		}

		// invalid result discard
		Cache.Remove(InKey);

		// Found an existing cached task with dynamic mesh but no spatial
		if (ExistingTask)
		{
			TSharedPtr<MeshPartition::FBuildTask> Result = MakeShared<MeshPartition::FBuildTask>();
			Handle = Result->CreateHandle();

			Result->Key = ExistingTask->Key;
			Result->bBuildSpatial = true;
			Result->bIsCached = true;
			Result->Transform = ExistingTask->Transform;
			Result->Group = ExistingTask->Group;

			Result->BuildTask = Tasks::Launch(TEXT("ProcessModifiers_SetSpatial"),
				[WeakResult = Result.ToWeakPtr(), ExistingTask]()
				{
					if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin())
					{
						Result->Mesh = ExistingTask->Mesh;
						Result->MeshByteCount = ExistingTask->MeshByteCount;
						Result->Spatial = MakeShared<MeshPartition::FMeshABBTree3>();
						Result->Spatial->SetMesh(Result->Mesh.Get(), /*bAutoBuild=*/true);
						Result->SpatialByteCount = Result->Spatial->GetByteCount();
					}
				},
				Tasks::Prerequisites(ExistingTask->BuildTask));

			Cache.Add(InKey, Result);
			OutHandle = Handle;
			return true;
		}
	}

	return false;
}

MeshPartition::FBuildTaskHandle FMeshBuilder::Build(const MeshPartition::FBuilderSettings& InSettings, const UE::MeshPartition::FModifierGroup& InModifierGroup, bool bAllowDDC)
{
	// with a read-only input, we make a copy of the group first (whose arrays get stolen in Build)
	UE::MeshPartition::FModifierGroup ModifierGroupCopy = InModifierGroup;

	// Have to set build type and progress to background ops created before calling Build_Internal()
	ModifierGroupCopy.SetBuildType(InSettings.BuildType);
	ModifierGroupCopy.ProgressToState(UE::MeshPartition::FModifierGroup::EState::BackgroundOpsCreated);

	// TODO : in the future we can make this entire function an async task invocation
	return Build_Internal(InSettings, MoveTemp(ModifierGroupCopy), bAllowDDC);
}

MeshPartition::FBuildTaskHandle FMeshBuilder::Build_Internal(const MeshPartition::FBuilderSettings& InSettings, UE::MeshPartition::FModifierGroup&& InModifierGroup, bool bAllowDDC)
{
	using namespace BuildHelpers;
	using namespace BuildHelpers::Private;

	const bool bUseCache = InSettings.bCacheResult && CVarBuilderCacheEnabled.GetValueOnAnyThread();
	const FBlake3Hash Key = ComputeModifierGroupBuildKey(InSettings, InModifierGroup);
	
	// Query cache only for cacheable results
	if (bUseCache)
	{
		MeshPartition::FBuildTaskHandle CacheHandle;
		if (QueryCache(InSettings, Key, CacheHandle))
		{
			return CacheHandle;
		}
	}

	// Lock Cache if needed
	bool bCacheLocked = false;
	if (bUseCache)
	{
		CacheMutex.Lock();
		bCacheLocked = true;
	}

	// Unlock when exiting scope
	ON_SCOPE_EXIT
	{
		if (bCacheLocked)
		{
			CacheMutex.Unlock();
		}
	};

	// Launch a Spatial Build task if needed with required prerequisite
	auto LaunchSpatialBuild = [&InSettings](TSharedPtr<MeshPartition::FBuildTask> InResult, Tasks::FTask InPrerequisite) -> Tasks::FTask
	{
		if(InSettings.bBuildSpatial)
		{
			return Tasks::Launch(TEXT("ProcessModifiers_SetSpatial"),
				[WeakResult = InResult.ToWeakPtr()]()
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(BuildMegaMeshSpatial);
					if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
					{
						Result->Spatial = MakeShared<MeshPartition::FMeshABBTree3>();
						Result->Spatial->SetMesh(Result->Mesh.Get(), /*bAutoBuild=*/true);
						Result->SpatialByteCount = Result->Spatial->GetByteCount();
					}
				},
				Tasks::Prerequisites(InPrerequisite));
		}

		return Tasks::FTask();
	};

	// Launch a DDC Save task if needed with the required prerequisite
	auto LaunchDDCSave = [&InSettings, bAllowDDC, Key](TSharedPtr<MeshPartition::FBuildTask> InResult, Tasks::FTask InPrerequisite) -> Tasks::FTask
	{
		// Hijack the final task in the pipe to store the result to ddc.
		const bool bAllowDDCWrite = bAllowDDC && ShouldAllowDDCWriteForBuildGroup(InSettings, InResult->Group);
		if (bAllowDDCWrite)
		{
			// Start the timer now and move it into the task. It will track cycle count from now until the mesh is finally committed to the DDC
			COOK_STAT(auto Timer = UE::MeshPartition::BuildStats::UsageStats.TimeSyncWork());
			return Tasks::Launch(TEXT("ProcessModifiers_SaveResultToDDC"),
				[COOK_STAT(Timer = MoveTemp(Timer), ) WeakResult = InResult.ToWeakPtr()]() mutable
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(SaveMegaMeshMeshBuildToDDC);

					if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid())
					{
						if (!Result->bIsCancelled)
						{
							const int64 BytesProcessed = SaveBuiltMegaMeshToDDC(Result->Key, *Result->Mesh);
							COOK_STAT(Timer.AddMiss(BytesProcessed));
						}
						else
						{
							// Cancel the timer if the task was canceled.
							COOK_STAT(Timer.Cancel());
						}
					}
				},
				Tasks::Prerequisites(InPrerequisite)
			);
		}

		return Tasks::FTask();
	};

	// New Task needs to be created (not found in cache)
	TSharedPtr<MeshPartition::FBuildTask> Result = MakeShared<MeshPartition::FBuildTask>();
	Result->Key = Key;

	// Add to Cache
	if (bUseCache)
	{
		Result->bIsCached = true;
		Cache.Add(Key, Result);
	}

	// Handle to return
	MeshPartition::FBuildTaskHandle Handle = Result->CreateHandle();

	Result->Group = MoveTemp(InModifierGroup);
	Result->TaskGraph = MakeShared<MeshPartition::FModifierTaskGraph>();
	Result->Transform = InSettings.Transform;

	Tasks::FTask LastTaskInPipeline;

	// Read from DDC is possible - but may still fail to hit a valid entry.
	if (bAllowDDC && InSettings.bAllowDDCRead)
	{
		COOK_STAT(auto Timer = UE::MeshPartition::BuildStats::UsageStats.TimeSyncWork());

		FSharedBuffer MeshData = TryGetBuiltMegaMeshFromDDC(Key);
		if (!MeshData.IsNull())
		{
			COOK_STAT(Timer.AddHit(MeshData.GetSize()));
			LastTaskInPipeline = Tasks::Launch(TEXT("ProcessModifiers_SerializeFromDDC"),
				[COOK_STAT(Timer = MoveTemp(Timer),) WeakResult = Result.ToWeakPtr(), MeshData = MoveTemp(MeshData)]()
				{
					if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(SerializeFromDDC);
						MeshPartition::FMeshData Mesh;
						FMemoryReaderView Reader(MeshData.GetView(), true);
						Mesh.Serialize(Reader);
						Result->MeshByteCount = Mesh.GetByteCount();
						Result->Mesh = MakeShared<MeshPartition::FMeshData>(MoveTemp(Mesh));
					}
				});
		}
	}

	// We missed the DDC so launch the rebuild process
	if (!LastTaskInPipeline.IsValid())
	{
		// Note: we can't capture Result in the lambdas below because we don't want the UObject pointers of the
		//  modifiers to be accessible from our background threads. The GetDescriptors calls below will give us
		//  copied arrays that are just the descriptors.
		Tasks::FTask ProcessModifiersTask = Tasks::Launch(TEXT("ProcessModifiers_ProcessModifierGroup"),
			[Group = Result->Group.CreateAsyncBuildGroup(),
			TaskGraph = Result->TaskGraph,
			Settings = InSettings]
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::ProcessModifiersTask);

				Tasks::FTask ProcessingComplete = ProcessModifierGroup(Settings, Group, TaskGraph);
				// Nesting the task marks it as a completion dependency without blocking the current worker thread.
				// Subsequent tasks will not start until the GraphComplete task is finished.
				Tasks::AddNested(ProcessingComplete);
			}
		);

		Tasks::FTask SetProcessedMeshTask = Tasks::Launch(TEXT("ProcessModifiers_SetProcessedMesh"),
			[WeakResult = Result.ToWeakPtr()]() mutable
			{
				if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
				{
					Result->Mesh = MakeShared<MeshPartition::FMeshData>(MoveTemp(Result->TaskGraph->GetResultMesh()));
				}
			},
			Tasks::Prerequisites(ProcessModifiersTask)
		);

		LastTaskInPipeline = SetProcessedMeshTask;

		if (InSettings.SimplifierOptions.IsSet() && (InSettings.BuildType != MeshPartition::EBuildType::SimplifiedPreviewSection))
		{
			LastTaskInPipeline = Tasks::Launch(TEXT("ProcessModifiers_Simplify"),
				[WeakResult = Result.ToWeakPtr(), SimplifierOptions = InSettings.SimplifierOptions.GetValue()]
				{
					if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
					{
						TSharedPtr<MeshPartition::FMeshData> Mesh = Result->Mesh;
						if (Mesh->VertexCount() >= SimplifierOptions.MinVertexCount)
						{
							UE::Geometry::MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
							SimplifyOptions.TargetEdgeLength = SimplifierOptions.TargetEdgeLength;

							FDynamicMesh3 SourceMesh;
							FDynamicMesh3 SimplifiedMesh;

							Mesh->ConvertToDynamicMesh(SourceMesh);
							//todo(luc.eygasier): can we avoid the multiple copies.
							UE::Geometry::MeshClusterSimplify::Simplify(SourceMesh, SimplifiedMesh, SimplifyOptions);
		
							Mesh->Reset();
							Mesh->AppendDynamicMesh(SimplifiedMesh, FTransform());
						}
					}
				},
				Tasks::Prerequisites(LastTaskInPipeline)
			);
		}

		LastTaskInPipeline = Tasks::Launch(TEXT("ProcessModifiers_PostProcessMesh"),
			[WeakResult = Result.ToWeakPtr(),
			BuildType = InSettings.BuildType,
			bRecomputeNormals = InSettings.bRecomputeNormals,
			bRecomputeTangents = InSettings.bRecomputeTangents,
			TaskGraph = Result->TaskGraph]
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::PostProcessMeshTask);
				if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
				{
					TSharedPtr<MeshPartition::FMeshData> ResultMesh = Result->Mesh;
					if (bRecomputeNormals)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::RecomputeNormals);
						// Require deterministic normals for all build types except interactive builds where performance is key and determinism doesn't matter. 
						const bool bRequireDeterministicNormals = !(BuildType == MeshPartition::EBuildType::InteractiveModifier || BuildType == MeshPartition::EBuildType::InteractiveBase) ;
						ResultMesh->RecomputeNormals(/* bRequireDeterministicNormals */ bRequireDeterministicNormals);
					}

					if (bRecomputeTangents)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::RecomputeTangents);
						ResultMesh->RecomputeTangents();
					}
				}
			},
			Tasks::Prerequisites(LastTaskInPipeline)
		);

		if (InSettings.FilterBounds.Num() > 0)
		{
			LastTaskInPipeline = Tasks::Launch(TEXT("ProcessModifiers_FilterOutOfBounds"),
				[WeakResult = Result.ToWeakPtr(), FilterBounds = InSettings.FilterBounds, FilterBoundsMode = InSettings.FilterBoundsMode, Transform = InSettings.Transform]
				{
					if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
					{
						TSharedPtr<MeshPartition::FMeshData> ResultMesh = Result->Mesh;

						FilterOutOfBoundsTriangles(*ResultMesh, FilterBounds, Transform, FilterBoundsMode);
					}
				},
				Tasks::Prerequisites(LastTaskInPipeline)
			);
		}
		
		LastTaskInPipeline = Tasks::Launch(TEXT("ProcessModifiers_PrecomputeMeshSize"),
			[WeakResult = Result.ToWeakPtr()]
			{
				if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
				{
					TSharedPtr<MeshPartition::FMeshData> ResultMesh = Result->Mesh;
					Result->MeshByteCount = ResultMesh->GetByteCount();
				}
			},
			Tasks::Prerequisites(LastTaskInPipeline)
		);

		if (InSettings.TexcoordGenerationOptions.IsSet())
		{
			LastTaskInPipeline = Tasks::Launch(TEXT("ProcessModifiers_GenerateTexcoords"),
				[WeakResult = Result.ToWeakPtr(), TexGenOptions = InSettings.TexcoordGenerationOptions.GetValue()]
				{
					if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::Build::GenerateTexcoords);
						MeshPartition::FChannelTextureRenderer::CreateSectionUVLayout(*Result->Mesh, TexGenOptions, Result->GetTransform());
					
						Result->Mesh->SummarizeUVRegion();
					}
				},
				Tasks::Prerequisites(LastTaskInPipeline)
			);
		}

		Tasks::FTask SaveToDDCTask = LaunchDDCSave(Result, LastTaskInPipeline);

		// We will add the DDC Write to the main task chain only if this isn't cached so data can be stolen,
		// in this case make sure we cache before the final build result is valid to consumers.
		if (!Result->bIsCached && SaveToDDCTask.IsValid())
		{
			LastTaskInPipeline = SaveToDDCTask;
		}
	}

	Tasks::FTask SpatialBuildTask = LaunchSpatialBuild(Result, LastTaskInPipeline);

	// #todo: the UV layout and remapping should be included in the DDC'd mesh data but that requires a larger refactor to texture channel render pipeline.
	// For now we will just inject this task after the ddc data is saved and both DDC hit and miss will rebuild this information.
	if (InSettings.ChannelRenderSettings.IsSet())
	{
		LastTaskInPipeline = Tasks::Launch(TEXT("ProcessModifiers_TextureChannels"),
			[WeakResult = Result.ToWeakPtr(), ChannelMap = InSettings.ChannelRenderSettings->ChannelMap, TexelSize = InSettings.ChannelRenderSettings->TexelSize]() mutable
			{
				if (TSharedPtr<MeshPartition::FBuildTask> Result = WeakResult.Pin(); Result.IsValid() && !Result->bIsCancelled)
				{
					UObject* const Owner = nullptr;
					constexpr bool bDownloadToAsset = false;
					Tasks::TTask<MeshPartition::FSectionChannels> RenderSectionChannels = FChannelTextureRenderer::BuildTextureForSection(*Result->Mesh, Owner, bDownloadToAsset, ChannelMap, TexelSize);
					Result->SectionChannels = MakeShared<MeshPartition::FSectionChannels>(MoveTemp(RenderSectionChannels.GetResult()));
				}
			},
			Tasks::Prerequisites(LastTaskInPipeline)
		);
	}
	
	if (SpatialBuildTask.IsValid())
	{
		Result->bBuildSpatial = true;
		LastTaskInPipeline = UE::Tasks::Launch(TEXT("MeshBuilder_Finalize"), []{}, UE::Tasks::Prerequisites(SpatialBuildTask, LastTaskInPipeline));
	}

	Result->BuildTask = LastTaskInPipeline;

	return Handle;
}
} // namespace UE::MeshPartition
