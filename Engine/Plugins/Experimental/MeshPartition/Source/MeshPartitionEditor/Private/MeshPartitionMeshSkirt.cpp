// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMeshSkirt.h"

#include "MeshPartitionMeshData.h"

#include "IndexTypes.h"
#include "MeshAdapter/MeshBoundaries.h"

namespace UE::MeshPartition
{

void AddMeshSkirt(FMeshData& InOutMeshData, const FMeshSkirtSettings& InSettings)
{
	using namespace UE::Geometry;

	MeshBoundaries::FBoundaryFilterOptions FilterOptions;
	FilterOptions.MinPerimeter = InSettings.BoundaryMinPerimeter;

	TArray<int32> BoundaryVertLoops, BoundaryTriLoops;
	TArray<int32> WeldMap = MeshBoundaries::ComputeVertexWeldMapping(InOutMeshData, InSettings.VertexSnapTolerance);
	MeshBoundaries::ComputeBoundaryLoops(InOutMeshData, BoundaryVertLoops, &BoundaryTriLoops, [&WeldMap](int32 VID) { return WeldMap[VID]; }, FilterOptions);

	const TArray<FName> WeightLayerNames = InOutMeshData.GetWeightLayerNames();
	const int32 NumSourceUVChannels = InOutMeshData.GetNumSourceUVChannels();

	// Map welded VID index to the corresponding (non-welded) VID on the given triangle
	// (used to make sure we copy UVs from the actual vertex on a given boundary edge, accounting for UV seams)
	auto GetTriCornerAtWeldedPos = [&InOutMeshData, &WeldMap](int32 TID, int32 WeldedVID) -> int32
	{
		const FIndex3i Tri = InOutMeshData.GetTriangle(TID);
		for (int32 I = 0; I < 3; ++I)
		{
			if (WeldMap[Tri[I]] == WeldedVID)
			{
				return Tri[I];
			}
		}
		return INDEX_NONE;
	};

	// Compute the extruded position for a boundary vertex using its neighbours
	auto ComputeSkirtPos = [&InOutMeshData, &InSettings](int32 WeldedCur, int32 WeldedPrev, int32 WeldedNext) -> FVector3d
	{
		FVector3d SourcePos = InOutMeshData.GetVertex(WeldedCur);
		FVector3d PrevPos = InOutMeshData.GetVertex(WeldedPrev);
		FVector3d NextPos = InOutMeshData.GetVertex(WeldedNext);
		FVector3d SourceN = (FVector3d)InOutMeshData.GetVertexNormal(WeldedCur);
		FVector3d PrevEdgeOffDir = SourceN ^ (SourcePos - PrevPos);
		FVector3d NextEdgeOffDir = SourceN ^ (NextPos - SourcePos);
		PrevEdgeOffDir.Normalize();
		NextEdgeOffDir.Normalize();
		FVector3d OffDir = PrevEdgeOffDir + NextEdgeOffDir;
		OffDir.Normalize();
		FVector3d UsePushDirection = InSettings.PushDirection;
		if (InSettings.PushMethod == EMeshSkirtDirectionMethod::VertexNormal)
		{
			UsePushDirection = -SourceN;
		}
		return SourcePos + OffDir * InSettings.Width + UsePushDirection * InSettings.PushDown;
	};

	// Create a skirt vertex and copy per-vertex attributes from the specified source VID
	auto CreateSkirtVertex = [&InOutMeshData, &WeightLayerNames, NumSourceUVChannels](const FVector3d& NewPos, int32 SourceVID) -> int32
	{
		const int32 NewVID = InOutMeshData.AppendVertex(NewPos);
		// Copy attribute values from source to the new vertex
		for (const FName& LayerName : WeightLayerNames)
		{
			InOutMeshData.SetWeightLayerValue(LayerName, NewVID, InOutMeshData.GetWeightLayerValue(LayerName, SourceVID));
		}
		InOutMeshData.SetChannelUV(NewVID, InOutMeshData.GetChannelUV(SourceVID));
		for (int32 ChannelIdx = 0; ChannelIdx < NumSourceUVChannels; ++ChannelIdx)
		{
			InOutMeshData.SetVertexUV(NewVID, InOutMeshData.GetVertexUV(SourceVID, ChannelIdx), ChannelIdx);
		}
		// We directly copy the source vertex normal also (to keep skirt geometry shading consistent, regardless of the skirt geometry)
		InOutMeshData.SetVertexNormal(NewVID, InOutMeshData.GetVertexNormal(SourceVID));

		return NewVID;
	};

	// non-welded boundary VID -> skirt VID
	TMap<int32, int32> BoundaryToSkirtVID;

	auto GetOrCreateOffsetVert = [&GetTriCornerAtWeldedPos, &ComputeSkirtPos, &CreateSkirtVertex, &BoundaryToSkirtVID]
		(const MeshBoundaries::FBoundaryVertexContext& Ctx) -> MeshBoundaries::FBoundaryOffsetVert
	{
		const int32 NonWeldedBoundaryVID = GetTriCornerAtWeldedPos(Ctx.EdgeTID, Ctx.WeldedVID);
		if (NonWeldedBoundaryVID == INDEX_NONE)
		{
			return { INDEX_NONE, INDEX_NONE };
		}
		if (int32* Found = BoundaryToSkirtVID.Find(NonWeldedBoundaryVID))
		{
			return { NonWeldedBoundaryVID, *Found };
		}
		const FVector3d SkirtPos = ComputeSkirtPos(Ctx.WeldedVID, Ctx.WeldedPrev, Ctx.WeldedNext);
		const int32 NewSkirtVID = CreateSkirtVertex(SkirtPos, NonWeldedBoundaryVID);
		BoundaryToSkirtVID.Add(NonWeldedBoundaryVID, NewSkirtVID);
		return { NonWeldedBoundaryVID, NewSkirtVID };
	};

	auto AddTri = [&InOutMeshData](const FIndex3i& Tri, int32 SourceTID)
	{
		if (Tri.A == INDEX_NONE || Tri.B == INDEX_NONE || Tri.C == INDEX_NONE)
		{
			return;
		}
		int32 NewTID = InOutMeshData.AppendTriangle(Tri);
		if (ensure(NewTID >= 0))
		{
			InOutMeshData.SetBaseID(NewTID, InOutMeshData.GetBaseID(SourceTID));
		}
	};
	MeshBoundaries::ExtrudeBoundaryLoopEdges(InOutMeshData, BoundaryVertLoops, BoundaryTriLoops, GetOrCreateOffsetVert, AddTri);
}

} // namespace UE::MeshPartition
