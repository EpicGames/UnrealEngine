// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMeshView.h"

#include "DynamicMeshEditor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshPartitionMeshData.h"

namespace UE::MeshPartition
{
namespace MeshViewLocals
{
	static const FName BaseIDOverlayLayerName = TEXT("MegaMeshBaseID");
}

FDynamicSubmesh::FDynamicSubmesh(const FMeshData& InSourceMesh, const TArray<int>& InTriangles, TSet<Geometry::FIndex2i>&& InEdgesOnSubmeshBoundary)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FDynamicSubmesh::Construct);

	SourceMesh = &InSourceMesh;
	EdgesOnSubmeshBoundary = MoveTemp(InEdgesOnSubmeshBoundary);

	constexpr bool bUnsafe = true;

	int NextVertexIndex = 0;
	int NextTriangleIndex = 0;

	Mesh.EnableAttributes();

	const int32 NumSourceUVChannels = InSourceMesh.GetNumSourceUVChannels();
	Mesh.Attributes()->SetNumUVLayers(NumSourceUVChannels);

	TArray<Geometry::FDynamicMeshUVOverlay*> SourceUVOverlays;
	SourceUVOverlays.Reserve(NumSourceUVChannels);
	for (int32 ChannelIdx = 0; ChannelIdx < NumSourceUVChannels; ++ChannelIdx)
	{
		SourceUVOverlays.Add(Mesh.Attributes()->GetUVLayer(ChannelIdx));
	}

	const TArray<FName> WeightLayerNames = InSourceMesh.GetWeightLayerNames();
	Mesh.Attributes()->SetNumWeightLayers(WeightLayerNames.Num());

	Mesh.Attributes()->SetNumPolygroupLayers(1);
	Geometry::FDynamicMeshPolygroupAttribute* BaseIDLayer = Mesh.Attributes()->GetPolygroupLayer(0);
	BaseIDLayer->SetName(MeshViewLocals::BaseIDOverlayLayerName);
	BaseIDLayer->Initialize(INDEX_NONE);

	for (Geometry::FDynamicMeshUVOverlay* Overlay : SourceUVOverlays)
	{
		Overlay->InitializeTriangles(InTriangles.Num());
	}

	TMap<FName, Geometry::FDynamicMeshWeightAttribute*> SubmeshWeightLayers;
	int CurrentWeightLayerIndex = 0;
	for (const FName& WeightLayerName : WeightLayerNames)
	{
		Geometry::FDynamicMeshWeightAttribute* ResultWeightLayer = Mesh.Attributes()->GetWeightLayer(CurrentWeightLayerIndex++);
		ResultWeightLayer->SetName(WeightLayerName);
		SubmeshWeightLayers.Add(WeightLayerName, ResultWeightLayer);
	}

	auto FindOrAddVertex = [&](int SourceVID)
	{
		// add the vertex if it doesn't already exist:
		if (!VertexMap.ContainsFrom(SourceVID))
		{
			const int32 VertexIndex = NextVertexIndex++;
			VertexMap.Add(SourceVID, VertexIndex);
			Geometry::FVertexInfo VertexInfo;
			VertexInfo.Position = InSourceMesh.GetVertex(SourceVID);
			Mesh.InsertVertex(VertexIndex, VertexInfo, bUnsafe);

			for (int32 ChannelIdx = 0; ChannelIdx < NumSourceUVChannels; ++ChannelIdx)
			{
				const FVector2f SourceUV = InSourceMesh.GetVertexUV(SourceVID, ChannelIdx);
				SourceUVOverlays[ChannelIdx]->InsertElement(VertexIndex, &SourceUV.X);
			}

			for (const FName& WeightLayerName : WeightLayerNames)
			{
				const float WeightLayerValue = InSourceMesh.GetWeightLayerValue(WeightLayerName, SourceVID);
				SubmeshWeightLayers[WeightLayerName]->SetValue(VertexIndex, &WeightLayerValue);
			}

			return VertexIndex;
		}
		return VertexMap.GetTo(SourceVID);
	};

	if constexpr (bUnsafe)
	{
		Mesh.BeginUnsafeTrianglesInsert();
		Mesh.BeginUnsafeVerticesInsert();
	}

	for (int SourceTriangleID : InTriangles)
	{
		const int TriangleIndex = NextTriangleIndex++;

		const Geometry::FIndex3i SourceTriangle = InSourceMesh.GetTriangle(SourceTriangleID);
		Geometry::FIndex3i Triangle;
		Triangle.A = FindOrAddVertex(SourceTriangle.A);
		Triangle.B = FindOrAddVertex(SourceTriangle.B);
		Triangle.C = FindOrAddVertex(SourceTriangle.C);

		TriangleMap.Add(SourceTriangleID, TriangleIndex);

		if (Geometry::EMeshResult::Ok == Mesh.InsertTriangle(
			TriangleIndex,
			Triangle,
			0,
			/* bUnsafe = */ bUnsafe))
		{
			for (Geometry::FDynamicMeshUVOverlay* Overlay : SourceUVOverlays)
			{
				Overlay->SetTriangle(TriangleIndex, Triangle);
			}
			BaseIDLayer->SetScalarValue(TriangleIndex, InSourceMesh.GetBaseID(SourceTriangleID));
		}
	}

	if constexpr (bUnsafe)
	{
		Mesh.EndUnsafeVerticesInsert();
		Mesh.EndUnsafeTrianglesInsert();
	}
}

bool FDynamicSubmesh::VertexExistsInBaseMesh(int InVertexID) const
{
	check(SourceMesh);
	const int FromID = VertexMap.GetFrom(InVertexID);
	return FromID != INDEX_NONE && SourceMesh->IsVertex(FromID);
}

void FDynamicSubmesh::RemapToParentMesh(const FMeshData& InNewSourceMesh)
{
	// Remove any attributes which are no longer in the parent mesh from the submesh.
	// Due to how the submesh works, it has to latch on to any attribute layers which were added (globally) to the mesh,
	// even if this specific submesh wasn't intending to read or even modify those layers.
	// At some point in the future, attributes will become sparse data and views which neither read/write to those channels
	// will not need to hold data for them.

	SourceMesh = &InNewSourceMesh;

	TMap<FName, Geometry::FDynamicMeshWeightAttribute*> SubmeshWeightLayers;
	if (Mesh.Attributes())
	{
		for (int Index = 0; Index < Mesh.Attributes()->NumWeightLayers(); ++Index)
		{
			Geometry::FDynamicMeshWeightAttribute* WeightLayer = Mesh.Attributes()->GetWeightLayer(Index);
			SubmeshWeightLayers.Add(WeightLayer->GetName(), WeightLayer);
		}

		const TSet<FName> SourceMeshWeightLayers = TSet<FName>(InNewSourceMesh.GetWeightLayerNames());

		auto RemoveWeightLayerByName = [](Geometry::FDynamicMeshAttributeSet* Attributes, const FName& WeightLayerName)
		{
			for (int Index = 0; Index < Attributes->NumWeightLayers(); ++Index)
			{
				if (Attributes->GetWeightLayer(Index)->GetName() == WeightLayerName)
				{
					Attributes->RemoveWeightLayer(Index);
					return true;
				}
			}
			return false;
		};

		for (const TPair<FName, Geometry::FDynamicMeshWeightAttribute*>& SubmeshWeightLayer : SubmeshWeightLayers)
		{
			if (!SourceMeshWeightLayers.Contains(SubmeshWeightLayer.Key))
			{
				ensure(RemoveWeightLayerByName(Mesh.Attributes(), SubmeshWeightLayer.Key));
			}
		}
	}
}

bool FDynamicSubmesh::GetSubmeshInternalBoundaryEdges(TSet<int>& OutSubmeshInternalBoundaryEdges) const
{
	// This function can fail if the submesh boundary edges are also boundary edges in the parent mesh,
	// or if the submesh modified the boundary edges.
	bool bAllBoundaryEdgesMapped = true;

	for (int SubmeshEdgeID : Mesh.BoundaryEdgeIndicesItr())
	{
		Geometry::FIndex2i SubmeshEdgeVIDs = Mesh.GetEdgeV(SubmeshEdgeID);

		int BaseMeshVIDA = MapVertexToBaseMesh(SubmeshEdgeVIDs.A);
		int BaseMeshVIDB = MapVertexToBaseMesh(SubmeshEdgeVIDs.B);

		Geometry::FIndex2i SourceEdgeVIDs(BaseMeshVIDA, BaseMeshVIDB);
		SourceEdgeVIDs.Sort();

		if (EdgesOnSubmeshBoundary.Contains(SourceEdgeVIDs))
		{
			OutSubmeshInternalBoundaryEdges.Add(SubmeshEdgeID);
		}
		else
		{
			bAllBoundaryEdgesMapped = false;
		}
	}

	return bAllBoundaryEdgesMapped;
}

TMap<FName, Geometry::FDynamicMeshWeightAttribute*> FDynamicSubmesh::GetSubmeshWeightLayers()
{
	TMap<FName, Geometry::FDynamicMeshWeightAttribute*> Result;
	if (Mesh.Attributes())
	{
		for (int Index = 0; Index < Mesh.Attributes()->NumWeightLayers(); ++Index)
		{
			Geometry::FDynamicMeshWeightAttribute* WeightLayer = Mesh.Attributes()->GetWeightLayer(Index);

			Result.Add(WeightLayer->GetName(), WeightLayer);
		}
	}
	return Result;
}

FMeshView::FMeshView(FMeshData* InMesh, FBox InBounds, EMeshViewComponents InReadComponents, EMeshViewComponents InWriteComponents, const TArray<FName>& InUsedChannels)
: MeshInternal(InMesh)
, Bounds(InBounds)
, ReadComponents(InReadComponents)
, WriteComponents(InWriteComponents)
, UsedChannels(InUsedChannels)
{
	ensure(MeshInternal);
}

bool FMeshView::Compare(const MeshPartition::FMeshView& InOther)
{
	return VertexIDs == InOther.VertexIDs
		&& VertexPositions == InOther.VertexPositions
		&& TriangleIDs == InOther.TriangleIDs
		&& VertexUVs == InOther.VertexUVs
		&& AttributeWeightChannels.OrderIndependentCompareEqual(InOther.AttributeWeightChannels);
}

void FMeshView::Build()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshView::Build);

	if (ensure(MeshInternal != nullptr))
	{
		const bool bWritesVertexPos = EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexPos);
		const bool bReadsVertexPos = EnumHasAnyFlags(ReadComponents, EMeshViewComponents::VertexPos);
		const bool bNeedsDynamicSubmesh = EnumHasAnyFlags(ReadComponents | WriteComponents, EMeshViewComponents::DynamicSubmesh);
		const bool bNeedsVertexAttributeWeight = EnumHasAnyFlags(ReadComponents | WriteComponents, EMeshViewComponents::VertexAttributeWeight);
		const bool bNeedsVertexUVs = EnumHasAnyFlags(ReadComponents | WriteComponents, EMeshViewComponents::VertexUVs);
		
		if (bNeedsVertexUVs)
		{
			VertexUVs.SetNum(MeshInternal->GetNumSourceUVChannels());
		}

		// if the mesh view is reading from a dynamic submesh, it is more efficient to build other components while building the submesh
		if (bNeedsDynamicSubmesh)
		{
			VertexIDs.Reset(0);
			
			TSet<Geometry::FIndex2i> EdgesOnSubmeshBoundary;
			for (int TriangleID : MeshInternal->TriangleIndicesItr())
			{
				FVector3d VA, VB, VC;
				MeshInternal->GetTriVertices(TriangleID, VA, VB, VC);
				if (Bounds.IsInside(VA) && Bounds.IsInside(VB) && Bounds.IsInside(VC))
				{
					TriangleIDs.Add(TriangleID);
				} 
				else if (Bounds.IsInside(VA) || Bounds.IsInside(VB) || Bounds.IsInside(VC))
				{
					TriangleIDsTouchingBounds.Add(TriangleID);
					
					const Geometry::FIndex3i Triangle = MeshInternal->GetTriangle(TriangleID);
					Geometry::FIndex2i EdgeA(Triangle.A, Triangle.B);
					Geometry::FIndex2i EdgeB(Triangle.B, Triangle.C);
					Geometry::FIndex2i EdgeC(Triangle.C, Triangle.A);

					EdgeA.Sort();
					EdgeB.Sort();
					EdgeC.Sort();
					EdgesOnSubmeshBoundary.Append( { EdgeA, EdgeB, EdgeC });
				}
			}

			Submesh = MeshPartition::FDynamicSubmesh(*MeshInternal, TriangleIDs, MoveTemp(EdgesOnSubmeshBoundary));

			for (int SubmeshVertexID : Submesh.GetSubmesh().VertexIndicesItr())
			{
				const int VertexID = Submesh.MapVertexToBaseMesh(SubmeshVertexID);
				const FVector3d Vertex = Submesh.GetSubmesh().GetVertex(SubmeshVertexID);

				if (Bounds.IsInside(Vertex))
				{
					const int ViewVertexIndex = VertexIDs.Emplace(VertexID);
					if (bWritesVertexPos)
					{
						VertexPositions.EmplaceAt(ViewVertexIndex, Vertex);
					}
				}
			}
		}
		// Otherwise all the remaining components are retrieved per-vertex:
		else
		{
			VertexIDs.Reset(0);

			for (int VertexID : MeshInternal->VertexIndicesItr())
			{
				const FVector3d Vertex = MeshInternal->GetVertex(VertexID);
				if (Bounds.IsInside(Vertex))
				{
					const int ViewVertexIndex = VertexIDs.Emplace(VertexID);

					// Writing to vertex positions requires building a buffer of all vertex positions since it currently reuses the same buffer for writes.
					// #todo: it may be a better idea to separate writebacks to the vertex positions into a separate buffer of pairs of (vid, pos) to allow sparse writes
					if (bReadsVertexPos || bWritesVertexPos)
					{
						VertexPositions.Emplace(Vertex);
					}
					
					if (bNeedsVertexUVs)
					{
						for (int UVChannelIndex = 0; UVChannelIndex < MeshInternal->GetNumSourceUVChannels(); ++UVChannelIndex)
						{
							VertexUVs[UVChannelIndex].Emplace(MeshInternal->GetVertexUV(VertexID, UVChannelIndex));
						}
					}
				}
			}
		}

		if (bNeedsVertexAttributeWeight)
		{
			for (const FName WeightLayerName : MeshInternal->GetWeightLayerNames())
			{
				if (!UsedChannels.Contains(WeightLayerName))
				{
					continue;
				}

				TArray<float> WeightChannelValues = MeshInternal->GetWeightLayerValues(WeightLayerName, VertexIDs);
				AttributeWeightChannels.Emplace(WeightLayerName, MoveTemp(WeightChannelValues));
			}
		}
	}
}

void FMeshView::RemapParentMesh(FMeshData* InNewMeshInternal)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshView::RemapParentMesh);

	MeshInternal = InNewMeshInternal;
	Submesh.RemapToParentMesh(*InNewMeshInternal);
}

FVector3d FMeshView::GetVertexPos(int InVertexIndex) const
{
	if (EnumHasAnyFlags(ReadComponents, EMeshViewComponents::DynamicSubmesh))
	{
		const int32 VertexID = VertexIDs[InVertexIndex];
		const int32 SubmeshVID = Submesh.MapVertexToSubmesh(VertexID);
		return Submesh.GetSubmesh().GetVertex(SubmeshVID);
	}
	else
	{
		ensureMsgf(
			EnumHasAnyFlags(ReadComponents, EMeshViewComponents::VertexPos),
			TEXT("Attempted to retrieve vertex ids but mesh view components doesn't contain vertex data")
		);
		return VertexPositions[InVertexIndex];
	}
}

float FMeshView::GetVertexAttributeWeight(FName InChannelName, int InVertexIndex) const
{
	ensureMsgf(EnumHasAnyFlags(ReadComponents, EMeshViewComponents::VertexAttributeWeight),
			TEXT("Attempted to retrieve a vertex weight but mesh view components doesn't contain vertex weight data"));

	const TArray<float>* WeightChannel = AttributeWeightChannels.Find(InChannelName);

	if ((WeightChannel == nullptr) || (InVertexIndex >= WeightChannel->Num()))
	{
		return 0.f;
	}

	return (*WeightChannel)[InVertexIndex];
}

FVector2f FMeshView::GetVertexUV(int InVertexIndex, int InUVChannelIndex) const
{
	if (EnumHasAnyFlags(ReadComponents, EMeshViewComponents::DynamicSubmesh))
	{
		const int32 VertexID = VertexIDs[InVertexIndex];
		const int32 SubmeshVID = Submesh.MapVertexToSubmesh(VertexID);
		
		const Geometry::FDynamicMeshUVOverlay* UVOverlay = Submesh.GetSubmesh().Attributes()->GetUVLayer(InUVChannelIndex); 
		if (UVOverlay)
		{
			TArray<int> Elements;
			int ElementID;
			// return any element. Mesh Data does not support split attributes.
			if (UVOverlay->FindAnyElementIDAtVertex(SubmeshVID, ElementID))
			{
				return UVOverlay->GetElement(ElementID);
			}
		}
	}
	else if (ensureMsgf(
		EnumHasAnyFlags(ReadComponents, EMeshViewComponents::VertexUVs),
		TEXT("Attempted to retrieve vertex uvs but mesh view components doesn't contain uv data")
	))
	{
		return VertexUVs[InUVChannelIndex][InVertexIndex];
	}
	return {};
}

void FMeshView::SetVertexPos(int InVertexIndex, FVector3d InNewVertexPos)
{
	const bool bCanWriteVertexPos = EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexPos);

	if (!ensureMsgf(bCanWriteVertexPos, TEXT("MegaMesh Modifier attempted to write a vertex position without declaring it's intention to do so in it's modifier descriptor")))
	{
		return;
	}

	if (!ensureMsgf(Bounds.IsInsideOrOn(InNewVertexPos), TEXT("MegaMesh Modifier attempted to move a vertex outside allowed bounds!")))
	{
		return;
	}

	// Reflect any new vertex positions in the submesh immediately
	if (EnumHasAnyFlags(ReadComponents, EMeshViewComponents::DynamicSubmesh))
	{
		const int32 VertexID = VertexIDs[InVertexIndex];
		const int SubmeshVID = Submesh.MapVertexToSubmesh(VertexID);
		Submesh.GetSubmesh().SetVertex(SubmeshVID, InNewVertexPos);
	}

	VertexPositions[InVertexIndex] = InNewVertexPos;
}

void FMeshView::SetVertexUV(int InVertexIndex, FVector2f InNewVertexUV, int InUVChannelIndex)
{
	const bool bCanWriteVertexUV = EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexUVs);

	if (!ensureMsgf(bCanWriteVertexUV, TEXT("MegaMesh Modifier attempted to write a vertex uv without declaring it's intention to do so in it's modifier descriptor")))
	{
		return;
	}

	// Reflect any new vertex positions in the submesh immediately
	if (EnumHasAnyFlags(ReadComponents, EMeshViewComponents::DynamicSubmesh))
	{
		const int32 VertexID = VertexIDs[InVertexIndex];
		const int SubmeshVID = Submesh.MapVertexToSubmesh(VertexID);
		Geometry::FDynamicMeshUVOverlay* UVOverlay = Submesh.GetSubmesh().Attributes()->GetUVLayer(InUVChannelIndex); 
		if (UVOverlay)
		{
			TArray<int> Elements;
			UVOverlay->GetVertexElements(SubmeshVID, Elements);
			for (int ElementID : Elements)
			{
				UVOverlay->SetElement(ElementID, InNewVertexUV);
			}
		}
	}

	VertexUVs[InUVChannelIndex][InVertexIndex] = InNewVertexUV;
}

void FMeshView::SetVertexAttributeWeight(FName InChannelName, int InVertexIndex, float InNewVertexWeight)
{
	const bool bCanWriteWeight = EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexAttributeWeight);

	if (!ensureMsgf(bCanWriteWeight, TEXT("MegaMesh Modifier attempted to write a vertex weight without declaring it's intention to do so in it's modifier descriptor")))
	{
		return;
	}
	if (!ensureMsgf(!EnumHasAnyFlags(WriteComponents, EMeshViewComponents::DynamicSubmesh),
					TEXT("MegaMesh Modifier attempted to write vertex weights on the view instead of the dynamic submesh.")))
	{
		return;
	}

	TArray<float>* WeightChannel = AttributeWeightChannels.Find(InChannelName);
	if (!ensureMsgf(WeightChannel != nullptr, 
					TEXT("Modifier attempted to write vertex weights to a channel not found on view. Channels must be declared in FInstanceInfo::UsedChannels.")))
	{
		return;
	}

	if (!ensureMsgf(WeightChannel->IsValidIndex(InVertexIndex),
					TEXT("Modifier attempt to write vertex weights to invalid vertex index, %d. Channel has %d vertices"), InVertexIndex, WeightChannel->Num()))
	{
		return;
	}

	(*WeightChannel)[InVertexIndex] = InNewVertexWeight;
}

void FMeshView::Writeback()
{
	using namespace UE::MeshPartition;

	if (EnumHasAnyFlags(WriteComponents, EMeshViewComponents::DynamicSubmesh))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshView::WritebackSubmeshData);

		ensureMsgf(
			!EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexPos),
			TEXT("MeshView is writing to dynamic submesh and vertex pos at the same time, this doesn't make sense as dynamic submesh is a stronger way to write back to vertex pos!")
		);

		Geometry::FDynamicMesh3& DynamicMesh = Submesh.GetSubmesh();

		for (int TriangleID : TriangleIDs)
		{
			MeshInternal->RemoveTriangle(TriangleID, /* bRemoveIsolatedVertices = */ true);
		}

		const TArray<FName>& WeightLayerNames = MeshInternal->GetWeightLayerNames();
		TMap<FName, Geometry::FDynamicMeshWeightAttribute*> SubmeshWeightLayers = Submesh.GetSubmeshWeightLayers();

		const int32 SubmeshNumUVLayers = FMath::Min(DynamicMesh.Attributes()->NumUVLayers(), MeshPartition::FMeshData::MAX_SOURCE_UV_CHANNELS);

		// Note the submesh path always reads/writes UVs, because otherwise submesh-based modifiers will clear existing UVs
		// but we only grow additional UV layers if the write flag is enabled.
		// #todo: should we control the number/growth of UV layers more systematically in the definition, or in the modifier instance infos?
		if (EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexUVs))
		{
			if (MeshInternal->GetNumSourceUVChannels() < SubmeshNumUVLayers)
			{
				MeshInternal->SetNumSourceUVChannels(SubmeshNumUVLayers);
			}
		}

		TArray<Geometry::FDynamicMeshUVOverlay*> SubmeshUVOverlays;
		SubmeshUVOverlays.Reserve(SubmeshNumUVLayers);
		for (int32 ChannelIdx = 0; ChannelIdx < SubmeshNumUVLayers; ++ChannelIdx)
		{
			SubmeshUVOverlays.Add(DynamicMesh.Attributes()->GetUVLayer(ChannelIdx));
		}

		// It's possible that some modifiers do not respect the weight and polygroup layers.
		// For polygroup layers, this is an error on the part of the modifier author as we lose the information about which base
		// a triangle originated from. It's important that this information remains present on the dynamic mesh as it is preserved
		// through splits/merges and when we reinsert, the information is still there. As a fallback, assign everything to the 
		// base with index 0.
		const Geometry::FDynamicMeshPolygroupAttribute* SubmeshBaseIDLayer = nullptr;

		for (int PolygroupIndex = 0; PolygroupIndex < DynamicMesh.Attributes()->NumPolygroupLayers(); ++PolygroupIndex)
		{
			const Geometry::FDynamicMeshPolygroupAttribute* PolygroupLayer = DynamicMesh.Attributes()->GetPolygroupLayer(PolygroupIndex);
			if (PolygroupLayer->GetName() == MeshViewLocals::BaseIDOverlayLayerName)
			{
				SubmeshBaseIDLayer = PolygroupLayer;
			}
		}

		ensureMsgf(SubmeshBaseIDLayer, TEXT("MegaMeshMeshView's DynamicSubmesh is missing the polygroup layer to indicate the source Base index. This is required for channel texture generation"));

		TMap<int, int> SubmeshToFinalVID;

		SubmeshToFinalVID.Reserve(DynamicMesh.VertexCount());
		MeshInternal->ReserveAdditionalVertices(DynamicMesh.VertexCount());
		for (int SubmeshVertexIndex : DynamicMesh.VertexIndicesItr())
		{
			const FVector3d Position = DynamicMesh.GetVertex(SubmeshVertexIndex);
			const int FinalVID = MeshInternal->AppendVertex(Position);
			SubmeshToFinalVID.Add(SubmeshVertexIndex, FinalVID);

			for (const FName& WeightLayerName : WeightLayerNames)
			{
				// A cached submesh may not have weight layers that were added by modifiers loaded after caching occurred.
				if (SubmeshWeightLayers.Contains(WeightLayerName))
				{
					float WeightLayerValue = 0.;
					SubmeshWeightLayers[WeightLayerName]->GetValue(SubmeshVertexIndex, &WeightLayerValue);
					MeshInternal->SetWeightLayerValue(WeightLayerName, FinalVID, WeightLayerValue);
				}
			}
		}

		MeshInternal->ReserveAdditionalTriangles(DynamicMesh.TriangleCount());
		const int32 CopyNumUVLayers = FMath::Min(MeshInternal->GetNumSourceUVChannels(), SubmeshUVOverlays.Num());
		for (int SubmeshTriangleIndex : DynamicMesh.TriangleIndicesItr())
		{
			const Geometry::FIndex3i Triangle = DynamicMesh.GetTriangle(SubmeshTriangleIndex);
			Geometry::FIndex3i FinalTriangle;
			FinalTriangle.A = SubmeshToFinalVID[Triangle.A];
			FinalTriangle.B = SubmeshToFinalVID[Triangle.B];
			FinalTriangle.C = SubmeshToFinalVID[Triangle.C];
			const int FinalTID = MeshInternal->AppendTriangle(FinalTriangle);

			int BaseID = 0;
			if (SubmeshBaseIDLayer)
			{
				SubmeshBaseIDLayer->GetValue(SubmeshTriangleIndex, &BaseID);
			}
			MeshInternal->SetBaseID(FinalTID, BaseID);

			for (int32 ChannelIdx = 0; ChannelIdx < CopyNumUVLayers; ++ChannelIdx)
			{
				Geometry::FDynamicMeshUVOverlay* SourceOverlay = SubmeshUVOverlays[ChannelIdx];
				Geometry::FIndex3i ElementTri = SourceOverlay->GetTriangle(SubmeshTriangleIndex);

				if (ElementTri.A != INDEX_NONE)
				{
					for (int TriangleVertexIndex = 0; TriangleVertexIndex < 3; ++TriangleVertexIndex)
					{
						const int FinalVertexIndex = FinalTriangle[TriangleVertexIndex];
						const FVector2f SourceUV = SourceOverlay->GetElement(ElementTri[TriangleVertexIndex]);
						MeshInternal->SetVertexUV(FinalVertexIndex, SourceUV, ChannelIdx);
					}
				}
			}
		}

		TArray<TPair<int, int>> MergePairs;
		
		// Find all boundary vertices which were present before the submesh operation and merge them with the corresponding vertices of the source mesh.
		for (int SubmeshEdgeID : DynamicMesh.BoundaryEdgeIndicesItr())
		{
			const Geometry::FDynamicMesh3::FEdge Edge = DynamicMesh.GetEdge(SubmeshEdgeID);

			// Only merge vertex pairs of the submesh for vertices on the submesh boundary at the time of the submesh creation.
			Geometry::FIndex2i SourceMeshEdge = { Submesh.MapVertexToBaseMesh(Edge.Vert.A), Submesh.MapVertexToBaseMesh(Edge.Vert.B) };
			SourceMeshEdge.Sort();

			if (!Submesh.EdgesOnSubmeshBoundary.Contains(SourceMeshEdge))
			{
				continue;
			}

			for (int SubmeshVertexID : { Edge.Vert.A, Edge.Vert.B })
			{
				if (!Submesh.VertexExistsInBaseMesh(SubmeshVertexID))
				{
					continue;
				}
				const int BaseVertexID = Submesh.MapVertexToBaseMesh(SubmeshVertexID);

				MergePairs.Emplace(SubmeshToFinalVID[SubmeshVertexID], BaseVertexID);
			}
		}

		MeshInternal->MergeVertexPairs(MergePairs, { TriangleIDsTouchingBounds });
	}
	else if (EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexPos))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshView::WritebackVertexPositions);

		for (int VertexIndex = 0; VertexIndex < VertexIDs.Num(); ++VertexIndex)
		{
			const int VertexID = VertexIDs[VertexIndex];
			const FVector3d VertexPos = VertexPositions[VertexIndex];
			MeshInternal->SetVertex(VertexID, VertexPos);
		}
	}

	if (EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexAttributeWeight) && !EnumHasAnyFlags(WriteComponents, EMeshViewComponents::DynamicSubmesh))
	{
		for (const TPair<FName, TArray<float>>& Channel : AttributeWeightChannels)
		{
			const FName& ChannelName = Channel.Key;
			const TArray<float>& Weights = Channel.Value;

			MeshInternal->SetWeightLayerValues(ChannelName, VertexIDs, Weights);
		}
	}

	if (EnumHasAnyFlags(WriteComponents, EMeshViewComponents::VertexUVs) && !EnumHasAnyFlags(WriteComponents, EMeshViewComponents::DynamicSubmesh))
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshView::WritebackUVs);

		for (int UVChannelIndex = 0; UVChannelIndex < VertexUVs.Num(); ++UVChannelIndex)
		{
			const TArray<FVector2f>& UVChannel = VertexUVs[UVChannelIndex];
			for (int VertexIndex = 0; VertexIndex < VertexIDs.Num(); ++VertexIndex)
			{
				const int VertexID = VertexIDs[VertexIndex];
				const FVector2f VertexUV = UVChannel[VertexIndex];
				MeshInternal->SetVertexUV(VertexID, VertexUV, UVChannelIndex);
			}
		}
	}
}

void FMeshView::Release()
{
	Submesh = MeshPartition::FDynamicSubmesh{};
	VertexIDs.Empty();
	TriangleIDs.Empty();
	VertexPositions.Empty();
	VertexUVs.Empty();
	AttributeWeightChannels.Empty();
	TriangleIDsTouchingBounds.Empty();
}

bool FMeshView::GetSubmeshInternalBoundaryEdges(TSet<int32>& OutSubmeshEIDs) const
{
	return Submesh.GetSubmeshInternalBoundaryEdges(OutSubmeshEIDs);
}

double FMeshView::GetMemoryUsageMB() const
{
	uint64 ByteCount = Submesh.GetByteCount();
	ByteCount += VertexIDs.GetAllocatedSize();
	ByteCount += VertexPositions.GetAllocatedSize();
	ByteCount += VertexUVs.GetAllocatedSize();
	for (const TArray<FVector2f>& UVs : VertexUVs)
	{
		ByteCount += UVs.GetAllocatedSize();
	}
	ByteCount += TriangleIDs.GetAllocatedSize();
	ByteCount += TriangleIDsTouchingBounds.GetAllocatedSize();
	for (const TPair<FName, TArray<float>>& Pair : AttributeWeightChannels)
	{
		ByteCount += Pair.Value.GetAllocatedSize();
	}
	return static_cast<double>(ByteCount) / (1024 * 1024);
}
} // namespace UE::MeshPartition
