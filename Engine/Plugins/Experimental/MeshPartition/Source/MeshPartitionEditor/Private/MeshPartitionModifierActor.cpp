// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModifierActor.h"
#include "MeshPartitionModifierComponent.h"

#include "ModelingObjectsCreationAPI.h"

#include "AssetRegistry/AssetData.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Modifiers/MeshPartitionSplineModifier.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionModifierActor)
#define LOCTEXT_NAMESPACE "MegaMeshModifierActor"

namespace UE::MeshPartition
{
namespace ModifierActorLocals
{
	USceneComponent* CreateNewComponentOnActor(AActor* TargetActor, UClass* ComponentType)
	{
		if (!ensure(TargetActor && ComponentType))
		{
			return nullptr;
		}
		
		TargetActor->Modify();
		FName NewComponentName;
		if (ComponentType->HasMetaData(TEXT("DisplayName")))
		{
			NewComponentName = *ComponentType->GetMetaData(TEXT("DisplayName"));
			while (!FComponentEditorUtils::IsComponentNameAvailable(NewComponentName.ToString(), TargetActor))
			{
				NewComponentName.SetNumber(NewComponentName.GetNumber() + 1);
			}
		}
		else
		{
			NewComponentName = *FComponentEditorUtils::GenerateValidVariableName(ComponentType, TargetActor);
		}
		USceneComponent* Component = NewObject<USceneComponent>(TargetActor, ComponentType, NewComponentName, RF_Transactional);
		Component->SetupAttachment(TargetActor->GetRootComponent());
		Component->OnComponentCreated();
		TargetActor->AddInstanceComponent(Component);
		Component->RegisterComponent();
		Component->ResetRelativeTransform();
		return Component;
	}
	
	USplineComponent* CreateSplineComponent(AModifierActor* NewActor, const double SplineLength)
	{
		auto SetScaledSpline = [SplineLength](USplineComponent & Component)
		{
			const FVector StartPoint(0.0, 0.0, 0.0);
			const FVector EndPoint(SplineLength, 0.0, 0.0);
			constexpr float StartParam = 0.f;
			constexpr float EndParam = 1.f;

			Component.ClearSplinePoints(false);
			Component.AddPoint(FSplinePoint(StartParam, StartPoint), false);
			Component.AddPoint(FSplinePoint(EndParam, EndPoint), false);
			Component.UpdateSpline();
		};

		USplineComponent* NewSplineComponent = StaticCast<USplineComponent*>(CreateNewComponentOnActor(NewActor, USplineComponent::StaticClass()));
		SetScaledSpline(*NewSplineComponent);
		return NewSplineComponent;
	}
	
	void PostSpawnModifierActor(AModifierActor* NewActor, UModifierComponent* ModifierComponent, const AActor* ClosestActor)
	{
		if (!ensure(NewActor && ModifierComponent))
		{
			return;
		}
		
		if (ModifierComponent->IsA(MeshPartition::USplineModifier::StaticClass()))
		{
			FBox BoxExtents = ClosestActor ? ClosestActor->GetComponentsBoundingBox() : FBox(-FVector::OneVector, FVector::OneVector);
			const double SplineLength = 0.5 * BoxExtents.GetExtent().Length();
			USplineComponent* NewSplineComponent = CreateSplineComponent(NewActor, SplineLength);
			MeshPartition::USplineModifier* SplineModifier = StaticCast<MeshPartition::USplineModifier*>(ModifierComponent);
			SplineModifier->SetSplineComponent(NewSplineComponent, false /*don't update now*/);
		}
	}
}

AModifierActor::AModifierActor()
	: Modifier(nullptr)
{	
	DefaultSceneRoot = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"));
	SetRootComponent(DefaultSceneRoot);
}

UModifierActorFactory::UModifierActorFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DisplayName = LOCTEXT("MegaMeshModifierActorDisplayName", "Mesh Partition Modifier Actor");
	NewActorClass = MeshPartition::AModifierActor::StaticClass();
}

bool UModifierActorFactory::CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg)
{
	if (UActorFactory::CanCreateActorFrom(AssetData, OutErrorMsg))
	{
		return true;
	}

	if (AssetData.IsValid())
	{
		if (AssetData.IsInstanceOf(AModifierActor::StaticClass()))
		{
			return true;
		}
		if (const UClass* AssetClass = Cast<UClass>(AssetData.GetAsset()))
        {
        	if (AssetClass->IsChildOf(UModifierComponent::StaticClass()))
        	{
        		return true;
        	}
        }
	}

	OutErrorMsg = NSLOCTEXT("CanCreateActor", "NoMegaMeshModifier", "A valid Mesh Partition Modifier must be specified.");
	return false;
}

AActor* UModifierActorFactory::SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams)
{
	MeshPartition::AModifierActor* NewActor = Cast<MeshPartition::AModifierActor>(Super::SpawnActor(InAsset, InLevel, InTransform, InSpawnParams));
	if (NewActor)
	{
		UClass* AssetClass = Cast<UClass>(InAsset);
		if (AssetClass && AssetClass->IsChildOf(UModifierComponent::StaticClass()))
		{
			USceneComponent* NewComponent = ModifierActorLocals::CreateNewComponentOnActor(NewActor, AssetClass);
			if (UModifierComponent* ModifierComponent = Cast<UModifierComponent>(NewComponent))
			{
				if (!NewActor->bIsEditorPreviewActor)
				{
					NewActor->Modifier = ModifierComponent;
					AActor* ClosestActor = ModifierComponent->BindToNearestMeshPartition();
					ModifierActorLocals::PostSpawnModifierActor(NewActor, ModifierComponent, ClosestActor);
					if (MeshPartition::AMeshPartition* MeshPartition = ModifierComponent->GetAffectedMeshPartition())
					{
						NewActor->AttachToActor(MeshPartition, FAttachmentTransformRules::KeepWorldTransform);
					}
				}
			}
			NewActor->PostEditChange();
		}
	}
	return NewActor;
}

FString UModifierActorFactory::GetDefaultActorLabel(UObject* Asset) const
{
	if (const UClass* AssetClass = Cast<UClass>(Asset))
	{
		if (AssetClass->HasMetaData(TEXT("DisplayName")))
		{
			return AssetClass->GetMetaData(TEXT("DisplayName"));
		}
	}
	return Super::GetDefaultActorLabel(Asset);
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
