// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModifierComponent.h"

#include "Algo/AnyOf.h"
#include "Components/BillboardComponent.h"
#include "CoreGlobals.h" // GIsTransacting
#include "Engine/Texture2D.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionDefinition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "MeshPartitionActorDescUtils.h"
#include "MeshPartitionDependencyContext.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionEditorUtils.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "MeshPartitionModifierComponentDesc.h"

#include "PrimitiveSceneProxy.h"
#include "MeshElementCollector.h"
#include "SceneView.h"
#include "PrimitiveDrawingUtils.h"
#include "UObject/ICookInfo.h"
#include "UObject/UObjectIterator.h"

// For TEDS 
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"

const FName MegaMeshModifierProperties::PropertiesVersionNumberName = TEXT("MegaMeshPropertiesVersionNumber");
const FName MegaMeshModifierProperties::MegaMeshModifiersNum = TEXT("MegaMeshModifiersNum");
const FName MegaMeshModifierProperties::MegaMeshModifierPath = TEXT("MegaMeshModifierPath");
const FName MegaMeshModifierProperties::MegaMeshGUID = TEXT("MegaMeshGUID");
const FName MegaMeshModifierProperties::Class = TEXT("MegaMeshModifierClass");
const FName MegaMeshModifierProperties::BaseGrowth = TEXT("MegaMeshBaseGrowth");
const FName MegaMeshModifierProperties::Type = TEXT("MegaMeshModifierType");
const FName MegaMeshModifierProperties::Priority = TEXT("MegaMeshModifierPriority");
const FName MegaMeshModifierProperties::Complexity = TEXT("MegaMeshModifierComplexity");
const FName MegaMeshModifierProperties::ComplexityMultiplier = TEXT("MegaMeshModifierComplexityMultiplier");
const FName MegaMeshModifierProperties::IsContiguous = TEXT("MegaMeshModifierIsContiguous");
const FName MegaMeshModifierProperties::IsDisabled = TEXT("MegaMeshModifierIsDisabled");

const uint32 MegaMeshModifierProperties::PropertiesVersionNumber = 4;

namespace UE::MeshPartition
{
const FName MegaMeshModifierType::Base = TEXT("Base");

namespace MegaMeshModifierComponentLocals
{
	class FMegaMeshModifierComponentSceneProxy final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FMegaMeshModifierComponentSceneProxy(const MeshPartition::UModifierComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
		{
			InitializeFromModifier(InComponent);
		}

		void InitializeFromModifier(const MeshPartition::UModifierComponent* InComponent)
		{
			bDrawBounds = InComponent->ShouldDrawBoundingBox();
			bIsBase = InComponent->IsBase();
			BoundingBox = InComponent->ComputeCombinedBounds();
#if WITH_EDITORONLY_DATA
			LineColor = InComponent->EditorUnselectedModifierColor();
#else
			LineColor = FLinearColor::White;
#endif
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_MegaMeshModifierComponentSceneProxy_GetDynamicMeshElements);

			if (bIsBase) // Don't draw proxy for base modifiers
			{
				return;
			}

			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];
					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					// Taking into account the min and maximum drawing distance
					const float DistanceSqr = (View->ViewMatrices.GetViewOrigin() - GetLocalToWorld().GetOrigin()).SizeSquared();
					if (DistanceSqr < FMath::Square(GetMinDrawDistance()) || DistanceSqr > FMath::Square(GetMaxDrawDistance()))
					{
						continue;
					}

					if (bDrawBounds)
					{
						DrawWireBox(PDI, BoundingBox, LineColor, SDPG_World, 3.0f, 1.0f, true);
					}
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			FPrimitiveViewRelevance Result;
			// TODO: Add support for show flags like we do for splines... 
			bool bDrawRelevance = bDrawBounds && IsShown(View); 

			Result.bDrawRelevance = bDrawRelevance;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}

		virtual uint32 GetMemoryFootprint(void) const override { return sizeof(*this) + GetAllocatedSize(); }
		uint32 GetAllocatedSize(void) const { return FPrimitiveSceneProxy::GetAllocatedSize(); }

	private:
		bool bDrawBounds;
		bool bIsBase;
		FBox BoundingBox;
		FLinearColor LineColor;
	};

	static FString SpritePath = TEXT("/MeshPartition/Icons/S_ModifierIcon.S_ModifierIcon");
	static FAutoConsoleVariableRef CVarSpritePath(
		TEXT("MegaMesh.Modifiers.SpritePath"),
		SpritePath,
		TEXT("Sets the path of the icon to use for the editor-only sprite billboard"),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
		{
			SetSpritePath(SpritePath);
		}));

	void SetSpritePath(const FString& Path)
	{
		UTexture2D* EditorSpriteTexture = LoadObject<UTexture2D>(
			nullptr, Path);
		if (!EditorSpriteTexture)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "Unable to load \"%ls\".", *Path);
			return;
		}

		for (TObjectIterator<UModifierComponent> It; It; ++It)
		{
			UModifierComponent* Modifier = *It;
			if (Modifier && Modifier->SpriteComponent)
			{
				bool bShouldRegister = false;
				if (Modifier->IsRegistered())
				{
					Modifier->SpriteComponent->UnregisterComponent();
					bShouldRegister = true;
				}

				Modifier->SpriteComponent->Sprite = EditorSpriteTexture;
				
				if (bShouldRegister)
				{
					Modifier->SpriteComponent->RegisterComponent();
				}
			}
		}
	}

	static float SpriteSize = 0.5f;
	static FAutoConsoleVariableRef CVarSpriteSize(
		TEXT("MegaMesh.Modifiers.SpriteSize"),
		SpriteSize,
		TEXT("Sets the size of the editor-only sprite billboard"),
		FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
			{
				SetSpriteSize(SpriteSize);
			}));

	void SetSpriteSize(float Size)
	{
		for (TObjectIterator<UModifierComponent> It; It; ++It)
		{
			UModifierComponent* Modifier = *It;
			if (Modifier && Modifier->SpriteComponent)
			{
				bool bShouldRegister = false;
				if (Modifier->IsRegistered())
				{
					Modifier->SpriteComponent->UnregisterComponent();
					bShouldRegister = true;
				}

				Modifier->SpriteComponent->SetWorldScale3D(FVector(MegaMeshModifierComponentLocals::SpriteSize));
				Modifier->SpriteComponent->SetVisibility(Modifier->ShouldShowSpriteComponent() && Size > 0);

				if (bShouldRegister)
				{
					Modifier->SpriteComponent->RegisterComponent();
				}
			}
		}
	}
}

IModifierBackgroundOp::IModifierBackgroundOp(const FName& InOperationName)
: OperationName(InOperationName)
{
}

UModifierComponent::UModifierComponent()
	: PreviewSection(nullptr)
	, AffectedMegaMesh(nullptr)
	, Type(NAME_None)
	, CacheKey(HasAnyFlags(RF_ClassDefaultObject) ? FGuid() : FGuid::NewGuid())
	, LastAppliedBounds()
{
}

void UModifierComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(UE::MeshPartition::FCustomVersion::GUID);

	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType().IsNone())
	{
		const FName PreviousModifierDefaultType = TEXT("Misc");
		SetType(PreviousModifierDefaultType);
	}
}

void UModifierComponent::PreEditChange(FProperty* InPropertyAboutToChange)
{
	Super::PreEditChange(InPropertyAboutToChange);

	if (InPropertyAboutToChange == nullptr || !IsInRelevantWorld())
	{
		return;
	}
	
	const FName PropertyName = InPropertyAboutToChange->GetFName();
	UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent();
	
	if ((PropertyName == GET_MEMBER_NAME_CHECKED(UModifierComponent, AffectedMegaMesh)) && (EditorComponent != nullptr))
	{
		// AffectedMegaMesh is about to change. Tell the previous one to update where we were (and any in flight bounds), and remove it from its
		//  current modifiers. This will happen on next tick, so it will see the change at that point.
		EditorComponent->OnModifierAssigned();
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void UModifierComponent::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (!IsInRelevantWorld())
	{
		return;
	}

	const FName MemberPropertyName = InPropertyChangedEvent.MemberProperty ? InPropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	
	if (MemberPropertyName == GET_MEMBER_NAME_CHECKED(MeshPartition::UModifierComponent, AffectedMegaMesh))
	{
		if (UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent())
		{
			EditorComponent->OnModifierAssigned();
		}
		else
		{
			// Mark ourselves as not registered
			bNeedToRegisterWithMeshPartition = true;
		}
	}

	//todo(luc.eygasier): refactor this when async OnChanged is implemented
	PropertyChanged(InPropertyChangedEvent);

	// Trigger a transient state change if the changed property is a transient property.
	EChangeType ModifierChangeType = EChangeType::StateChange;
	if (FProperty* ChangedProperty = InPropertyChangedEvent.Property)
	{
		if (EnumHasAnyFlags(ChangedProperty->GetPropertyFlags(), CPF_Transient))
		{
			ModifierChangeType = EChangeType::TransientStateChange;
		}
	}

	OnChanged(ComputeBounds(), ModifierChangeType);
}

#if WITH_EDITOR
void UModifierComponent::PreEditUndo()
{
	// Remember our current mesh partition so we can detect changes to it across undo/redo
	PreUndoMeshPartition = GetAffectedMeshPartition();
	Super::PreEditUndo();
}

void UModifierComponent::PostEditUndo()
{
	if (!IsInRelevantWorld())
	{
		Super::PostEditUndo();
		return;
	}

	AMeshPartition* CurrentMegaMesh = GetAffectedMeshPartition();
	if (PreUndoMeshPartition != CurrentMegaMesh)
	{
		// Make sure we unregister from the previous MegaMesh
		UMeshPartitionEditorComponent* PreviousMeshPartitionComponent = PreUndoMeshPartition ?
			Cast<UMeshPartitionEditorComponent>(PreUndoMeshPartition->GetMeshPartitionComponent()) : nullptr;
		if (PreviousMeshPartitionComponent)
		{
			PreviousMeshPartitionComponent->OnModifierAssigned();
		}

		// Register with the new MegaMesh
		bNeedToRegisterWithMeshPartition = true;
		RegisterWithMegaMeshIfNeeded();
	}

	// This will end up triggering OnChanged through PostEditChangeProperty, though it will
	//  likely also be called through OnRegister
	Super::PostEditUndo();
}
#endif // WITH_EDITOR

void UModifierComponent::PostDuplicate(bool bDuplicateForPIE)
{
	if (!bDuplicateForPIE && IsInRelevantWorld())
	{	// A duplicated modifier must be assigned a new modifier GUID  (by default it will assign the class default GUID)
		UpdateCacheKey(true);
	}
}

void UModifierComponent::OnRegister()
{
#if WITH_EDITORONLY_DATA
	if (!SpriteComponent)
	{
		// Prevents USceneComponent from creating the SpriteComponent in OnRegister so that we can do 
		//  it ourselves in CreateModifierSpriteComponent
		bVisualizeComponent = false;
	}
#endif

	Super::OnRegister();

	if (!IsInRelevantWorld())
	{
		return;
	}

	AActor* Owner = GetOwner();
	AMeshPartition* MeshPartition = GetAffectedMeshPartition();
	
	if ((IsBase()) && (Owner != nullptr) && (MeshPartition != nullptr))
	{
		Owner->AttachToActor(MeshPartition, FAttachmentTransformRules::KeepRelativeTransform);

		#if WITH_EDITORONLY_DATA
		Owner->SetLockLocation(true);
		#endif // WITH_EDITORONLY_DATA

	}

	// This is a main place where we push an update to the Mesh Partition, because it gets hit in a variety
	//  of scenarios that are otherwise hard to catch: BP construction script rerun, movement of parent
	//  component via detail panel, undo/redo of parent movement.
	// Strictly speaking, we shouldn't need the OnChanged call if we trigger the registration call, so
	//  we could check bNeedToRegisterWithMeshPartition here and do one or the other. But it doesn't hurt,
	//  and makes us robust to a scenario where the component IS registered but doesn't realize it,
	//  and fails to trigger an update for that reason.
	RegisterWithMegaMeshIfNeeded();
	// this is not a "real" change to the modifier, we are only requesting an update within this modifiers bounds.
	OnChanged(ComputeBounds(), EChangeType::TransientChange);

#if WITH_EDITOR
	{
		// TODO: When Actor Components get general registration into TEDS, we can remove this and let the base classes handle it.
		using namespace UE::Editor::DataStorage;
		ICompatibilityProvider* Storage = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
		if (Storage)
		{
			Storage->AddCompatibleObject(const_cast<UModifierComponent*>(this));
		}
	}

	CreateModifierSpriteComponent();
#endif
}

void UModifierComponent::OnUnregister()
{
	if (!IsInRelevantWorld())
	{
		Super::OnUnregister();
		return;
	}

	SelectionOverrideDelegate.Unbind();

	// In most cases we don't need to trigger changes here because we'll hit other update paths, but it does seem to be the
	//  only reliable place to handle undo of initial creation or redoing deletion. So we'll condition this on whether
	//  we're dealing with undo/redo.
	if (GIsTransacting)
	{
		// Do the same things we do for modifier destruction
		if (UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent())
		{
			EditorComponent->OnModifierAssigned();
		}
		// this is not a "real" change to the modifier, we are only requesting an update within this modifiers bounds.
		OnChanged(ComputeBounds(), EChangeType::TransientChange);
		bNeedToRegisterWithMeshPartition = true;
	}

	Super::OnUnregister();
}

void UModifierComponent::OnComponentDestroyed(bool bInDestroyingHierarchy)
{
	if (!IsInRelevantWorld())
	{
		Super::OnComponentDestroyed(bInDestroyingHierarchy);
		return;
	}

	// Note that even if we could handle things purely from the megamesh listening to level actor list changes, we would
	//  still need to call OnModifierAssigned to handle the case of the component being individually deleted from its actor
	//  without deleting the actor. But we can't since we will only update on tick, at which point things might be outdated.
	if (UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent())
	{
		EditorComponent->OnModifierAssigned();
	}
	// Need to invalidate any in-flight bounds and our current location, in case we come back with changed parameters.
	OnChanged(ComputeBounds(), EChangeType::StateChange);

	// If we undelete, we will need to re-register.
	bNeedToRegisterWithMeshPartition = true;
	
	Super::OnComponentDestroyed(bInDestroyingHierarchy);
}

TUniquePtr<class FWorldPartitionComponentDesc> UModifierComponent::CreateClassComponentDesc() const
{
	return MakeUnique<FWorldPartitionModifierComponentDesc>();
}


TStructOnScope<FActorComponentInstanceData> UModifierComponent::GetComponentInstanceData() const
{
	return MakeStructOnScope<FActorComponentInstanceData, FModifierComponentInstanceData>(this);
}

FBox UModifierComponent::GetStreamingBoundsEditor() const
{
	return ComputeCombinedBounds();
}

void UModifierComponent::PostEditComponentMove(bool bFinished)
{
	Super::PostEditComponentMove(bFinished);

	if ((bFinished || IsInteractive()) && IsInRelevantWorld())
	{
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void UModifierComponent::OnUpdateTransform(EUpdateTransformFlags InUpdateTransformFlags, ETeleportType InTeleport)
{
	Super::OnUpdateTransform(InUpdateTransformFlags, InTeleport);

	UpdateCacheKey(/* bInChangeCacheKey */ true);
}

FBoxSphereBounds UModifierComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	return ComputeCombinedBounds().InverseTransformBy(GetComponentTransform()).TransformBy(LocalToWorld);
}

FPrimitiveSceneProxy* UModifierComponent::CreateSceneProxy()
{
	return new MegaMeshModifierComponentLocals::FMegaMeshModifierComponentSceneProxy(this);
}

void UModifierComponent::PrepareResources()
{
	UE::Tasks::FTask WaitTask = UE::Tasks::Launch(TEXT("UModifierComponent::PrepareResources::Wait"), []() {},
									UE::Tasks::Prerequisites(GetAsyncPrepareResourcesTask()),
									UE::Tasks::ETaskPriority::Normal,
									UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
	WaitTask.Wait();
}

void UModifierComponent::MarkAsRegisteredWithMeshPartition(const AMeshPartition* MeshPartitionIn)
{
	ensure(MeshPartitionIn && GetAffectedMeshPartition() == MeshPartitionIn);
	bNeedToRegisterWithMeshPartition = false;
}

AMeshPartition* UModifierComponent::GetAffectedMeshPartition() const
{
	return AffectedMegaMesh.Get();
}

UMeshPartitionEditorComponent* UModifierComponent::GetMeshPartitionEditorComponent() const
{
	const AMeshPartition* MegaMesh = GetAffectedMeshPartition();
	
	if (MegaMesh == nullptr)
	{
		return nullptr;
	}

	return Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent());
}

void UModifierComponent::SetAffectedMeshPartition(AMeshPartition* InMeshPartition)
{
	if (AffectedMegaMesh != InMeshPartition)
	{
		bNeedToRegisterWithMeshPartition = true;
	}
	AffectedMegaMesh = InMeshPartition;
}

void UModifierComponent::BP_SetAffectedMegaMesh(AMeshPartition* InMegaMesh)
{
	if (InMegaMesh == AffectedMegaMesh)
	{
		return;
	}

	// Unregister from the previous MegaMesh
	if (UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent())
	{
		AffectedMegaMesh = nullptr;
		EditorComponent->OnModifierAssigned();
	}

	SetAffectedMeshPartition(InMegaMesh);
	RegisterWithMegaMeshIfNeeded();
}

AActor* UModifierComponent::BindToNearestMeshPartition()
{
	if (!IsInRelevantWorld())
	{
		return nullptr;
	}

	UWorld* World = GetWorld();
	if (!ensure(World))
	{
		return nullptr;
	}

	const FVector3d CurrentLocation = GetComponentToWorld().GetLocation();
	double NearestSquaredDistance = TNumericLimits<double>::Max();
	AMeshPartition* NearestMeshPartition = nullptr;
	AActor* NearestActor = nullptr;
	
	auto IsInsideBounds = [&CurrentLocation, &NearestSquaredDistance, &NearestMeshPartition, &NearestActor](AMeshPartition* MeshPartition, AActor* Actor, const FBox& InBounds)
	{
		if (InBounds.IsInside(CurrentLocation))
		{
			NearestSquaredDistance = 0.0;
			NearestMeshPartition = MeshPartition;
			NearestActor = Actor;
			return true;
		}

		double SquaredDistance = InBounds.ComputeSquaredDistanceToPoint(CurrentLocation);
		if (SquaredDistance < NearestSquaredDistance)
		{
			NearestSquaredDistance = SquaredDistance;
			NearestMeshPartition = MeshPartition;
			NearestActor = Actor;
		}
		return false;
	};
	
	for (TActorIterator<AActor> It(World, AMeshPartition::StaticClass()); It; ++It)
	{
		AMeshPartition* MegaMesh = Cast<AMeshPartition>(*It);
		if (!MegaMesh)
		{
			continue;
		}

		bool bIsInside = false;
		UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent());
		EditorComponent->ForAllPreviewSections([MegaMesh, IsInsideBounds, &bIsInside](APreviewSection* PreviewSection)
		{
			if (!PreviewSection)
			{
				return true;
			}

			FBox PreviewBounds = PreviewSection->GetComponentsBoundingBox();
			bIsInside = IsInsideBounds(MegaMesh, PreviewSection, PreviewBounds);
			return !bIsInside;
		});

		if (bIsInside)
		{
			SetAffectedMeshPartition(MegaMesh);
			RegisterWithMegaMeshIfNeeded();
			return NearestActor;
		}

		EditorComponent->ForAllCurrentModifiers([MegaMesh, IsInsideBounds, &bIsInside](UModifierComponent* ModifierComponent)
		{
			UMeshProviderModifier* ProviderModifier = Cast<UMeshProviderModifier>(ModifierComponent);
			if (!ProviderModifier)
			{
				return true;
			}

			FBox ProviderBounds = ProviderModifier->ComputeCombinedBounds();
			bIsInside = IsInsideBounds(MegaMesh, ProviderModifier->GetOwner(), ProviderBounds);
			return !bIsInside;
		});

		if (bIsInside)
		{
			SetAffectedMeshPartition(MegaMesh);
			RegisterWithMegaMeshIfNeeded();
			return NearestActor;
		}
	}

	if (NearestMeshPartition)
	{
		SetAffectedMeshPartition(NearestMeshPartition);
		RegisterWithMegaMeshIfNeeded();
	}
	return NearestActor;
}

AActor* UModifierComponent::BP_BindToNearestMeshPartition()
{
	return BindToNearestMeshPartition();
}

void UModifierComponent::SetPreviewSection(MeshPartition::APreviewSection* InPreviewSection)
{
	PreviewSection = InPreviewSection;
}

void UModifierComponent::SetBaseGrowth(const FBaseGrowth& InBaseGrowth)
{
	BaseGrowth = InBaseGrowth;
}

void UModifierComponent::UpdateLastAppliedBounds()
{
	LastAppliedBounds = ComputeBounds();
}

bool UModifierComponent::IntersectsAnyBounds(TConstArrayView<FBox> InBoundsToTest) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UModifierComponent::IntersectsAnyBounds);

	return Algo::AnyOf(InBoundsToTest, [ModifierBoundingBoxes = ComputeBounds()](const FBox& InBounds)
					{
						return Algo::AnyOf(ModifierBoundingBoxes, [InBounds](const FBox& ModifierBounds) { return ModifierBounds.Intersect(InBounds); });
					});
}

FName UModifierComponent::BuildPropertyKey(const FString& InModifierIndex, const FName& InKey)
{
	const FString PropertyKey = InModifierIndex + TEXT(".") + InKey.ToString();
	return *PropertyKey;
}

void UModifierComponent::OnChanged(TConstArrayView<FBox> InBoundingBoxes, EChangeType InChangeType)
{
	const AMeshPartition* MegaMesh = GetAffectedMeshPartition();

	// always update the cache key, no matter if there's an assigned megamesh or not.
	UpdateCacheKey(InChangeType == EChangeType::StateChange);
	
	if ((MegaMesh == nullptr) || bIgnoreChanged || !IsInRelevantWorld())
	{
		return;
	}

	UMeshPartitionEditorComponent* MegaMeshEditorComponent = GetMeshPartitionEditorComponent();

	if (MegaMeshEditorComponent == nullptr)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "MeshPartition::UModifierComponent::OnChanged : Cannot get MegaMeshEditorComponent.");
		return;
	}

	TArray<FBox> ChangedBounds;
	ChangedBounds.Reserve(LastSubmittedBounds.Num() + LastAppliedBounds.Num() + InBoundingBoxes.Num());

	ChangedBounds.Append(LastSubmittedBounds);
	ChangedBounds.Append(LastAppliedBounds);
	ChangedBounds.Append(InBoundingBoxes);

	MegaMeshEditorComponent->OnModifierChanged(this, ChangedBounds, InChangeType);
	LastSubmittedBounds = InBoundingBoxes;
}

TArray<FName> UModifierComponent::GetMegaMeshDefinitionChannels() const
{
	if (GetMegaMeshDefinition())
	{
		return GetMegaMeshDefinition()->GetChannelMap().GetChannels();
	}

	return TArray<FName>();
}

TArray<FName> UModifierComponent::GetDefinitionPriorityLayers() const
{
	if (GetMegaMeshDefinition())
	{
		return TArray<FName>(GetMegaMeshDefinition()->GetModifierTypePriorities());
	}

	return TArray<FName>();
}

void UModifierComponent::SetPriorityLayer(const FName PriorityLayer)
{
	Type = FPriorityLayerName(PriorityLayer);
}

FName UModifierComponent::GetPriorityLayer() const
{
	return Type.GetName();
}

FColor UModifierComponent::EditorUnselectedModifierColor() const
{
	//TODO: Pull this from the Mega Mesh Definition
	return FColor::Blue;
}


FBox UModifierComponent::GetOwnerBounds(const bool bShouldBeRegistered) const
{
	const AActor* Owner = GetOwner();
	FBox Box(ForceInit);

	if (Owner == nullptr)
	{
		return Box;
	}
	
	constexpr bool bIncludeFromChildActors = false;
	Owner->ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [&Box, bShouldBeRegistered](const UPrimitiveComponent* InPrimComp)
		{
			if (InPrimComp->IsRegistered() || !bShouldBeRegistered)
			{
				Box += InPrimComp->Bounds.GetBox();
			}
		});

	return Box;
}

int32 UModifierComponent::GetMegaMeshClassVersion() const
{
	return EditorUtils::GetMegaMeshClassVersionFromClass(GetClass());
}

void UModifierComponent::SetIsDisabledFlag(const bool bInIsDisabled)
{
	if (bIsDisabled != bInIsDisabled)
	{
		bIsDisabled = bInIsDisabled;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

FGuid UModifierComponent::UpdateCacheKey(bool bInChangeCacheKey)
{
	const int32 MegaMeshClassVersion = GetMegaMeshClassVersion();
	const bool bCacheKeyIsDeterministic = (MegaMeshClassVersion > 0);	// positive versions are deterministic, and anything else is non-deterministic on-change modified

	if (bCacheKeyIsDeterministic)
	{
		FDependencyHash DependencyHash;
		GatherDependencies(DependencyHash);

		// deterministic cache keys are always recomputed
		CacheKey = DependencyHash.GetDependentDataHash();
		// include the modifier cache key
		CacheKey.C = HashCombine(CacheKey.C, MegaMeshClassVersion);
	}
	else
	{
		// non-deterministic cache keys are only modified on change, and set to a new unique GUID
		if (bInChangeCacheKey)
		{
			CacheKey = FGuid::NewGuid();
		}
		// update the modifier cache key -- set the bottom 8 bits of C to the modifier version
		// this way we can update it in the future without changing the rest of the cache key
		CacheKey.C = (CacheKey.C & ~0xff) | ((uint32) MegaMeshClassVersion & 0xff);
	}
	return CacheKey;
}

void UModifierComponent::GatherDependencies(MeshPartition::IDependencyInterface& InDependencyContext) const
{
	// (you should not add yourself, your class, or your parent UObject as a dependency - that is done automatically)
	InDependencyContext += GetComponentTransform();
}

UMeshPartitionDefinition* UModifierComponent::GetMegaMeshDefinition() const
{
	const UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent();

	if (EditorComponent == nullptr)
	{
		return nullptr;
	}

	return EditorComponent->GetMegaMeshDefinition();;
}

void UModifierComponent::SetDrawBounds(bool bEnabled)
{
	bDrawBounds = bEnabled;
	MarkRenderStateDirty();
}

bool UModifierComponent::ShouldDrawBoundingBox() const
{
	return bDrawBounds;
}

void UModifierComponent::RegisterWithMegaMeshIfNeeded()
{
	if (!bNeedToRegisterWithMeshPartition || !IsInRelevantWorld())
	{
		return;
	}
	if (UMeshPartitionEditorComponent* MegaMeshComponent = GetMeshPartitionEditorComponent())
	{
		MegaMeshComponent->OnModifierAssigned();
		bNeedToRegisterWithMeshPartition = false;
	}
}

bool UModifierComponent::IsInRelevantWorld() const
{
	UWorld* World = GetWorld();
	return !IsTemplate() && IsValid(this)
		// This filters out things like the BP preview world. Seems to still be ok for building the megamesh in commandlets
		&& IsValid(World) && World->WorldType == EWorldType::Editor;
}

#if WITH_EDITOR
void UModifierComponent::CreateModifierSpriteComponent()
{
	if (SpriteComponent)
	{
		// Theoretically we shouldn't need to do anything if the sprite has already been
		//  set up.
		return;
	}

	UTexture2D* EditorSpriteTexture = nullptr;
	{
		FCookLoadScope EditorOnlyScope(ECookLoadType::EditorOnly);
		EditorSpriteTexture = LoadObject<UTexture2D>(
			nullptr,
			MegaMeshModifierComponentLocals::SpritePath);
	}

	bVisualizeComponent = true;
	// This accepts null texture if we failed to load (uses a default marker)
	CreateSpriteComponent(EditorSpriteTexture, /*bRegister*/ false);

	if (SpriteComponent)
	{
		SpriteComponent->SetWorldScale3D(FVector(MegaMeshModifierComponentLocals::SpriteSize));
		SpriteComponent->SetVisibility(ShouldShowSpriteComponent() && MegaMeshModifierComponentLocals::SpriteSize > 0);
		SpriteComponent->RegisterComponent();
	}
}
#endif // WITH_EDITOR
} // namespace UE::MeshPartition
