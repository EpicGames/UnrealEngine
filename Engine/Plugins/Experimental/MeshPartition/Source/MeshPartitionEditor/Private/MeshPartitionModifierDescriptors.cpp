// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModifierDescriptors.h"

#include "GameFramework/Actor.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierUtils.h"
#include "MeshPartitionDefinition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "MeshPartitionModifierComponentDesc.h"
#include "Engine/LevelStreaming.h"
#include "MeshPartition.h"
#include "Algo/ForEach.h"
#include "Algo/AnyOf.h"
#include "Algo/IsSorted.h"

namespace UE::MeshPartition
{
const bool& FBaseGrowth::operator[](const uint32 InIndex) const
{
	switch (InIndex)
	{
		case 0:
			return X;
		case 1:
			return Y;
		case 2:
			return Z;
	}

	checkf(false, TEXT("Index should not exceed 2."));
	return X;
}

FString FBaseGrowth::ToString() const
{
	return FString::Printf(TEXT("(X=%c,Y=%c,Z=%c)"), X, Y, Z);
}

bool FBaseGrowth::InitFromString(const FString& InSourceString)
{
	uint8 BufferX = 0;
	uint8 BufferY = 0;
	uint8 BufferZ = 0;
	// The initialization is only successful if the X, Y, and Z values can all be parsed from the string
	const bool bSuccess = FParse::Value(*InSourceString, TEXT("X="), BufferX) && FParse::Value(*InSourceString, TEXT("Y="), BufferY) && FParse::Value(*InSourceString, TEXT("Z="), BufferZ);

	X = BufferX != 0;
	Y = BufferY != 0;
	Z = BufferZ != 0;

	return bSuccess;
}

FModifierDesc::FModifierDesc()
	: ModifierPath()
	, ClassPath()
	, Type()
	, Priority(0.)
	, OwnerGuid(FGuid())
	, MegaMeshGuid(FGuid())
	, Bounds(EForceInit::ForceInit)
	, BaseGrowth()
	, Complexity(0.f)
	, ComplexityMultiplier(1.f)
	, bIsContiguous(false)
	, bIsDisabled(false)
	, bIsBase(false)
{
}

// build a modifier desc from a loaded modifier
FModifierDesc::FModifierDesc(const MeshPartition::UModifierComponent& InModifier)
	: ModifierPath(FSoftObjectPath(&InModifier))
	, ClassPath(InModifier.GetClass())
	, Type(InModifier.GetType())
	, Priority(InModifier.GetPriority())
	, OwnerGuid(FGuid())							// filled out below
	, MegaMeshGuid(FGuid())							// filled out below
	, Bounds(InModifier.ComputeCombinedBounds())
	, BaseGrowth(InModifier.GetBaseGrowth())
	, Complexity(InModifier.GetComplexity())
	, ComplexityMultiplier(InModifier.GetComplexityMultiplier())
	, bIsContiguous(InModifier.IsContiguous())
	, bIsDisabled(InModifier.IsDisabled())
	, bIsBase(InModifier.IsBase())
{
	if (const AActor* Owner = InModifier.GetOwner())
	{
		OwnerGuid = Owner->GetActorGuid();
	}
	if (const AMeshPartition* MeshPartition = InModifier.GetAffectedMeshPartition())
	{
		MegaMeshGuid = MeshPartition->GetActorGuid();
	}
}

bool FModifierDesc::IsValid() const
{
	return ModifierPath.IsValid();
}

FModifierDesc::FModifierDesc(const FWorldPartitionActorDescInstance& InActorDescInstance, int32 InModifierIndex)
	: MeshPartition::FModifierDesc()
{
	const FString ModifierIndexString = FString::Printf(TEXT("%d"), InModifierIndex);

	auto GetModifierProperty = [&InActorDescInstance, &ModifierIndexString](FName PropertyKey, FName* OutPropertyValue, bool bLogError = true)
		{
			FName ModifierPropertyKey = MeshPartition::UModifierComponent::BuildPropertyKey(ModifierIndexString, PropertyKey);
			if (!InActorDescInstance.GetProperty(ModifierPropertyKey, OutPropertyValue))
			{
				if (bLogError)
				{
					UE_LOGF(LogMegaMeshEditor, Error, "Failed to build modifier descriptor for actor '%ls' modifier '%ls' : could not find property `%ls`.  You may need to resave the modifier to update the properties.",
						*InActorDescInstance.GetActorName().ToString(),
						*ModifierIndexString,
						*ModifierPropertyKey.ToString());
				}
				return false;
			}
			return true;
		};

	int32 ModifierVersion = 0;
	FName ModifierVersionString;
	if (InActorDescInstance.GetProperty(MegaMeshModifierProperties::PropertiesVersionNumberName, &ModifierVersionString))
	{
		LexFromString(ModifierVersion, *ModifierVersionString.ToString());
	}

	FName ModifierPathName;
	if (!GetModifierProperty(MegaMeshModifierProperties::MegaMeshModifierPath, &ModifierPathName))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Failed to build modifier descriptor, Modifier %ls has no ModifierPath in the actor descriptor.", *InActorDescInstance.GetActorName().ToString());
		*this = MeshPartition::FModifierDesc();
		return;
	}
	this->ModifierPath = FSoftObjectPath(ModifierPathName.ToString());

	FName MegaMeshGUIDName;
	if (GetModifierProperty(MegaMeshModifierProperties::MegaMeshGUID, &MegaMeshGUIDName, false))
	{
		this->MegaMeshGuid = FGuid(MegaMeshGUIDName.ToString());
	}
	else
	{
		// an invalid MegaMeshGuid is fine, and this modifier may still be used, as long as the MegaMeshGuid is overridden by a LevelInstanceAdapter
		this->MegaMeshGuid = FGuid();
	}

	if (!GetModifierProperty(MegaMeshModifierProperties::Type, &this->Type))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Failed to build modifier descriptor (no Type property).");
		*this = MeshPartition::FModifierDesc();
		return;
	}

	FName PriorityName;
	if (!GetModifierProperty(MegaMeshModifierProperties::Priority, &PriorityName, false))
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Modifier %ls has no Priority property, and needs to be resaved.  Until it is resaved, compiled sections will assume it has a default priority of 0.0, which may not render properly.", *InActorDescInstance.GetActorName().ToString());
		this->Priority = 0.0;
	}
	else
	{
		LexFromString(this->Priority, *PriorityName.ToString());
	}

	FName ModifierClassName;
	if (!GetModifierProperty(MegaMeshModifierProperties::Class, &ModifierClassName, false))
	{
		// we assume modifier class version is zero if it's old enough that it hasn't recorded its class name.
		UE_LOGF(LogMegaMeshEditor, Warning, "Modifier %ls has no Class property, and needs to be resaved.  Until it is resaved we won't be able to detect class implementation changes (version bumps).", *InActorDescInstance.GetActorName().ToString());
	}
	else
	{
		this->ClassPath = FSoftClassPath(ModifierClassName.ToString());
	}

	this->OwnerGuid = InActorDescInstance.GetGuid();
	if (!this->OwnerGuid.IsValid())
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Failed to build modifier descriptor (invalid OwnerGuid).");
		*this = MeshPartition::FModifierDesc();
		return;
	}

	this->Bounds = InActorDescInstance.GetEditorBounds();
	if (!this->Bounds.IsValid)
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Failed to build modifier descriptor (invalid Bounds).");
		*this = MeshPartition::FModifierDesc();
		return;
	}

	FName ComplexityName;
	if (!GetModifierProperty(MegaMeshModifierProperties::Complexity, &ComplexityName))
	{
		*this = MeshPartition::FModifierDesc();
		return;
	}
	LexFromString(this->Complexity, *ComplexityName.ToString());

	FName ComplexityMultiplierName;
	if (!GetModifierProperty(MegaMeshModifierProperties::ComplexityMultiplier, &ComplexityMultiplierName))
	{
		*this = MeshPartition::FModifierDesc();
		return;
	}
	LexFromString(this->ComplexityMultiplier, *ComplexityMultiplierName.ToString());

	this->bIsContiguous = false;
	if (ModifierVersion >= 2)
	{
		FName IsContiguousName;
		if (!GetModifierProperty(MegaMeshModifierProperties::IsContiguous, &IsContiguousName))
		{
			*this = MeshPartition::FModifierDesc();
			return;
		}
		else
		{
			LexFromString(this->bIsContiguous, *IsContiguousName.ToString());
		}
	}

	this->bIsDisabled = false;
	if (ModifierVersion >= 3)
	{
		FName IsDisabledName;
		if (!GetModifierProperty(MegaMeshModifierProperties::IsDisabled, &IsDisabledName))
		{
			*this = MeshPartition::FModifierDesc();
			return;
		}
		else
		{
			LexFromString(this->bIsDisabled, *IsDisabledName.ToString());
		}
	}

	if (ModifierVersion >= 4)
	{
		FName BaseGrowthName;
		if (!GetModifierProperty(MegaMeshModifierProperties::BaseGrowth, &BaseGrowthName))
		{
			*this = MeshPartition::FModifierDesc();
			return;
		}
		else
		{
			this->BaseGrowth.InitFromString(BaseGrowthName.ToString());
		}
	}

	check(this->ModifierPath.IsValid());
}

FModifierDesc::FModifierDesc(const FWorldPartitionActorDescInstance& InActorDescInstance, const FWorldPartitionModifierComponentDesc& InComponentDesc)
	: FModifierDesc()
{
	ModifierPath = InComponentDesc.ModifierPath;
	ClassPath = InComponentDesc.GetComponentNativeClass();
	Type = InComponentDesc.Type;
	Priority = InComponentDesc.Priority;
	OwnerGuid = InActorDescInstance.GetGuid();
	MegaMeshGuid = InComponentDesc.MegaMeshGuid;

	// Compose the actor's world transform from the descriptor's container-local actor
	// transform and the container instance's world transform. For actors directly in the
	// main world the container is identity (or null for detached/unregistered descriptors);
	// for actors inside a level instance it is the LI's world placement. Without applying
	// it, modifier bounds inside an LI end up in LI-local space and fail to intersect base
	// bounds in world space, dropping them from FModifierGroup::ComputeBaseBounds-driven
	// grouping. Null and identity both degrade to no extra transform, which is the correct
	// behavior for non-LI cases.
	const UActorDescContainerInstance* ContainerInstance = InActorDescInstance.GetContainerInstance();
	const FTransform ContainerTransform = ContainerInstance ? ContainerInstance->GetTransform() : FTransform::Identity;
	const FTransform ActorWorldTransform = InActorDescInstance.GetActorTransform() * ContainerTransform;

	// Legacy path, for backwards compatibility.
	// The deprecated ComponentTransform was Modifier->GetComponentTransform() at desc-build time --
	// a snapshot that goes stale when the actor moves and the component desc is not regenerated.
	// We can't recover the per-component offset reliably, so reconstruct from the live actor
	// transform and assume the component sits at the actor origin. This is correct for bases
	// (the common case) but produces wrong bounds for sub-component modifiers in v7 data; resave
	// to upgrade to v8 (which stores ComponentToActorTransform explicitly).
	if (!InComponentDesc.bHasComponentToActorTransform)
	{
		// Warn per modifier so the user knows exactly which asset to resave to upgrade to v8.
		UE_LOGF(LogMegaMeshEditor, Warning,
				"Pre-v8 modifier desc data on actor '%ls' modifier '%ls'; sub-component modifier bounds may be incorrect. Resave the asset to upgrade.",
				*InActorDescInstance.GetActorName().ToString(),
				*InComponentDesc.ModifierPath.ToString());

		Bounds = InComponentDesc.LocalBounds.TransformBy(ActorWorldTransform);
	}
	else
	{
		Bounds = InComponentDesc.LocalBounds.TransformBy(InComponentDesc.ComponentToActorTransform * ActorWorldTransform);
	}

	BaseGrowth = InComponentDesc.BaseGrowth;
	Complexity = InComponentDesc.Complexity;
	ComplexityMultiplier = InComponentDesc.ComplexityMultiplier;
	bIsContiguous = InComponentDesc.bIsContiguous;
	bIsDisabled = InComponentDesc.bIsDisabled;
	bIsBase = InComponentDesc.bIsBase;
}

MeshPartition::FModifierGroup BuildModifierGroupForBase(TConstArrayView<MeshPartition::FModifierDesc> InBase, TConstArrayView<MeshPartition::FModifierDesc> InModifiers)
{
	MeshPartition::FModifierGroup ModifierGroup;

	// Add all base modifiers and compute bounds
	for (const MeshPartition::FModifierDesc& Base : InBase)
	{
		ModifierGroup.AddBase(Base);
	}

	// Get all intersecting modifiers.
	for (const MeshPartition::FModifierDesc& Modifier : InModifiers)
	{
		if (Modifier.Bounds.Intersect(ModifierGroup.ComputeBaseBounds()))
		{
			ModifierGroup.AddModifier(Modifier);
		}
	}

	return ModifierGroup;
}

namespace Private
{
	TArray<const MeshPartition::FModifierDesc> GetBaseNeighbors(const TArray<MeshPartition::FModifierDesc>& InBaseReferences, const TArray<MeshPartition::FModifierDesc>& InBases)
	{
		// Expand the reference bounds by an error margin to account for floating point error
		constexpr double ErrorMargin = 1e-4;

		TArray<const MeshPartition::FModifierDesc> Neighbors;

		// Copy the base if it intersects at least one base reference.
		Algo::CopyIf(InBases, Neighbors, [&InBaseReferences](const MeshPartition::FModifierDesc& Base)
			{
				FBox BaseBounds = Base.Bounds.ExpandBy(ErrorMargin);
	
				return Algo::AnyOf(InBaseReferences, [BaseBounds](const MeshPartition::FModifierDesc& BaseReference)
					{
						return BaseBounds.Intersect(BaseReference.Bounds);
					});
			});

		// Sort the resulting neighbors by overlapping volume
		Algo::SortBy(Neighbors, [&InBaseReferences](const MeshPartition::FModifierDesc& Base)
			{
				float OverlappingVolume = 0.f;
				for (const MeshPartition::FModifierDesc& RefBase : InBaseReferences)
				{
					// The expansion of the RefBase bounds is important here as this is what gives the mathematical properties
					// which prefers bases sharing edges vs. sharing corners.
					OverlappingVolume += RefBase.Bounds.ExpandBy(ErrorMargin).Overlap(Base.Bounds).GetVolume();
				}
				// comparison is done with < operator but we want larger overlapping volumes to appear first.
				return -1 * OverlappingVolume;
			});

		return Neighbors;
	}

	void MergeContiguousModifierGroups(const TArray<MeshPartition::FModifierDesc>&  InContiguousModifierDescriptors,
										const TArray<MeshPartition::FModifierDesc>&  InModifierDescriptors,
										TArray<UE::MeshPartition::FModifierGroup>& OutResults)
	{
		// For each existing contiguous modifier, we iterate through all already formed groups.
		for (const MeshPartition::FModifierDesc& ContiguousModifierDescriptor : InContiguousModifierDescriptors)
		{
			TArray<MeshPartition::FModifierDesc> CurrentBase;
			for (auto Iterator = OutResults.CreateIterator(); Iterator; ++Iterator)
			{
				const UE::MeshPartition::FModifierGroup& ModifierGroup = *Iterator;
				// If the current group contains this contiguous modifier, we aggregate bases.
				if (ModifierGroup.ModifierDescs().Contains(ContiguousModifierDescriptor))
				{
					for (const MeshPartition::FModifierDesc& Base : ModifierGroup.BaseDescs())
					{
						CurrentBase.AddUnique(Base);
					}
				
					Iterator.RemoveCurrentSwap();
				}
			}
			if (CurrentBase.IsEmpty())
			{
				continue;
			}
			// Creating a new group containing all aggregated bases.
			OutResults.Add(BuildModifierGroupForBase(CurrentBase, InModifierDescriptors));
		}
	}
} // namespace Private

void FModifierGroup::ForAllModifiers(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc, bool bSkipInvalidModifiers) const
{
	ForEachModifier_Internal(AllModifierPtrs(), InFunc, bSkipInvalidModifiers);
}

void FModifierGroup::ForEachBase(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc) const
{
	ForEachModifier_Internal(BasePtrs(), InFunc);
}

void FModifierGroup::ForEachModifier(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc) const
{
	ForEachModifier_Internal(ModifierPtrs(), InFunc);
}

TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> FModifierGroup::AllResolvedModifierPtrs() const
{
	TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> ModifierPtrs;

	auto CopyModifier = [&ModifierPtrs](MeshPartition::UModifierComponent* Modifier)
	{
		ModifierPtrs.Emplace(Modifier);
		return true;
	};

	ForAllModifiers(CopyModifier);

	return ModifierPtrs;
}

void FModifierGroup::ForEachModifier_Internal(TConstArrayView<TSoftObjectPtr<MeshPartition::UModifierComponent>> InModifierPtrs, TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc, bool bSkipInvalidModifiers) const
{
	check(IsInGameThread());
	ensure(CurrentState >= EState::ModifiersResolved);

	for (const TSoftObjectPtr<MeshPartition::UModifierComponent>& ModifierPtr : InModifierPtrs)
	{
		UModifierComponent* Modifier = ModifierPtr.Get();

		if (Modifier == nullptr)
		{
			ensure(bSkipInvalidModifiers);
			continue;
		}

		if (!InFunc(Modifier))
		{
			break;
		}
	}
}

int8 FModifierGroup::CompareModifierOrder(TConstArrayView<FName> InLayerPriorities, const MeshPartition::FModifierDesc& InModifier, const MeshPartition::FModifierDesc& InOtherModifier)
{
	constexpr int8 ApplyBefore = -1;
	constexpr int8 Unresolved  = 0;
	constexpr int8 ApplyAfter  = 1;

	if (InModifier.IsBase() && !InOtherModifier.IsBase())
	{
		return ApplyBefore;
	}
	else if (!InModifier.IsBase() && InOtherModifier.IsBase())
	{
		return ApplyAfter;
	}
	if (InLayerPriorities.Num() > 0)
	{
		// When IndexOfByKey returns -1, we want this to correspond to the weakest priority value, hence the cast to u32
		const uint32 TypePriority = static_cast<uint32>(InLayerPriorities.IndexOfByKey(InModifier.Type));
		const uint32 OtherTypePriority = static_cast<uint32>(InLayerPriorities.IndexOfByKey(InOtherModifier.Type));

		// Lower priority implies it should go before
		if (TypePriority < OtherTypePriority)
		{
			return ApplyBefore;
		}
		else if (TypePriority > OtherTypePriority)
		{
			return ApplyAfter;
		}
	}
	if (InModifier.Priority < InOtherModifier.Priority)
	{
		return ApplyBefore;
	}
	// if priorities are equal, sort by guid to ensure determinism and consistency across modifier boundaries
	else if (InModifier.Priority == InOtherModifier.Priority)
	{
		if (InModifier.ModifierPath == InOtherModifier.ModifierPath)
		{
			return Unresolved;
		}
		else
		{
			// use LexicalLess to ensure stable ordering across different processes
			return InModifier.ModifierPath.LexicalLess(InOtherModifier.ModifierPath) ? ApplyBefore : ApplyAfter;
		}
	}
	return ApplyAfter;
}

bool FModifierGroup::ShouldApplyInstanceBefore(TConstArrayView<FName> InLayerPriorities, const FInstanceInfo& InInstance, const FInstanceInfo& InOtherInstance) const
{
	const int8 Result = CompareModifierOrder(InLayerPriorities, GetModifierDesc(InInstance), GetModifierDesc(InOtherInstance));

	// If the sorting order couldn't be resolved by just the modifier data (usually because they are instances of the same modifier), we sort by instance ID.
	if (Result == 0)
	{
		return InInstance.InstanceID < InOtherInstance.InstanceID;
	}

	return Result == -1;
}
	
bool FModifierGroup::ShouldApplyModifierBefore(TConstArrayView<FName> InLayerPriorities, const MeshPartition::FModifierDesc& InModifier, const MeshPartition::FModifierDesc& InOtherModifier)
{
	const int8 Result = CompareModifierOrder(InLayerPriorities, InModifier, InOtherModifier);
	// There should not be any ambiguity in the sorting order between two different modifiers.
	check(Result != 0);
	return Result == -1;
}
	
bool FModifierGroup::HasDependency(const FInstanceInfo& InInstance, const FInstanceInfo& InOtherInstance) const
{
	// Modifiers which write to the same spatial region of the source mesh must be mutually exclusive.
	const bool bWriteToSameBounds = InInstance.Bounds.Intersect(InOtherInstance.Bounds);

	return bWriteToSameBounds;
}

bool FModifierGroup::HasDependency(const MeshPartition::FModifierDesc& InModifier, const MeshPartition::FModifierDesc& InOtherModifier)
{
	ensure(InModifier != InOtherModifier);
	// Two bases never have dependencies on each other even if their bounds overlap.
	if (InModifier.IsBase() && InOtherModifier.IsBase())
	{
		return false;
	}

	// Modifiers which write to the same spatial region of the source mesh must be mutually exclusive.
	const bool bWriteToSameBounds = InModifier.Bounds.Intersect(InOtherModifier.Bounds);

	return bWriteToSameBounds;
}

bool FModifierGroup::ValidateIsSorted(TConstArrayView<FName> InLayerPriorities) const
{
	const bool bIsSorted = Algo::IsSorted(ModifierDescs(), [InLayerPriorities](const MeshPartition::FModifierDesc& A, const MeshPartition::FModifierDesc& B)
		{
			return ShouldApplyModifierBefore(InLayerPriorities, A, B);
		});
	return bIsSorted;
}

bool FModifierGroup::IsEmpty() const
{
	return ModifierDescriptors.Num() == 0;
}

void FModifierGroup::Add(const MeshPartition::FModifierDesc& InDesc)
{
	if (InDesc.IsBase())
	{
		AddBase(InDesc);
	}
	else
	{
		AddModifier(InDesc);
	}
}

void FModifierGroup::AddBase(const MeshPartition::FModifierDesc& InBaseDesc)
{
	if (ensure(CurrentState < EState::DescriptorsFinalized))
	{
		ensure(InBaseDesc.IsBase());

		ModifierDescriptors.EmplaceAt(NumBases++, InBaseDesc);
	}
}

void FModifierGroup::AddModifier(const MeshPartition::FModifierDesc& InModifierDesc)
{
	if (ensure(CurrentState < EState::DescriptorsFinalized))
	{
		ensure(!InModifierDesc.IsBase());

		ModifierDescriptors.Add(InModifierDesc);
	}
}

void FModifierGroup::AddModifierSorted(TConstArrayView<FName> InModifierTypePriorities, const MeshPartition::FModifierDesc& InModifierDesc)
{
	if (ensure(CurrentState < EState::DescriptorsFinalized))
	{
		ensure(!InModifierDesc.IsBase());

		for (FModifierIndex ModifierIndex : ModifierIndices())
		{
			if (ShouldApplyModifierBefore(InModifierTypePriorities, InModifierDesc, GetModifierDesc(ModifierIndex)))
			{
				ModifierDescriptors.EmplaceAt(ModifierIndex, InModifierDesc);
				return;
			}
		}

		ModifierDescriptors.Add(InModifierDesc);
	}
}

void FModifierGroup::Sort(TConstArrayView<FName> InModifierTypePriorities)
{
	if (ensure(CurrentState < EState::DescriptorsFinalized) &&
		ensure(ModifierPointers.IsEmpty()) &&
		ensure(ModifierCacheKeys.IsEmpty()) &&
		ensure(ModifierOps.IsEmpty()) &&
		ensure(InstanceInfos.IsEmpty()) &&
		ensure(!bAsyncGroup))
	{
		SortModifierDescriptors(InModifierTypePriorities, ModifierDescriptors);
	}
}

FBox FModifierGroup::ComputeBaseBounds() const
{
	FBox BaseBounds(ForceInit);
	for (const MeshPartition::FModifierDesc& ModifierDesc : BaseDescs())
	{
		BaseBounds += ModifierDesc.Bounds;
	}

	for (const MeshPartition::FModifierDesc& ModifierDesc : ModifierDescs())
	{
		for (uint32 Index = 0; Index < 3; ++Index)
		{
			if (ModifierDesc.BaseGrowth[Index])
			{
				BaseBounds.Max[Index] = FMath::Max(BaseBounds.Max[Index], ModifierDesc.Bounds.Max[Index]);
				BaseBounds.Min[Index] = FMath::Min(BaseBounds.Min[Index], ModifierDesc.Bounds.Min[Index]);
			}
		}
	}
	return BaseBounds;
}

double FModifierGroup::ComputeBaseComplexity() const
{
	return Algo::Accumulate(BaseDescs(), 0.0, [](const double Complexity, const MeshPartition::FModifierDesc& Desc)
	{
		return Complexity * Desc.ComplexityMultiplier + Desc.Complexity;
	});
}

double FModifierGroup::ComputeTotalComplexity() const
{
	return Algo::Accumulate(ModifierDescriptors, 0.0, [](const double Complexity, const MeshPartition::FModifierDesc& Desc)
	{
		return Complexity * Desc.ComplexityMultiplier + Desc.Complexity;
	});
}

void FModifierGroup::RemoveDisabledModifiers()
{
	// #todo: It's a bit unfortunate we can't remove disabled modifiers before the group becomes "finalized".
	// With the current logic in the megamesh builder, the disabled modifiers are removed quite a bit later than the group finalizing and being set up.
	// This is due to place in which we check if the group can be stored in DDC or not.
	// At the absolute latest, we should not be calling this function once background ops are created as it would indicate
	// a larger perf cost being paid unnecessarily.
	if (!ensure(CurrentState < EState::BackgroundOpsCreated))
	{
		return;
	}

	TArray<FModifierIndex> ModifiersToRemove;
	for (FModifierIndex ModifierIndex : AllModifierIndices())
	{
		if (GetModifierDesc(ModifierIndex).bIsDisabled)
		{
			ModifiersToRemove.Add(ModifierIndex);
		}
	}

	for (int32 Index = ModifiersToRemove.Num() - 1; Index >= 0; --Index)
	{
		FModifierIndex ModifierIndex = ModifiersToRemove[Index];
		if (ModifierDescriptors[ModifierIndex].IsBase())
		{
			NumBases -= 1;
		}
		ModifierDescriptors.RemoveAt(ModifierIndex, EAllowShrinking::No);

		if (ModifierPointers.IsValidIndex(ModifierIndex))
		{
			ModifierPointers.RemoveAt(ModifierIndex, EAllowShrinking::No);
		}
		if (ModifierCacheKeys.IsValidIndex(ModifierIndex))
		{
			ModifierCacheKeys.RemoveAt(ModifierIndex, EAllowShrinking::No);
		}
		if (ModifierOps.IsValidIndex(ModifierIndex))
		{
			ModifierOps.RemoveAt(ModifierIndex, EAllowShrinking::No);
		}
	}
	ModifierDescriptors.Shrink();
	ModifierPointers.Shrink();
	ModifierCacheKeys.Shrink();
	ModifierOps.Shrink();
}

void FModifierGroup::ProgressToState(EState InTarget)
{
	if (InTarget >= EState::DescriptorsFinalized && EState::DescriptorsFinalized > CurrentState)
	{
		CurrentState = EState::DescriptorsFinalized;
	}
	if (InTarget >= EState::ModifiersResolved && EState::ModifiersResolved > CurrentState)
	{
		ResolveModifierPtrs();
		CurrentState = EState::ModifiersResolved;
	}
	if (InTarget >= EState::BackgroundOpsCreated && EState::BackgroundOpsCreated > CurrentState)
	{
		checkf(BuildType.IsSet(), TEXT("Progressing through creating background ops requires knowledge of the build type"));
		CreateBackgroundOps(BuildType.GetValue());
		CurrentState = EState::BackgroundOpsCreated;
	}
	if (InTarget >= EState::InstancesReady && EState::InstancesReady > CurrentState)
	{
		InitInstances();
		CurrentState = EState::InstancesReady;
	}
}

void FModifierGroup::ResolveModifierPtrs()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::MeshPartition::FModifierGroup::ResolveModifierPtrs);

	check(IsInGameThread());
	ensure(CurrentState >= EState::DescriptorsFinalized);

	ModifierPointers.SetNum(ModifierDescriptors.Num());
	ModifierCacheKeys.SetNum(ModifierDescriptors.Num());
	for (int32 ModifierIndex = 0; ModifierIndex < ModifierDescriptors.Num(); ModifierIndex++)
	{
		const MeshPartition::FModifierDesc& ModifierDesc = GetModifierDesc(FModifierIndex(ModifierIndex));

		MeshPartition::UModifierComponent* ModifierComponent = nullptr;
		if (ModifierResolver)
		{
			ModifierComponent = ModifierResolver->ResolveModifier(ModifierDesc);
		}
		else
		{
			// default behavior is to resolve by path
			UObject* ModifierObject = ModifierDesc.ModifierPath.ResolveObject();
			ModifierComponent = Cast<MeshPartition::UModifierComponent>(ModifierObject);
		}

		if (ModifierComponent != nullptr)
		{
			ModifierPointers[ModifierIndex] = ModifierComponent;
			ModifierCacheKeys[ModifierIndex] = ModifierComponent->GetCacheKey();
		}
		else
		{
			// This can be caused by the WP ActorDesc being out of sync with the files on disk, or if some change has caused actor references to fail (i.e. redirects)
			UE_LOGF(LogMegaMeshEditor, Warning, "Could not resolve Mesh Partition modifier '%ls', results will not be correct.", *ModifierDesc.ModifierPath.ToString()); 

			// strip the failed modifier from the group
			ModifierDescriptors.RemoveAt(ModifierIndex);	// keep order on this list (we're currently iterating in order)
			ModifierPointers.RemoveAtSwap(ModifierIndex);	// ok to swap, remainder of the list is uninitialized
			ModifierCacheKeys.RemoveAtSwap(ModifierIndex);
			// ModifierOps are not yet initialized

			if (ModifierIndex < NumBases)
			{
				NumBases--;
			}

			ModifierIndex--;
		}
	}
}

void FModifierGroup::PrepareResources()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::MeshPartition::FModifierGroup::PrepareResources);

	check(IsInGameThread());
	ensure(CurrentState >= EState::ModifiersResolved);

	for (FModifierIndex ModifierIndex : AllModifierIndices())
	{
		ensure(!GetModifierDesc(ModifierIndex).bIsDisabled);
		GetModifierPtr(ModifierIndex)->PrepareResources();
	}
}

void FModifierGroup::CreateBackgroundOps(MeshPartition::EBuildType InBuildType)
{
	check(IsInGameThread());

	ensure(CurrentState >= EState::ModifiersResolved);

	ModifierOps.SetNum(ModifierDescriptors.Num());
	for (FModifierIndex ModifierIndex : AllModifierIndices())
	{
		ensure(!GetModifierDesc(ModifierIndex).bIsDisabled);
		ModifierOps[ModifierIndex] = GetModifierPtr(ModifierIndex)->CreateBackgroundOp(InBuildType);
	}
}

void FModifierGroup::InitInstances()
{
	check(bAsyncGroup);

	ensure(CurrentState >= EState::BackgroundOpsCreated);

	TArray<FInstanceInfo> Instances;

	for (FModifierIndex Modifier : ModifierIndices())
	{
		Instances.Empty();

		const TSharedPtr<const MeshPartition::IModifierBackgroundOp> Op = GetModifierOp(Modifier);

		if (!Op)
		{
			continue;
		}

		Op->GetInstancesInBounds(ComputeBaseBounds(), Instances);
		for (FInstanceInfo& Instance : Instances)
		{
			Instance.ModifierIndex = Modifier;
		}

		InstanceInfos.Append(MoveTemp(Instances));
	}
}

FModifierGroup FModifierGroup::CreateAsyncBuildGroup()
{
	ensure(CurrentState >= EState::BackgroundOpsCreated);

	FModifierGroup Result = *this;
	Result.bAsyncGroup = true;
	return Result;
}

TWeakObjectPtr<MeshPartition::UModifierComponent> FModifierGroup::GetModifierPtr(FModifierIndex InIndex) const
{
	check(IsInGameThread());
	check(!bAsyncGroup);
	check((InIndex >= 0) && (InIndex < ModifierPointers.Num()));

	ensure(CurrentState >= EState::ModifiersResolved);

	TWeakObjectPtr<MeshPartition::UModifierComponent> ModifierPtr = ModifierPointers[InIndex].Get();

	ensure(ModifierPtr.IsValid());

	return ModifierPtr;
}

FGuid FModifierGroup::ComputeBaseModifierSetHash() const
{
	FGuid BaseSetHash;
	
	for (const MeshPartition::FModifierDesc& BaseDesc : BaseDescs())
	{
		BaseSetHash = Utils::BetterHashCombine(BaseSetHash, FGuid::NewDeterministicGuid(BaseDesc.ModifierPath.ToString()));
	}
	
	return BaseSetHash;
}

FGuid FModifierGroup::ComputeBaseCacheKey() const
{
	ensure(CurrentState >= EState::ModifiersResolved);

	FGuid Result;
	for (FModifierIndex BaseIndex : BaseIndices())
	{
		Result = FGuid::Combine(Result, GetModifierCacheKey(BaseIndex));
	}
	return Result;
}

FGuid FModifierGroup::ComputeModifierSetHash() const
{
	// #todo: this ensure should be enabled. However, because of the current implementation of the WorldPartitionBuilder we actually
	// get the modifier set hash _before_ level instances are loaded and therefore can't lock the groupings until after they've been
	// loaded and all modifiers they contain are added to this group.
	//ensure(bGroupFinalized);

	FGuid ModifierSetHash = FGuid(0x546382EB, 0x09AE4DBD, 0xA7128E1B, 0xB9F60B66);
	
	for (const MeshPartition::FModifierDesc& ModifierDesc : ModifierDescriptors)
	{
		ModifierSetHash = Utils::BetterHashCombine(ModifierSetHash, FGuid::NewDeterministicGuid(ModifierDesc.ModifierPath.ToString()));
	}

	return ModifierSetHash;
}

FGuid FModifierGroup::UpdateAndComputeModifierGroupHash()
{
	ensure(CurrentState >= EState::ModifiersResolved);

	FGuid ModifiersHash = FGuid(0x546382EB, 0x09AE4DBD, 0xA7128E1B, 0xB9F60B66);

	ModifierCacheKeys.SetNum(ModifierDescriptors.Num());
	for (FModifierIndex ModifierIndex : AllModifierIndices())
	{
		// hash in the modifier cache key (updated if necessary)
		const FGuid ModifierCacheKey = GetModifierPtr(ModifierIndex)->UpdateCacheKey();
		ModifierCacheKeys[ModifierIndex] = ModifierCacheKey;

		UE_LOGF(LogMegaMeshEditor, Verbose, "------ Modifier %ls -- CacheKey %ls", *GetModifierPtr(ModifierIndex)->GetName(), *ModifierCacheKey.ToString());
		ModifiersHash = FGuid::Combine(ModifiersHash, ModifierCacheKey);
	}

	UE_LOGF(LogMegaMeshEditor, Verbose, "---- ModifiersHash %ls", *ModifiersHash.ToString());

	return ModifiersHash;
}

FBlake3Hash FModifierGroup::ComputeGroupBuildHash() const
{
	ensure(CurrentState >= EState::ModifiersResolved);

	FBlake3 Hasher;

	auto UpdateHashPODType = []<typename Type UE_REQUIRES(TIsPODType<Type>::Value)> (FBlake3& Hasher, const Type& InData)
	{
		Hasher.Update(reinterpret_cast<const uint8*>(&InData), sizeof(Type));
	};

	auto UpdateHashFString = [](FBlake3& Hasher, const FString& InStr)
	{
		FStringView Name(InStr);
		Hasher.Update(Name.GetData(), Name.Len() * sizeof(TCHAR));
	};

	for (FModifierIndex Modifier : AllModifierIndices())
	{
		ensure(ModifierCacheKeys[Modifier].IsValid());
		
		// Include the modifier content hash. This represents the state of all the properties of a modifier which contribute to its
		// effects when applied to a base mesh.
		UpdateHashPODType(Hasher, ModifierCacheKeys[Modifier]);

		// Include the modifier path in the group hash. This is to ensure that even if the content hash of the modifier (the modifier cache key)
		// changes, the group will not match. Groups point to specific modifiers, while modifier cache keys represent only the content state of
		// a given modifier. Two modifiers may share identical content states (eg. if they have the identical base mesh) and yet should be treated
		// as distinct.
		UpdateHashFString(Hasher, GetModifierDesc(Modifier).ModifierPath.ToString());
		// Include the class path of a modifier in case the same modifier path suddenly changes to point to a different type. This can happen
		// when recompiling blueprints, changing the class of something in the editor, redirectors, etc. Most of this should be captured by the
		// content hash but to be sure we include to class path.
		UpdateHashFString(Hasher, GetModifierDesc(Modifier).ClassPath.ToString());
	}

	return Hasher.Finalize();
}

void SortModifierDescriptors(TConstArrayView<const FName> InModifierTypePriorities, TArray<MeshPartition::FModifierDesc>& InOutModifierDescriptors)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::MeshPartition::SortModifierDescriptors)

	InOutModifierDescriptors.Sort([InModifierTypePriorities](const MeshPartition::FModifierDesc& ModifierDesc, const MeshPartition::FModifierDesc& Other)
	{
		return MeshPartition::FModifierGroup::ShouldApplyModifierBefore(InModifierTypePriorities, ModifierDesc, Other);
	});
}

TArray<MeshPartition::FModifierGroup> BuildModifierGroups(TConstArrayView<MeshPartition::FModifierDesc> InModifierDescriptors, const MeshPartition::FBuilderSettings& InSettings)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::MeshPartition::BuildModifierGroups)

	TArray<UE::MeshPartition::FModifierGroup> Results;

	TArray<MeshPartition::FModifierDesc> BaseDescriptors;
	TArray<MeshPartition::FModifierDesc> ModifierDescriptors;
	TArray<MeshPartition::FModifierDesc> ContiguousModifierDescriptors;

	for (int Index = 0; Index < InModifierDescriptors.Num(); ++Index)
	{
		if (InModifierDescriptors[Index].IsBase())
		{
			BaseDescriptors.Add(InModifierDescriptors[Index]);
		}
		else
		{
			ModifierDescriptors.Add(InModifierDescriptors[Index]);

			if (InModifierDescriptors[Index].bIsContiguous)
			{
				ContiguousModifierDescriptors.Add(InModifierDescriptors[Index]);
			}
		}
	}

	// First sorting all modifiers so any processing will evaluate modifiers in the proper order.
	SortModifierDescriptors(InSettings.TypePriorities, ModifierDescriptors);

	// Remove all base modifiers that are already over the limit
	BaseDescriptors.RemoveAll([&Results, &ModifierDescriptors, MaxComplexity = InSettings.MaxSectionComplexity](const MeshPartition::FModifierDesc& Base)
	{
		UE::MeshPartition::FModifierGroup ModifierGroup = BuildModifierGroupForBase({Base}, ModifierDescriptors);

		if (ModifierGroup.ComputeTotalComplexity() >= MaxComplexity)
		{
			Results.Emplace(MoveTemp(ModifierGroup));
			return true;
		}

		return false;
	});

	while (!BaseDescriptors.IsEmpty())
	{
		MeshPartition::FModifierDesc NextBase = BaseDescriptors.Pop();
		TArray<MeshPartition::FModifierDesc> CurrentBase = { NextBase };

		UE::MeshPartition::FModifierGroup CurrentGroup = BuildModifierGroupForBase(CurrentBase, ModifierDescriptors);

		TArray<const MeshPartition::FModifierDesc> Neighbors = Private::GetBaseNeighbors(CurrentBase, BaseDescriptors);
		bool IsGrowing = true;

		// The base keeps growing until constraint are not satisfied or there is no neighbour to add.
		while (!Neighbors.IsEmpty() && IsGrowing)
		{
			IsGrowing = false;
			
			for (const MeshPartition::FModifierDesc& Neighbour : Neighbors)
			{
				TArray<MeshPartition::FModifierDesc> TentativeBaseGroup = CurrentBase;

				TentativeBaseGroup.Emplace(Neighbour);

				UE::MeshPartition::FModifierGroup TentativeGroup = BuildModifierGroupForBase(TentativeBaseGroup, ModifierDescriptors);

				if (TentativeGroup.ComputeTotalComplexity() <= InSettings.MaxSectionComplexity)
				{
					CurrentBase.AddUnique(Neighbour);

					BaseDescriptors.Remove(Neighbour);
					CurrentGroup = MoveTemp(TentativeGroup);
					IsGrowing = true;
				}
			}

			// Update the neighbors since the base changed.
			if (IsGrowing)
			{
				Neighbors = Private::GetBaseNeighbors(CurrentBase, BaseDescriptors);
			}
		}

		Results.Emplace(MoveTemp(CurrentGroup));
	}
	Private::MergeContiguousModifierGroups(ContiguousModifierDescriptors, ModifierDescriptors, Results);

	return Results;
}
} // namespace UE::MeshPartition