// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModifierGraphCache.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionEditorModule.h"
#include "Algo/MaxElement.h"
#include "HAL/IConsoleManager.h"
#include "MeshPartitionEditorSubsystem.h"

static TAutoConsoleVariable<bool> CVarCacheEnabled(
	TEXT("MegaMesh.Cache.Enabled.PreviewSection"),
	true,
	TEXT("Enables the MegaMesh Preview Section build cache")
);

static TAutoConsoleVariable<float> CVarCacheMemoryBudgetMB(
	TEXT("MegaMesh.Cache.BudgetMB.PreviewSection"),
	3. * 1024.,
	TEXT("Memory budget for the MegaMesh Preview Section build cache in MB")
);

static TAutoConsoleVariable<float> CVarMergedBaseCacheMemoryBudgetMB(
	TEXT("MegaMesh.Cache.BudgetMB.MergedBase"),
	5. * 1024.,
	TEXT("Memory budget for the MegaMesh Preview Section build cache in MB")
);

namespace
{
	/**
	 * Initial max number of entries in cache. 
	 * This number won't ever be hit as the LruCache is just used to track least recently used cache entries.
	 * The system manually releases elements when memory budget is exceeded.
	 */
	constexpr uint64 CacheMaxNumElements = 65536;
}

namespace UE::MeshPartition
{

bool FModifierGraphCache::IsCachingEnabled()
{
	return CVarCacheEnabled.GetValueOnAnyThread();
}

FModifierGraphCache::FModifierGraphCache()
	: MergedBaseCache(CacheMaxNumElements)
	, LeastRecentlyUsedModifiers(CacheMaxNumElements)
{

}

void FModifierGraphCache::CacheMergedBaseMesh(const FGuid& InCacheKey, const FMeshData& InMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierGraphCache::CacheMergedBaseMesh);

	ensure(IsCachingEnabled());

	FWriteScopeLock Lock(MergedBaseCacheMutex);

	if (!ensure(InCacheKey.IsValid()))
	{
		return;
	}

	// Cache already contains an entry for this key.
	// This can happen if two exactly equal ModifierGroups are flowing through builders concurrently.
	// There was a race between the two builders and one already committed the result to the cache while the other was still processing.
	// There is no need to commit this entry to the cache because it will already exist and would be exactly equal to the other one.
	if (MergedBaseCache.Contains(InCacheKey))
	{
		return;
	}

	const double CacheMemoryBudgetMB = CVarMergedBaseCacheMemoryBudgetMB.GetValueOnAnyThread();
	const double NewEntryMemorySizeMB = static_cast<double>(InMesh.GetByteCount()) / (1024.0 * 1024.0);

	// New cache entry exceeds the entire cache budget
	if (NewEntryMemorySizeMB >= CacheMemoryBudgetMB)
	{
		UE_LOGF(LogMegaMeshEditor, Log,
			   "Attempted to store a mesh in the MegaMesh cache which completely exceeds the maximum cache size.\n" "Consider increasing the cache memory budget (MegaMesh.Cache.MemoryBudgetMB [new_size])\n" "Otherwise, disabling the cache altogether is possible to reduce memory consumption " "at the cost of editor iteration performance. (MegaMesh.Cache.Enabled 0)");
		return;
	}

	while (BaseMeshMemoryUsedMB + NewEntryMemorySizeMB > CacheMemoryBudgetMB && MergedBaseCache.Num() > 0)
	{
		double EvictedSize = RemoveLeastRecentlyUsedMergedBase_Unsafe();
		UE_LOGF(LogMegaMeshEditor, Verbose, "Cache memory limit exceeded. Evicted entry with %.3f MB to create space for new entry", EvictedSize);
	}

	AddMergedBaseToCache_Unsafe(InCacheKey, InMesh);
}

bool FModifierGraphCache::GetCachedMergedBaseMesh(const FGuid& InCacheKey, FMeshData& OutCachedMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierGraphCache::GetCachedMergedBaseMesh);
	
	if (!IsCachingEnabled())
	{
		return false;
	}

	FReadScopeLock Lock(MergedBaseCacheMutex);

	if (const FMergedBaseCacheEntry* Entry = MergedBaseCache.FindAndTouch(InCacheKey))
	{
		CopyMesh(Entry->MeshData, OutCachedMesh);
		return true;
	}

	return false;
}

double FModifierGraphCache::GetTotalMemoryUsageMB() const
{
	return GetMergedBaseCacheMemoryUsageMB() + GetViewCacheMemoryUsageMB();
}

double FModifierGraphCache::GetViewCacheMemoryUsageMB() const
{
	UE::TUniqueLock BaseCacheDataLock(BaseViewCacheDataMutex);
	
	double MemoryUsage = 0;
	for (const TPair<FBaseGroupGuid, double>& Modifier : LeastRecentlyUsedModifiers)
	{
		MemoryUsage += Modifier.Value;
	}

	return MemoryUsage;
}

double FModifierGraphCache::FModifierCacheData::GetMemoryUsageMB() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FModifierCacheData::GetMemoryUsageMB)

	double MemoryUsage = static_cast<double>(LocalViewHashes.GetAllocatedSize()) / (1024. * 1024.);

	for (const MeshPartition::FMeshView& View : MeshViews)
	{
		MemoryUsage += View.GetMemoryUsageMB();
	}
	return MemoryUsage;
}

void FModifierGraphCache::Clear()
{
	{
		FWriteScopeLock Lock(MergedBaseCacheMutex);
		MergedBaseCache.Empty(CacheMaxNumElements);
	}

	{
		UE::TUniqueLock Lock(BaseViewCacheDataMutex);
		BaseViewCacheData.Empty();
		LeastRecentlyUsedModifiers.Empty(CacheMaxNumElements);
	}

	BaseMeshMemoryUsedMB = 0;
}

bool FModifierGraphCache::CopyBaseGroupViewCacheData(const FGuid& InBaseGroupKey, TConstArrayView<FSoftObjectPath> InModifierPaths, FBaseGroupViewCacheData& OutBaseGroupViewCacheData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierGraphCache::CopyBaseGroupViewCacheData)

	TSharedPtr<FBaseGroupViewCacheData> FullBaseGroupCacheData = nullptr;

	{
		UE::TUniqueLock Lock(BaseViewCacheDataMutex);
		if (TSharedPtr<FBaseGroupViewCacheData>* FullBaseGroupCacheDataPtr = BaseViewCacheData.Find(InBaseGroupKey))
		{
			FullBaseGroupCacheData = *FullBaseGroupCacheDataPtr;
		}
	}

	if (FullBaseGroupCacheData)
	{
		UE::TUniqueLock Lock(FullBaseGroupCacheData->Mutex);

		for (const FSoftObjectPath& ModifierPath : InModifierPaths)
		{
			if (const FModifierCacheData* ModifierCacheData = FullBaseGroupCacheData->ModifierCacheData.Find(ModifierPath))
			{
				OutBaseGroupViewCacheData.ModifierCacheData.Emplace(ModifierPath, {});
			}
		}

		ParallelFor(InModifierPaths.Num(), [&](int Index)
		{
			const FSoftObjectPath& ModifierPath = InModifierPaths[Index];
			if (FullBaseGroupCacheData->ModifierCacheData.Contains(ModifierPath))
			{
				OutBaseGroupViewCacheData.ModifierCacheData[ModifierPath] = FullBaseGroupCacheData->ModifierCacheData[ModifierPath];
			}
		});

		return true;
	}

	return false;
}

void FModifierGraphCache::CacheBaseGroupViewCacheData(const FGuid& InBaseGroupCacheKey, FBaseGroupViewCacheData&& InBaseGroupViewCacheData)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierGraphCache::CacheBaseGroupViewCacheData)

	if (!IsCachingEnabled())
	{
		return;
	}

	TSharedPtr<FBaseGroupViewCacheData> FullBaseGroupCacheData;

	{
		UE::TUniqueLock Lock(BaseViewCacheDataMutex);
		FullBaseGroupCacheData = BaseViewCacheData.FindOrAdd(InBaseGroupCacheKey, MakeShared<FBaseGroupViewCacheData>());
	}

	// Update recently used cache entries:
	{
		UE::TUniqueLock Lock(BaseViewCacheDataMutex);

		for (const TPair<FSoftObjectPath, FModifierCacheData>& ModifierData : InBaseGroupViewCacheData.ModifierCacheData)
		{
			const FSoftObjectPath& ModifierPath = ModifierData.Key;

			const double ModifierMemoryUsageMB = ModifierData.Value.GetMemoryUsageMB();

			if (LeastRecentlyUsedModifiers.Contains(ModifierPath))
			{
				TPair<FBaseGroupGuid, double>& ModifierLruData = LeastRecentlyUsedModifiers.FindAndTouchChecked(ModifierPath);
				ModifierLruData.Value = ModifierMemoryUsageMB;
			}
			else
			{
				LeastRecentlyUsedModifiers.Add(ModifierPath, TPair<FBaseGroupGuid, double>(InBaseGroupCacheKey, ModifierMemoryUsageMB));
			}
		}
	}

	// Update the cache data with new data:
	{
		UE::TUniqueLock Lock(FullBaseGroupCacheData->Mutex);

		for (TPair<FSoftObjectPath, FModifierCacheData>& InModifierData : InBaseGroupViewCacheData.ModifierCacheData)
		{
			const FSoftObjectPath& ModifierPath = InModifierData.Key;

			FullBaseGroupCacheData->ModifierCacheData.FindOrAdd(ModifierPath, {});
		}

		ParallelFor(InBaseGroupViewCacheData.ModifierCacheData.GetMaxIndex(), [&](int Index)
		{
			const FSetElementId Id = FSetElementId::FromInteger(Index);
			if (InBaseGroupViewCacheData.ModifierCacheData.IsValidId(Id))
			{
				TPair<FSoftObjectPath, FModifierCacheData>& InModifierData = InBaseGroupViewCacheData.ModifierCacheData.Get(Id);

				FModifierCacheData& ModifierCacheData = FullBaseGroupCacheData->ModifierCacheData.FindChecked(InModifierData.Key);
				ModifierCacheData = MoveTemp(InModifierData.Value);
			}
		});
	}

	EnsureViewMemoryBudget();
}

void FModifierGraphCache::EnsureViewMemoryBudget()
{
	// #todo: currently the view cache budget and the layer mesh cache budget are distinct and separate
	// This is a simplification for now because the plan is to remove the layer cache and I don't think it's worth
	// investing much time in forcing them to play perfectly nice when swapping between them
	const double CacheMemoryBudget = CVarCacheMemoryBudgetMB.GetValueOnAnyThread();
	double CurrentViewCacheMemoryUsageMB = GetViewCacheMemoryUsageMB();

	if (CurrentViewCacheMemoryUsageMB > CacheMemoryBudget)
	{
		UE::TUniqueLock BaseViewLock(BaseViewCacheDataMutex);
		while (CurrentViewCacheMemoryUsageMB > CacheMemoryBudget && (LeastRecentlyUsedModifiers.Num() > 0))
		{
			CurrentViewCacheMemoryUsageMB -= RemoveLeastRecentlyUsedViewCache_Unsafe();
		}
	}
}

void FModifierGraphCache::CopyMesh(const FMeshData& InSourceMesh, FMeshData& OutMesh)
{
	OutMesh.Copy(InSourceMesh);
}

void FModifierGraphCache::AddMergedBaseToCache_Unsafe(const FGuid& InCacheKey, const FMeshData& InMesh)
{
	FMergedBaseCacheEntry& Entry = MergedBaseCache.AddUninitialized_GetRef(InCacheKey);
	Entry = FMergedBaseCacheEntry();

	CopyMesh(InMesh, Entry.MeshData);

	const SIZE_T InMeshSizeMB = (InMesh.GetByteCount() / (1024 * 1024));
	BaseMeshMemoryUsedMB += InMeshSizeMB;
}

double FModifierGraphCache::RemoveLeastRecentlyUsedMergedBase_Unsafe()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierGraphCache::RemoveLeastRecentlyUsedMergedBase_Unsafe);

	FMergedBaseCacheEntry LruEntry = MergedBaseCache.RemoveLeastRecent();

	const double MeshSizeMB = static_cast<double>(LruEntry.MeshData.GetByteCount()) / (1024. * 1024.);

	BaseMeshMemoryUsedMB -= MeshSizeMB;

	return MeshSizeMB;
}

double FModifierGraphCache::RemoveLeastRecentlyUsedViewCache_Unsafe()
{
	if (!ensure(LeastRecentlyUsedModifiers.Num() > 0))
	{
		return 0.;
	}

	const FSoftObjectPath LeastRecentModifierPath = LeastRecentlyUsedModifiers.GetLeastRecentKey();
	const TPair<FBaseGroupGuid, double> LeastRecentModifier = LeastRecentlyUsedModifiers.RemoveLeastRecent();
	const FBaseGroupGuid LeastRecentModifierBaseGroup = LeastRecentModifier.Key;
	const double LeastRecentModifierMemoryUsage = LeastRecentModifier.Value;

	TSharedPtr<FBaseGroupViewCacheData> BaseCacheData = nullptr;

	{
		if (TSharedPtr<FBaseGroupViewCacheData>* BaseCacheDataPtr = BaseViewCacheData.Find(LeastRecentModifierBaseGroup))
		{
			BaseCacheData = *BaseCacheDataPtr;
		}
	}

	if (ensure(BaseCacheData))
	{
		UE::TUniqueLock Lock(BaseCacheData->Mutex);
		const int32 Removed = BaseCacheData->ModifierCacheData.Remove(LeastRecentModifierPath);
		ensure(Removed > 0);
	}

	return LeastRecentModifierMemoryUsage;
}

} // namespace UE::MeshPartition