// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModifierTaskGraph.h"
#include "MeshPartitionMeshView.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionDefinition.h"
#include "Async/ParallelFor.h"
#include "MeshPartitionEditorSubsystem.h"

#include "MeshPartitionModifierGraphCache.h"
#include "VisualLogger/VisualLogger.h"
#include "MeshPartitionEditorModule.h"

static TAutoConsoleVariable<bool> CVarValidateModifierDeterminism(
	TEXT("MegaMesh.Cache.ValidateModifierDeterminism"),
	false,
	TEXT("Enables a debug mode for the MegaMesh Modifier Task Graph which validates that unchanged modifiers generate consistent and exactly equal outputs across graph executions\n")
	TEXT("Enabling this mode will drastically slow down the execution of the graph.")
);

static TAutoConsoleVariable<bool> CVarOutputGraphViz(
	TEXT("MegaMesh.TaskGraph.LogDotGraph"),
	false,
	TEXT("Outputs dotgraph representation of the modifier graph after execution.")
);

namespace UE::MeshPartition
{
bool IsTopologyModifier(EMeshViewComponents InWriteViewComponents)
{
	return EnumHasAnyFlags(InWriteViewComponents, EMeshViewComponents::DynamicSubmesh);
}

bool ModifierNeedsExclusiveWriteback(EMeshViewComponents InWriteViewComponents)
{
	return IsTopologyModifier(InWriteViewComponents);
}

struct FScopedCounterIncrement
{
	FScopedCounterIncrement(uint64& InCounterToIncrement)
		: Start(FPlatformTime::Cycles64())
		, CounterToIncrement(InCounterToIncrement)
	{}

	~FScopedCounterIncrement()
	{
		CounterToIncrement += Delta();
	}

	uint64 Delta()
	{
		return FPlatformTime::Cycles64() - Start;
	}

	uint64 Start;
	uint64& CounterToIncrement;
};

void PrepareWeightLayers(FMeshData& InOutBaseMesh, const MeshPartition::FModifierGroup& InGroup)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MegaMeshModifierTaskGraph::PrepareWeightLayers)

	TSet<FName> ModifierChannels;

	for (const FInstanceInfo& InstanceInfo : InGroup.Instances())
	{
		ModifierChannels.Append(InstanceInfo.UsedChannels);
	}

	for (const FName& ModifierChannel : ModifierChannels)
	{
		InOutBaseMesh.InitializeWeightLayer(ModifierChannel);
	}
}

FModifierTaskGraph::FModifierTaskGraph()
	: MeshPipe(TEXT("MegaMesh_TaskGraph_MeshPipe"))
{
}

FModifierTaskGraph::~FModifierTaskGraph()
{
	// Ensure we wait on the main task before destroying the task graph.
	WaitForCompletion();
}

Tasks::FTask FModifierTaskGraph::Execute(FMeshData&& InBaseMesh, const FGuid& InBaseGroupCacheKey, MeshPartition::FModifierGroup&& InModifierGroup, const FTransform& InTransform, TConstArrayView<FName> InModifierTypePriorities, const bool bInUseCache)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierTaskGraph::Execute)

	BaseGroupKey = InBaseGroupCacheKey;
	Mesh = MoveTemp(InBaseMesh);
	Group = MoveTemp(InModifierGroup);

	Group.ProgressToState(MeshPartition::FModifierGroup::EState::InstancesReady);

	bUsingCache = MeshPartition::FModifierGraphCache::IsCachingEnabled() && bInUseCache;

	ExecuteStartTime = FPlatformTime::Cycles64();

	if (bUsingCache)
	{
		// Create a local working copy of existing cache data for this modifier group
		// In an ideal world, this cache data could be mutated in place without requiring this extra copy.
		// However, because two task graphs can be executing on the same base group we cannot allow this to happen.

		TArray<FSoftObjectPath> UsedModifierPaths;
		for (const MeshPartition::FModifierDesc& Modifier : Group.ModifierDescs())
		{
			UsedModifierPaths.Emplace(Modifier.ModifierPath);
		}

		if (MeshPartition::FModifierGraphCache* Cache = UMeshPartitionEditorSubsystem::GetGraphCache())
		{
			Cache->CopyBaseGroupViewCacheData(InBaseGroupCacheKey, UsedModifierPaths, ActiveGroupCacheData);
		}
	}

	PrepareWeightLayers(Mesh, Group);

	BuildDependencyGraph(InModifierTypePriorities);

	if (bUsingCache)
	{
		PrepareViewCache();
	}

	Tasks.SetNumZeroed(Group.Instances().Num());

	const bool bValidateDeterminism = CVarValidateModifierDeterminism.GetValueOnAnyThread();

	ParallelFor(Group.Instances().Num(), [&, this] (int InInstanceIndex) mutable
	{
		FInstanceIndex InstanceIndex(InInstanceIndex);
		const FInstanceInfo& InstanceDesc = Group.GetInstanceInfo(InstanceIndex);

		// Transform the bounds into the local space of the mesh so we can compare vertices against it.
		const FBox LocalspaceInstanceBounds = InstanceDesc.Bounds.InverseTransformBy(InTransform);

		Tasks::FTaskEvent TriggerEvent(TEXT("MMMTaskGraph_TriggerEvent"));

		Tasks::FTask BuildViewTask = MeshPipe.LaunchShared(
			TEXT("MMMTaskGraph_BuildViewTask"),
			[this, InstanceIndex, bValidateDeterminism]
			{
				UE_MT_SCOPED_READ_ACCESS(MeshAccessDetector);

				if (bIsCancelled)
				{
					return;
				}

				FModifierInstanceTask& Task = Tasks[InstanceIndex];

				FScopedCounterIncrement ScopedIncrement(Task.PerformanceCounter);

				Task.State = ETaskState::Active;

				if (!Task.bUsingCachedView || bValidateDeterminism)
				{
					Task.MeshView.Build();
				}
				else
				{
					MeshPartition::FModifierGraphCache::FModifierCacheData& ModifierCacheData = ActiveGroupCacheData.ModifierCacheData.FindChecked(Group.GetModifierDesc(Task.InstanceIndex).ModifierPath);
					Task.MeshView = ModifierCacheData.MeshViews[Group.GetInstanceInfo(Task.InstanceIndex).InstanceID];
					Task.MeshView.RemapParentMesh(&Mesh);
				}
			},
			Tasks::Prerequisites(TriggerEvent)
		);

		// Intermediate task which dynamically adds the async prepare task as a prerequisite only if this instance was not able to
		// reuse cached results.
		Tasks::FTask WaitAsyncInit = MeshPipe.LaunchShared(
			TEXT("MMMTaskGraph_WaitAsyncInit"),
			[this, InstanceIndex, bValidateDeterminism]
			{
				if (bIsCancelled)
				{
					return;
				}

				FModifierInstanceTask& Task = Tasks[InstanceIndex];
				if (!Task.bUsingCachedView || bValidateDeterminism)
				{
					TSharedPtr<const MeshPartition::IModifierBackgroundOp> ModifierOp = Group.GetModifierOp(InstanceIndex);
					const UE::Tasks::FTask AsyncPrepare = ModifierOp->GetAsyncPrepareTask();
					if (AsyncPrepare.IsValid())
					{
						UE::Tasks::AddNested(AsyncPrepare);
					}
				}
			},
			Tasks::Prerequisites(BuildViewTask),
			Tasks::ETaskPriority::Inherit,
			// Inline the task so that it does not get scheduled. This task is an immediate return
			// in most cases and does nothing. We don't want to put unnecessary pressure on the scheduler.
			Tasks::EExtendedTaskPriority::Inline
		);

		Tasks::FTask ComputeTask = Tasks::Launch(
			TEXT("MMMTaskGraph_ComputeTask"),
			[this, InstanceIndex, Transform = InTransform, bValidateDeterminism] ()
			{
				if (bIsCancelled)
				{
					return;
				}
				
				FModifierInstanceTask& Task = Tasks[InstanceIndex];

				FScopedCounterIncrement ScopedIncrement(Task.PerformanceCounter);

				if (!Task.bUsingCachedView || bValidateDeterminism)
				{
					TSharedPtr<const MeshPartition::IModifierBackgroundOp> ModifierOp = Group.GetModifierOp(Task.InstanceIndex);
					if (ensure(ModifierOp)) // Shouldn't have added an instance if this was null
					{
						ModifierOp->ApplyModifications(Task.MeshView, Transform, Group.GetInstanceInfo(Task.InstanceIndex));
					}

					// The modifier's result was recomputed, downstream modifiers are no longer permitted use their cached data:
					for (FInstanceIndex Downstream : DownstreamMap[InstanceIndex])
					{
						Tasks[Downstream].bDebugWasCachePoisoned = true;
						Tasks[Downstream].bUsingCachedView = false;
					}
				}

				if (bValidateDeterminism && Task.bUsingCachedView)
				{
					ValidateInstanceOutputsForDeterminism(Task);
				}
			},
			Tasks::Prerequisites(WaitAsyncInit)
		);

		Tasks::FTask WriteTask;

		if (ModifierNeedsExclusiveWriteback(InstanceDesc.WriteViewComponents))
		{
			WriteTask = MeshPipe.LaunchExclusive(
				TEXT("MegaMesh_TaskGraph_ExclusiveWriteback"),
				[this, InstanceIndex]()
				{
					UE_MT_SCOPED_WRITE_ACCESS(MeshAccessDetector);

					if (bIsCancelled)
					{
						return;
					}

					FModifierInstanceTask& Task = Tasks[InstanceIndex];

					FScopedCounterIncrement ScopedIncrement(Task.PerformanceCounter);

					ApplyTaskWriteback(Task);
				},
				Tasks::Prerequisites(ComputeTask)
			);
		}
		else
		{
			WriteTask = MeshPipe.LaunchShared(
				TEXT("MegaMesh_TaskGraph_ParallelWriteback"),
				[this, InstanceIndex]()
				{
					UE_MT_SCOPED_READ_ACCESS(MeshAccessDetector);

					if (bIsCancelled)
					{
						return;
					}

					FModifierInstanceTask& Task = Tasks[InstanceIndex];

					FScopedCounterIncrement ScopedIncrement(Task.PerformanceCounter);

					ApplyTaskWriteback(Task);
				},
				Tasks::Prerequisites(ComputeTask)
			);
		}

		Tasks::FTask CompleteInstanceTask = Tasks::Launch(
			TEXT("MMMTaskGraph_OnTaskCompleted"),
			[this, InstanceIndex]()
			{
				if (bIsCancelled)
				{
					return;
				}

				OnTaskCompleted(InstanceIndex);
			},
			Tasks::Prerequisites(WriteTask)
		);


		new (&Tasks[InstanceIndex]) FModifierInstanceTask{
			.State = ETaskState::Waiting,
			.WritebackBounds = InstanceDesc.Bounds,
			.TriggerEvent = TriggerEvent,
			.BuildViewTask = BuildViewTask,
			.ComputeTask = ComputeTask,
			.WriteTask = WriteTask,
			.CompleteTask = CompleteInstanceTask,
			.MeshView = MeshPartition::FMeshView(&Mesh, LocalspaceInstanceBounds, InstanceDesc.ReadViewComponents, InstanceDesc.WriteViewComponents, InstanceDesc.UsedChannels),
			.InstanceIndex = InstanceIndex,
			.bUsingCachedView = InstancesReusingCachedViews[InstanceIndex],
		};
	});


	// Create dependency links between start and end tasks
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierTaskGraph::Execute_SetupPrerequisiteEdges);
		ParallelFor(Tasks.Num(), [this](int TaskIndex)
		{
			FModifierInstanceTask& Task = Tasks[TaskIndex];

			const TArray<FInstanceIndex>& PrerequisiteIndices = PrerequisiteMap[TaskIndex];

			for (FInstanceIndex PrerequisiteIndex : PrerequisiteIndices)
			{
				FModifierInstanceTask& Prereq = Tasks[PrerequisiteIndex];
				Task.TriggerEvent.AddPrerequisites(Prereq.WriteTask);
			}

			// once the task's prerequisites are set up, 
			Task.TriggerEvent.Trigger();
		});
	}

	// Build the final task dependency list now.
	// we're waiting to do this extra synchronous work after the graph is already launched to avoid holding the graph back

	TArray<Tasks::FTask> AllCompletionTasks; 
	for (const FModifierInstanceTask& Task : Tasks)
	{
		AllCompletionTasks.Add(Task.CompleteTask);
	}

	GraphCompleteTask = Tasks::Launch(TEXT("MegaMesh_TaskGraph_OnGraphComplete"),
		[this]()
		{
			OnGraphCompleted();
		},
		Tasks::Prerequisites(AllCompletionTasks)
	);

	return GraphCompleteTask;
}

void FModifierTaskGraph::BuildDependencyGraph(TConstArrayView<FName> InModifierTypePriorities)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierTaskGraph::BuildDependencyGraph);

	const int32 NumInstances = Group.Instances().Num();
	PrerequisiteMap.SetNum(NumInstances);
	DownstreamMap.SetNum(NumInstances);
	InstancesReusingCachedViews.SetNum(NumInstances);
	InstanceViewCacheKeys.SetNum(NumInstances);

	// Build the dependencies for each modifier in parallel. It is safe to modify the Dependency list for each modifier individually
	ParallelFor(NumInstances, [&](int InInstanceIndex)
	{
		const FInstanceIndex InstanceIndex(InInstanceIndex);
		const FInstanceInfo& InstanceDesc = Group.GetInstanceInfo(InstanceIndex);

		TArray<FInstanceIndex>& Prerequisites = PrerequisiteMap[InInstanceIndex];
		FGuid& ViewCacheKey = InstanceViewCacheKeys[InInstanceIndex];
		ViewCacheKey = FGuid();

		for (int OtherIndex = 0; OtherIndex < NumInstances; ++OtherIndex)
		{
			const FInstanceIndex OtherInstanceIndex(OtherIndex);
			if (InstanceIndex == OtherInstanceIndex)
			{
				continue;
			}

			const FInstanceInfo& OtherInstanceDesc = Group.GetInstanceInfo(OtherInstanceIndex);

			if (Group.ShouldApplyInstanceBefore(InModifierTypePriorities, OtherInstanceDesc, InstanceDesc))
			{
				if (Group.HasDependency(InstanceDesc, OtherInstanceDesc)
					// Enforce a global sequential consistency for modifiers which both write to topology.
					// This guarantees the VIDs assigned to each of their writebacks is reproducible and consistent.
					|| (IsTopologyModifier(InstanceDesc.WriteViewComponents) && IsTopologyModifier(OtherInstanceDesc.WriteViewComponents)))
				{
					Prerequisites.Add(OtherInstanceIndex);
					ViewCacheKey = FGuid::Combine(ViewCacheKey, Group.GetModifierCacheKey(OtherInstanceIndex));
				}
				// All topology modifiers which come before this one must be included in the cache key regardless if there is a direct dependency.
				// Optimally, this only needs to be the topology modifiers which have a direct dependency chain 
				// but doing it this way (affected by the global list of the topology modifiers) doesn't require
				// the dependency chain for that modifier to have been built already.
				// To achieve that, we could start by building the dependency chain for topology modifiers in a first pass
				// and then a second pass would process this all normally.
				else if (IsTopologyModifier(OtherInstanceDesc.WriteViewComponents))
				{
					ViewCacheKey = FGuid::Combine(ViewCacheKey, Group.GetModifierCacheKey(OtherInstanceIndex));
				}
			}
		}

		// Include the self key in the local view cache key
		const FGuid InstanceCacheKey = Group.GetModifierCacheKey(InstanceIndex);
		ensure(InstanceCacheKey.IsValid());
		ViewCacheKey = FGuid::Combine(ViewCacheKey, InstanceCacheKey);
	});

	// Forward pass to calculate all downstream modifiers for each modifier.
	// We need the downstream list to push the cache invalidations forwards.
	ParallelFor(NumInstances, [&](int InInstanceIndex)
	{
		const FInstanceIndex InstanceIndex(InInstanceIndex);

		TArray<FInstanceIndex>& Downstreams = DownstreamMap[InInstanceIndex];

		for (int OtherIndex = 0; OtherIndex < NumInstances; ++OtherIndex)
		{
			const FInstanceIndex OtherInstanceIndex(OtherIndex);
			if (InstanceIndex == OtherInstanceIndex)
			{
				continue;
			}

			if (Group.HasDependency(InstanceIndex, OtherInstanceIndex)
					|| (IsTopologyModifier(Group.GetInstanceInfo(InstanceIndex).WriteViewComponents) && IsTopologyModifier(Group.GetInstanceInfo(OtherInstanceIndex).WriteViewComponents)))
			{
				if (Group.ShouldApplyInstanceBefore(InModifierTypePriorities, InstanceIndex, OtherInstanceIndex))
				{
					Downstreams.Add(OtherInstanceIndex);
				}
			}
		}
	});
}

void FModifierTaskGraph::PrepareViewCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierTaskGraph::PrepareViewCache);

	ensure(bUsingCache);

	const int32 NumInstances = Group.Instances().Num();

	// Determine if the view cache can be reused for each modifier instance. Compare the local hash of each modifier instance with the cached view hash.
	// If they match, the view cache can be used. The cache may still be rejected if an upstream chain of indirect dependencies fails.
	ParallelFor(NumInstances, [this](int InInstanceIndex)
	{
		const FInstanceIndex InstanceIndex(InInstanceIndex);

		if (const MeshPartition::FModifierGraphCache::FModifierCacheData* ModifierCacheData = ActiveGroupCacheData.ModifierCacheData.Find(Group.GetModifierDesc(InstanceIndex).ModifierPath))
		{
			if (ModifierCacheData->CacheKey == Group.GetModifierCacheKey(InstanceIndex))
			{
				const FInstanceInfo& InstanceDesc = Group.GetInstanceInfo(InstanceIndex);
				// If the cache key matches the saved key, the instance count should guaranteed match the cached data
				// failing this ensure would indicate a bug in the cache key update code for the modifier type.
				if (ensure(InstanceDesc.InstanceID < ModifierCacheData->LocalViewHashes.Num()))
				{
					InstancesReusingCachedViews[InstanceIndex] = (InstanceViewCacheKeys[InstanceIndex] == ModifierCacheData->LocalViewHashes[InstanceDesc.InstanceID]);
				}
			}
		}
	});


	// Pre-allocate the per-instance data to the correct size so cached data can be placed into the array in parallel during writeback
	for (FInstanceIndex InstanceIndex : Group.InstanceIndices())
	{
		const FInstanceInfo& InstanceDesc = Group.GetInstanceInfo(InstanceIndex);
		MeshPartition::FModifierGraphCache::FModifierCacheData& ModifierCacheData = ActiveGroupCacheData.ModifierCacheData.FindOrAdd(Group.GetModifierDesc(InstanceIndex).ModifierPath, {});

		ModifierCacheData.CacheKey = Group.GetModifierCacheKey(InstanceIndex);

		ModifierCacheData.MeshViews.SetNum(FMath::Max(ModifierCacheData.MeshViews.Num(), InstanceDesc.InstanceID + 1));
		ModifierCacheData.LocalViewHashes.SetNum(FMath::Max(ModifierCacheData.LocalViewHashes.Num(), InstanceDesc.InstanceID + 1));

		ModifierCacheData.LocalViewHashes[InstanceDesc.InstanceID] = InstanceViewCacheKeys[InstanceIndex];
	}
}

void FModifierTaskGraph::WaitForCompletion()
{
	Tasks::Wait({ GraphCompleteTask });
}

void FModifierTaskGraph::Cancel()
{
	bIsCancelled = true;
}

bool FModifierTaskGraph::IsComplete() const
{
	return GraphCompleteTask.IsCompleted();
}

void FModifierTaskGraph::ApplyTaskWriteback(FModifierInstanceTask& Task)
{
	// if somehow the compute task was not completed, which would be a critical failure in the task graph scheduling,
	// early out before applying any writeback to the mesh to try to prevent fatal errors.
	if (!ensure(Task.ComputeTask.IsCompleted()))
	{
		return;
	}

	Task.MeshView.Writeback();
}

void FModifierTaskGraph::OnTaskCompleted(int TaskIndex)
{
	FModifierInstanceTask& Task = Tasks[TaskIndex];

	ensure(Task.WriteTask.IsCompleted());

	Task.State = ETaskState::Completed;

	if (bUsingCache)
	{
		// If the task wasn't already using cached outputs, write the outputs back to the cache
		if (!Task.bUsingCachedView)
		{
			MeshPartition::FModifierGraphCache::FModifierCacheData& ModifierCacheData = ActiveGroupCacheData.ModifierCacheData.FindChecked(Group.GetModifierDesc(Task.InstanceIndex).ModifierPath);
			ModifierCacheData.MeshViews[Group.GetInstanceInfo(Task.InstanceIndex).InstanceID] = MoveTemp(Task.MeshView);
		}
	}
	else
	{
		// Manually release resources in the mesh view to free up resources since we don't call a destructor when a task completes.
		Task.MeshView.Release();
	}
}

void FModifierTaskGraph::OnGraphCompleted()
{
	if (CVarOutputGraphViz.GetValueOnAnyThread())
	{
		LogDotGraph();
	}

	if (bIsCancelled)
	{
		return;
	}

	if (bUsingCache)
	{
		if (MeshPartition::FModifierGraphCache* Cache = UMeshPartitionEditorSubsystem::GetGraphCache())
		{
			Cache->CacheBaseGroupViewCacheData(BaseGroupKey, MoveTemp(ActiveGroupCacheData));
		}
	}

	ExecuteEndTime = FPlatformTime::Cycles64();
}

void FModifierTaskGraph::ValidateInstanceOutputsForDeterminism(FModifierInstanceTask& InTask)
{
	const MeshPartition::FModifierGraphCache::FModifierCacheData& ModifierCacheData = ActiveGroupCacheData.ModifierCacheData.FindChecked(Group.GetModifierDesc(InTask.InstanceIndex).ModifierPath);

	const MeshPartition::FMeshView& CachedView = ModifierCacheData.MeshViews[Group.GetInstanceInfo(InTask.InstanceIndex).InstanceID];
	if (!InTask.MeshView.Compare(CachedView))
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Mismatch between cached view and recomputed view for modifier: %ls", *Group.GetModifierOp(InTask.InstanceIndex)->GetOperationName().ToString());
	}

	// Throw away the computed data and use the cached data in the writeback step.
	// We want to make sure that any visual/structural issues caused by the non-determinism are visible.
	InTask.MeshView = CachedView;
	InTask.MeshView.RemapParentMesh(&Mesh);
}

void FModifierTaskGraph::LogDotGraph()
{
	auto AppendNodeID = [&](FStringBuilderBase& Builder, FInstanceIndex Index)
	{
		Builder.Appendf(TEXT("\"%d(%d))\""), GetTypeHash(Group.GetModifierDesc(Index).ModifierPath), Group.GetInstanceInfo(Index).InstanceID);
	};

	TStringBuilder<0> Builder;

	// global graph style:
	Builder.Append(TEXT("digraph G {\n"));
	Builder.Append(TEXT("rankdir=\"LR\"\n"));
	Builder.Append(TEXT("bgcolor=\"#1a1a1a\"\n"));
	for (int32 InstanceID = 0; InstanceID < Tasks.Num(); ++InstanceID)
	{
		const FInstanceIndex InstanceIndex(InstanceID);
		const FInstanceInfo& InstanceDesc = Group.GetInstanceInfo(InstanceIndex);
		const TArray<FInstanceIndex>& Prerequisites = PrerequisiteMap[InstanceID];

		// append the node style:
		AppendNodeID(Builder, InstanceIndex);
		Builder.Append(TEXT("["));
		Builder.Appendf(TEXT("label=\"%s \\n(Instance: %d)\""), *Group.GetModifierOp(InstanceIndex)->GetOperationName().ToString(), InstanceDesc.InstanceID);
		Builder.Append(TEXT("style=\"filled\", "));
		Builder.Append(TEXT("color=\"#C0C0C0\","));

		if (EnumHasAnyFlags(InstanceDesc.WriteViewComponents, EMeshViewComponents::DynamicSubmesh))
		{
			Builder.Append(TEXT("color=\"#26BBFF\","));
		}

		if (!InstancesReusingCachedViews[InstanceID])
		{
			Builder.Append(TEXT("color=\"#FF4040\","));
		}
		// Use a special color to indicate tasks which were indirectly dirtied
		else if (Tasks[InstanceID].bDebugWasCachePoisoned)
		{
			Builder.Append(TEXT("color=\"#A139BF\","));
		}

		Builder.Append(TEXT("];\n"));

		// Append graph edges
		for (FInstanceIndex PrereqIndex : Prerequisites)
		{
			AppendNodeID(Builder, PrereqIndex);
			Builder.Append(TEXT("->"));
			AppendNodeID(Builder, InstanceIndex);
			Builder.Append(TEXT("["));
			Builder.Append(TEXT("penwidth=5., "));
			if (Tasks[PrereqIndex].bUsingCachedView)
			{
				Builder.Append(TEXT("color=\"#1FE44B\","));
			}
			else
			{
				Builder.Append(TEXT("color=\"red\","));
			}
			Builder.Append(TEXT("]"));
			Builder.Append(TEXT(";\n"));
		}
	}
	Builder.Append(TEXT("}\n"));
	UE_LOGF(LogMegaMeshEditor, Log, "\n\n%ls", *Builder);
}


MeshPartition::FBuildPerfStats FModifierTaskGraph::GetBuildPerfStats() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FModifierTaskGraph::GetBuildPerfStats);

	MeshPartition::FBuildPerfStats Stats{};

	if (!bIsCancelled && ExecuteEndTime != 0)
	{
		Stats.WallTime = static_cast<double>(ExecuteEndTime - ExecuteStartTime) * FPlatformTime::GetSecondsPerCycle64();

		for (const FModifierInstanceTask& Task : Tasks)
		{
			MeshPartition::FPerModifierBuildPerfStats& ModifierTiming = Stats.PerModifierTimings.FindOrAdd(Group.GetModifierDesc(Task.InstanceIndex).ModifierPath, {});
			ModifierTiming.AddInstanceStat(static_cast<double>(Task.PerformanceCounter) * FPlatformTime::GetSecondsPerCycle64());
		}
	}

	return Stats;
}
} // namespace UE::MeshPartition