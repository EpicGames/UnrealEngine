// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Task.h"
#include "Tasks/Pipe.h"
#include "MeshPartitionMeshView.h"
#include "SharedPipe.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionModifierGraphCache.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionBuildPerfStats.h"

class UTexture;

namespace UE::MeshPartition
{
class UMeshPartitionDefinition;

/**
* MegaMeshModifierTaskGraph builds a dependency graph to applying mega mesh modifiers in parallel and async.
* Overlapping modifiers apply in a region serially while respecting their relative type and sub priorities.
*/
class FModifierTaskGraph
{
public:
	FModifierTaskGraph();
	~FModifierTaskGraph();

	/**
	* Builds and launches the task graph from a given set of modifiers and priorities.
	* @param InBaseMesh Mesh to use as the base for modifier applications. Data is moved out of this parameter.
	* @param InBaseGroupCacheKey the cache key of all base modifiers
	* @param InModifiers the list of modifiers to apply to the `InBaseMesh`.
	* @param InTransform Worldspace transform for the final mesh.
	* @param InModifierTypePriorities The ordered list of modifier type priorities. If a modifier type can't be found in this list, it will be given the lowest priority.
	* @param bInUseCache If false, will force the cache to be disabled.
	* @return returns a task that is signalled when the graph execution completes.
	*/
	Tasks::FTask Execute(FMeshData&& InBaseMesh, const FGuid& InBaseGroupCacheKey, MeshPartition::FModifierGroup&& ModifierGroup, const FTransform& InTransform, TArrayView<const FName> InModifierTypePriorities, const bool bInUseCache = true);

	bool IsComplete() const;

	void WaitForCompletion();

	FMeshData& GetResultMesh() { return Mesh; }
	
	MeshPartition::FBuildPerfStats GetBuildPerfStats() const;

	void Cancel();

	bool IsCanceled() const { return bIsCancelled; }

private:
	enum class ETaskState
	{
		Invalid,
		/** Task is waiting for dependencies to complete execution. */
		Waiting,
		/** Task is active and writing back to the mesh. */
		Active,
		Completed
	};

	struct FModifierInstanceTask
	{
		ETaskState State = ETaskState::Invalid;

		FBox WritebackBounds;

		Tasks::FTaskEvent TriggerEvent;
		Tasks::FTask BuildViewTask;
		Tasks::FTask ComputeTask;
		Tasks::FTask WriteTask;
		Tasks::FTask CompleteTask;

		MeshPartition::FMeshView MeshView;
		FInstanceIndex InstanceIndex;

		uint64 PerformanceCounter = 0;

		/** Determines if, at view build time, this task can reuse the view stored in the cache. */
		bool bUsingCachedView = false;

		/** Track if the task's cache entry was explicitly dirtied by a modifier invalidating the work-region (for debug purposes only). */
		bool bDebugWasCachePoisoned = false;
	};

	void BuildDependencyGraph(TConstArrayView<FName> InModifierTypePriorities);

	void PrepareViewCache();

	void ApplyTaskWriteback(FModifierInstanceTask& InTask);

	void OnTaskCompleted(int TaskIndex);

	/** Called asynchronously when the full graph has completed execution */
	void OnGraphCompleted();

private:
	void LogDotGraph();

	void ValidateInstanceOutputsForDeterminism(FModifierInstanceTask& InTask);

private:
	/** Hash of the guids for all the bases running through the graph */
	FGuid BaseGroupKey;

	MeshPartition::FModifierGroup Group;

	/** Local working copy of cache data for the base group currently being operated on */
	MeshPartition::FModifierGraphCache::FBaseGroupViewCacheData ActiveGroupCacheData;

	/** Complete list of all the tasks, should never be modified during task execution. */
	TArray<FModifierInstanceTask> Tasks;
	
	/** Maps a task id to the list of task ids which must complete before the given task can start. */
	TArray<TArray<FInstanceIndex>> PrerequisiteMap;

	/** Maps a task id to the list of task ids which are downstream of the task index. */
	TArray<TArray<FInstanceIndex>> DownstreamMap;

	/** Cache key of the local view for each modifier instance */
	TArray<FGuid> InstanceViewCacheKeys;

	/** Maps the modifier instance index to a flag indicating if it can reuse its cached data or not. */
	TArray<bool> InstancesReusingCachedViews;

	/** The mesh to which all the modifications are applied */
	FMeshData Mesh;

	Tasks::FTask GraphCompleteTask;

	/** Ensures writebacks to the source mesh which need to be exclusive are exclusive, but shared writes and reads can still occur in parallel to each other. */
	FSharedPipe MeshPipe;

	/** Track the execution start time and end time to compute wall clock time */
	uint64 ExecuteStartTime = 0;
	uint64 ExecuteEndTime = 0;

	/**
	* Determines if this graph is using the cache.
	* This value is set at the moment Execute is called and latched in order to prevent
	* potential race condition if the user sets the flag while the graph is executing asynchronously.
	* It would be bad if the graph starts with caching disabled and then it were suddenly enabled before the graph completed.
	*/
	bool bUsingCache = false;

	bool bIsCancelled = false;

	UE_MT_DECLARE_RW_ACCESS_DETECTOR(MeshAccessDetector);
};
} // namespace UE::MeshPartition