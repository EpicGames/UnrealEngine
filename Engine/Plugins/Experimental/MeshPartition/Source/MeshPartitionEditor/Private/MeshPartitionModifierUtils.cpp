// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModifierUtils.h"
#include "Components/SplineComponent.h"
#include "Curves/RichCurve.h"
#include "MeshPartitionModifierComponent.h"
#include "UDynamicMesh.h"
#include "Engine/StaticMesh.h"
#include "Engine/Texture.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionActorDescUtils.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionActorDescUtils.h"
#include "MeshPartitionModifierComponentDesc.h"

namespace UE::MeshPartition::Utils
{
	TArray<FBox> CollectBoundingBoxesForSpline(
		const USplineComponent* InSplineComp,
		TFunctionRef<float(float)> GetWidthForDistanceAlongSpline,
		TFunctionRef<float(float)> GetHeightForDistanceAlongSpline,
		float InMaxSquaredDistance
	)
	{
		check(InSplineComp);

		TArray<FBox> Boxes;

		TArray<double> Distances;
		TArray<FVector> Points;
		InSplineComp->ConvertSplineToPolyLineWithDistances(ESplineCoordinateSpace::World, InMaxSquaredDistance, Points, Distances);
		
		check(Points.Num() >= 2);
		Boxes.Reserve(Points.Num() - 1);

		auto AddExtremaForPoint = [&](FBox& OutBounds, int32 PointIndex)
		{
			const float DistanceAlongSpline = Distances[PointIndex];

			const FVector Position = Points[PointIndex];

			const FVector Right = InSplineComp->GetRightVectorAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::World);
			const FVector Up = InSplineComp->GetUpVectorAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::World);
				
			const float HalfWidth = GetWidthForDistanceAlongSpline(DistanceAlongSpline);
			const float Height = GetHeightForDistanceAlongSpline(DistanceAlongSpline);

			OutBounds += Position + Right * HalfWidth;
			OutBounds += Position + Up * Height;
			OutBounds += Position - Right * HalfWidth;
			OutBounds += Position - Up * Height;
		};
		
		enum ETerminalPoint : int32
		{
			First = -1,
			Last = 1,
		};
		auto AddTerminalPoint = [&](FBox& OutBounds, int32 PointIndex, ETerminalPoint Terminal)
		{
			const float DistanceAlongSpline = Distances[PointIndex];
			const FVector Position = Points[PointIndex];
			const FVector Normal = InSplineComp->GetDirectionAtDistanceAlongSpline(DistanceAlongSpline, ESplineCoordinateSpace::World);

			OutBounds += Position + (int32)(Terminal) * Normal * GetWidthForDistanceAlongSpline(DistanceAlongSpline);
		};

		for (int32 Index = 0; Index < Points.Num() - 1; ++Index)
		{
			FBox SegmentBounds(ForceInit);

			AddExtremaForPoint(SegmentBounds, Index);
			AddExtremaForPoint(SegmentBounds, Index + 1);

			// First and last points need to be expanded by along the normal vector as well:
			if (Index == 0)
			{
				AddTerminalPoint(SegmentBounds, Index, ETerminalPoint::First);
			}
			else if (Index == Points.Num() - 2)
			{
				AddTerminalPoint(SegmentBounds, Index + 1, ETerminalPoint::Last);
			}

			Boxes.Add(SegmentBounds);
		}

		return Boxes;
	}

	FRectangleFalloffData::FRectangleFalloffData(const FVector2d& RectangleExtentIn, double CornerRadiusIn,
		double FalloffDistanceIn, ERectangleFalloffMode FalloffModeIn, TSharedPtr<const FRichCurve> FalloffCurveIn)
		: RectangleExtent(RectangleExtentIn)
		, FalloffDistance(FalloffDistanceIn)
		, FalloffCurve(FalloffCurveIn)
	{
		double ExtentMin = RectangleExtent.GetMin();
		ClampedRadius = FMath::Min(CornerRadiusIn, ExtentMin);
		ClampedFalloff = FMath::Min(FalloffDistance, ExtentMin);
		CornerCenter = RectangleExtent - ClampedRadius;
		// Switch falloff mode to smooth if we are set to curve but don't have curve data
		FalloffMode = (FalloffModeIn == ERectangleFalloffMode::CustomCurve && !FalloffCurve) ?
			ERectangleFalloffMode::Smooth : FalloffModeIn;
	}

	double GetRectangleFalloffAlpha(const FVector2d Local2DCoordinates, const FRectangleFalloffData& FalloffData)
	{
		if (FalloffData.FalloffDistance <= 0 && FalloffData.ClampedRadius <= 0)
		{
			return 1.0;
		}

		double DistanceFromEdge = -1;
		FVector2D Abs2DCoordinates(FMath::Abs(Local2DCoordinates.X), FMath::Abs(Local2DCoordinates.Y));

		// See if we're in the portion where we have to worry about the rounded corners
		if (FalloffData.ClampedRadius > 0
			&& Abs2DCoordinates.X > FalloffData.CornerCenter.X
			&& Abs2DCoordinates.Y > FalloffData.CornerCenter.Y)
		{
			DistanceFromEdge = FalloffData.ClampedRadius - (Abs2DCoordinates - FalloffData.CornerCenter).Length();
			if (DistanceFromEdge < 0)
			{
				// This means we're outside the corner radius, where we should not apply at all.
				return 0.0;
			}
		}

		if (FalloffData.FalloffDistance <= 0)
		{
			// Don't have regular falloff to apply, and already checked corner containment
			return 1.0;
		}

		if (DistanceFromEdge < 0) // If we didn't already get a distance from the corner check
		{
			DistanceFromEdge = (FalloffData.RectangleExtent - Abs2DCoordinates).GetMin();
		}

		// Should be the case because we shouldn't be dealing with points outside our extent, and
		//  we already have an early out above if we're outside the rounded corners or there is
		//  no falloff.
		checkSlow(DistanceFromEdge >= 0);

		// We handle the case of falloff being larger than our extents. For linear falloff we could just
		//  divide distance from edge by unclamped falloff distance and call it a day, because the center
		//  will be appropriately lowered (imagine a truncated pyramid's sides becoming less steep until
		//  we have a pyramid shrinking in height).
		// However this isn't good enough for any other falloff function, where we lose the smoothness of
		//  the end by doing it this way (imagine a sigmoid being cut in the middle). Instead, we divide
		//  distance from edge by clamped falloff distance, and then multiply by a factor by which a linear
		//  falloff would have decreased at the innermost end of the falloff (to lower our "pyramid").
		double FalloffMultiplier = FalloffData.ClampedFalloff / FalloffData.FalloffDistance;

		if (DistanceFromEdge >= FalloffData.ClampedFalloff)
		{
			// We're in the region that is "full" value
			return FalloffMultiplier;
		}

		double FalloffInput = DistanceFromEdge / FalloffData.ClampedFalloff;

		switch (FalloffData.FalloffMode)
		{
		case ERectangleFalloffMode::Linear:
			return FalloffMultiplier * FalloffInput;
		case ERectangleFalloffMode::Smooth:
			return FalloffMultiplier * FMath::SmoothStep(0.0, 1.0, FalloffInput);
		case ERectangleFalloffMode::CustomCurve:
			return FalloffMultiplier * FalloffData.FalloffCurve->Eval(FalloffInput);
		}
		ensure(false);
		return 1.0;
	};


	FGuid BetterHashCombine(const FGuid& GuidA, const FGuid& GuidB)
	{
		return FGuid(
			HashCombine(GuidA.A, GuidB.B),
			HashCombine(GuidA.B, GuidB.C),
			HashCombine(GuidA.C, GuidB.D),
			HashCombine(GuidA.D, GuidB.A)
		);
	}

	void FHashArchive::operator += (const UDynamicMesh& Mesh)
	{
		// this serializzation is read only, so should be ok to const cast it
		const_cast<UDynamicMesh*>(&Mesh)->Serialize(*this);
	}

	void FHashArchive::operator += (const UDynamicMesh* Mesh)
	{
		if (Mesh != nullptr)
		{
			*this += *Mesh;
		}
		else
		{
			FGuid ZeroId;
			*this << ZeroId;
		}
	}

	void FHashArchive::operator += (const UStaticMesh& Mesh)
	{
		// we use the lighting guid as a fast alternative to hashing the mesh contents
		*this += Mesh.GetLightingGuid();
	}

	void FHashArchive::operator += (const UStaticMesh* Mesh)
	{
		if (Mesh != nullptr)
		{
			*this += *Mesh;
		}
		else
		{
			FGuid ZeroId;
			*this << ZeroId;
		}
	}

	void FHashArchive::operator += (const UTexture& Texture)
	{
		if (Texture.Source.IsValid())
		{
			FGuid SourceId = Texture.Source.GetId();
			*this << SourceId;
		}
		else
		{
			// LightingGuid is a fallback, less stable as it's just a random GUID that's regenerated on change
			// (but still mostly stable as long as the texture does not change)
			FGuid LightingGuid = Texture.GetLightingGuid();
			*this << LightingGuid;
		}
	}

	void FHashArchive::operator += (const UTexture* Texture)
	{
		// prefer Source ID if we have it, as it is a content hash
		if (Texture != nullptr)
		{
			*this += *Texture;
		}
		else
		{
			FGuid ZeroId;
			*this << ZeroId;
		}
	}

	void ForEachMegaMeshDescInWorldPartition(UWorldPartition* InWorldPartition, TFunctionRef<void(MeshPartition::FModifierDesc& Descriptor)> PerModifierCallback, TFunctionRef<void(FCompiledSectionDescriptor& Descriptor)> PerSectionCallback)
	{
		// NOTE: this only grabs actor descs from the Main world/level -- not from any level instances
		FWorldPartitionHelpers::ForEachActorDescInstance(InWorldPartition,
			[PerModifierCallback, PerSectionCallback](const FWorldPartitionActorDescInstance* ActorDescInstance)
			{
				ForEachModifierDescInActorDesc(ActorDescInstance, PerModifierCallback);

				if (ActorDescInstance->GetActorNativeClass() == MeshPartition::ACompiledSection::StaticClass())
				{
					// this is a compiled section actor, gather its build info
					FCompiledSectionDescriptor CompiledSectionDescriptor;
					if (FCompiledSectionDescriptor::BuildFromActorDescInstance(*ActorDescInstance, CompiledSectionDescriptor))
					{
						PerSectionCallback(CompiledSectionDescriptor);
					}
				}

				return true;
			});
	}

	void ForEachModifierDescInActorDesc(
		const FWorldPartitionActorDescInstance* ActorDescInstance,
		TFunctionRef<void(MeshPartition::FModifierDesc& Descriptor)> PerModifierCallback)
	{
		// Old path: any actors serialized before the component descriptors change will take this path using actor desc property pair map
		// #todo: remove once all modifiers in older maps have been resaved.
		const int32 NumModifiers = ActorDesc::GetPropertyInt32(*ActorDescInstance, MegaMeshModifierProperties::MegaMeshModifiersNum, INDEX_NONE);
		if (NumModifiers != INDEX_NONE)
		{
			// this actor has megamesh modifiers, build their descriptors
			for (int32 ModifierIndex = 0; ModifierIndex < NumModifiers; ++ModifierIndex)
			{
				MeshPartition::FModifierDesc ModifierDesc(*ActorDescInstance, ModifierIndex);
				if (ModifierDesc.IsValid())
				{
					PerModifierCallback(ModifierDesc);
				}
			}
		}
		else
		{
			const FWorldPartitionActorDesc* ActorDesc = ActorDescInstance->GetActorDesc();
			for (const TUniquePtr<FWorldPartitionComponentDesc>& ComponentDesc : ActorDesc->GetComponentDescs())
			{
				ensure(ComponentDesc.IsValid());

				UClass* ComponentDescClass = ComponentDesc->GetComponentNativeClass();
				if (ComponentDescClass->IsChildOf<MeshPartition::UModifierComponent>())
				{
					const MeshPartition::FWorldPartitionModifierComponentDesc* ModifierComponentDesc = static_cast<const MeshPartition::FWorldPartitionModifierComponentDesc*>(ComponentDesc.Get());
					MeshPartition::FModifierDesc ModifierDesc(*ActorDescInstance, *ModifierComponentDesc);
					if (ModifierDesc.IsValid())
					{
						PerModifierCallback(ModifierDesc);
					}
				}
			}
		}

	}

	bool IsNonzeroWeightLayer(const UE::Geometry::FDynamicMesh3& Mesh, int32 LayerIndex)
	{
		using namespace UE::Geometry;
		if (LayerIndex < 0)
		{
			return false;
		}
		const FDynamicMeshAttributeSet* Attributes = Mesh.Attributes();
		if (!Attributes || LayerIndex >= Attributes->NumWeightLayers())
		{
			return false;
		}
		const FDynamicMeshWeightAttribute* WeightLayer = Attributes->GetWeightLayer(LayerIndex);
		if (!ensure(WeightLayer))
		{
			return false;
		}

		for (int32 Vid : Mesh.VertexIndicesItr())
		{
			float Weight;
			WeightLayer->GetValue(Vid, &Weight);
			if (Weight != 0)
			{
				return true;
			}
		}
		return false;
	}
}