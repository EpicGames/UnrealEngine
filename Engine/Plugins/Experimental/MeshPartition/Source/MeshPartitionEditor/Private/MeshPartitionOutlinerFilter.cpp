// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionOutlinerFilter.h"

#include "ActorTreeItem.h"
#include "ComponentTreeItem.h"
#include "ActorDescTreeItem.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "MeshPartition.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionInteractiveSection.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionModifierComponentDesc.h"

#define LOCTEXT_NAMESPACE "MegaMeshOutlinerFilter"

namespace UE::MeshPartition
{
FMeshPartitionOutlinerFilter::FMeshPartitionOutlinerFilter(TSharedPtr<FFilterCategory> InCategory)
	: FGenericFilter<const ISceneOutlinerTreeItem&>(InCategory, TEXT("MeshPartition"), LOCTEXT("MeshPartitionModifiers", "Mesh Partition"), FOnItemFiltered())
{
	SetToolTipText(LOCTEXT("MeshPartition_OutlinerFilter_ToolTip", "Only show actors which are relevant to the Mesh Partition system."));
}

bool FMeshPartitionOutlinerFilter::PassesFilter(const ISceneOutlinerTreeItem& InItem) const
{
	if (const FActorTreeItem* ActorItem = InItem.CastTo<FActorTreeItem>())
	{
		if (const AActor* Actor = ActorItem->Actor.Get())
		{
			const bool bHasModifier = Actor->GetComponentByClass<MeshPartition::UModifierComponent>() != nullptr;
			const bool bIsMeshPartitionActor = Actor->IsA<AMeshPartition>();
			const bool bIsABuiltMeshPartitionSection = Actor->IsA<MeshPartition::APreviewSection>() || Actor->IsA<MeshPartition::ACompiledSection>();
			return bHasModifier || bIsMeshPartitionActor || bIsABuiltMeshPartitionSection;
		}
	}
	else if (const FComponentTreeItem* ComponentItem = InItem.CastTo<FComponentTreeItem>())
	{
		if (const MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(ComponentItem->Component.Get()))
		{
			return true;
		}
		return false;
	}
	else if (const FActorDescTreeItem* ActorDescItem = InItem.CastTo<FActorDescTreeItem>())
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = ActorDescItem->ActorDescHandle.GetInstance())
		{
			// don't filter out compiled sections
			if (ActorDescInstance->GetActorNativeClass()->IsChildOf<ACompiledSection>())
			{
				return true;
			}

			// don't filter out any actors with modifier components
			// check all the unloaded actor's components to see if it has any modifier components.
			const FWorldPartitionActorDesc* ActorDesc = ActorDescInstance->GetActorDesc();
			for (const TUniquePtr<FWorldPartitionComponentDesc>& ComponentDesc : ActorDesc->GetComponentDescs())
			{
				if (!ensure(ComponentDesc.IsValid()))
				{
					continue;
				}

				UClass* ComponentDescClass = ComponentDesc->GetComponentNativeClass();
				if (ComponentDescClass->IsChildOf<MeshPartition::UModifierComponent>())
				{
					return true;
				}
			}
		}
		return false;
	}
	return true;
}


FBaseModifierFilter::FBaseModifierFilter(TSharedPtr<FFilterCategory> InCategory)
	: FGenericFilter<const ISceneOutlinerTreeItem&>(InCategory, TEXT("MeshPartition"), LOCTEXT("MeshPartitionBases", "Show Mesh Partition Bases"), FOnItemFiltered())
{
	SetToolTipText(LOCTEXT("MeshPartition_BaseModifierOutlinerFilter_ToolTip", "Show all actors with base mesh partition modifiers"));
}

bool FBaseModifierFilter::PassesFilter(const ISceneOutlinerTreeItem& InItem) const
{
	if (const FActorTreeItem* ActorItem = InItem.CastTo<FActorTreeItem>())
	{
		if (const AActor* Actor = ActorItem->Actor.Get())
		{
			int32 NumBases = 0;
			TInlineComponentArray<const MeshPartition::UModifierComponent*> Modifiers(Actor);
			for (const MeshPartition::UModifierComponent* Modifier : Modifiers)
			{
				NumBases += static_cast<int32>(Modifier->IsBase());
			}
			// show the actor if it has no modifiers or if not all the modifiers are bases.
			return Modifiers.Num() == 0 || (NumBases != Modifiers.Num());
		}
	}
	else if (const FComponentTreeItem* ComponentItem = InItem.CastTo<FComponentTreeItem>())
	{
		if (const MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(ComponentItem->Component.Get()))
		{
			if (Modifier->IsBase())
			{
				return false;
			}
		}
		return true;
	}
	else if (const FActorDescTreeItem* ActorDescItem = InItem.CastTo<FActorDescTreeItem>())
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = ActorDescItem->ActorDescHandle.GetInstance())
		{
			const FWorldPartitionActorDesc* ActorDesc = ActorDescInstance->GetActorDesc();
			int32 NumModifiers = 0;
			int32 NumBases = 0;
			for (const TUniquePtr<FWorldPartitionComponentDesc>& ComponentDesc : ActorDesc->GetComponentDescs())
			{
				if (!ensure(ComponentDesc.IsValid()))
				{
					continue;
				}

				UClass* ComponentDescClass = ComponentDesc->GetComponentNativeClass();
				if (ComponentDescClass->IsChildOf<MeshPartition::UModifierComponent>())
				{
					NumModifiers++;
					const MeshPartition::FWorldPartitionModifierComponentDesc* ModifierComponentDesc = static_cast<const MeshPartition::FWorldPartitionModifierComponentDesc*>(ComponentDesc.Get());
					const MeshPartition::FModifierDesc ModifierDesc(*ActorDescInstance, *ModifierComponentDesc);

					NumBases += static_cast<int32>(ModifierDesc.IsBase());
				}
			}
			return NumModifiers == 0 || (NumBases != NumModifiers);
		}
		return false;
	}
	return true;
}

FBuiltSectionFilter::FBuiltSectionFilter(TSharedPtr<FFilterCategory> InCategory)
	: FGenericFilter<const ISceneOutlinerTreeItem&>(InCategory, TEXT("MeshPartition"), LOCTEXT("MeshPartitionSections", "Show Built Mesh Partition Sections"), FOnItemFiltered())
{
	SetToolTipText(LOCTEXT("MeshPartition_BuiltSectionOutlinerFilter_ToolTip", "Show all built Mesh Partition sections"));
}

bool FBuiltSectionFilter::PassesFilter(const ISceneOutlinerTreeItem& InItem) const
{
	if (const FActorTreeItem* ActorItem = InItem.CastTo<FActorTreeItem>())
	{
		if (const AActor* Actor = ActorItem->Actor.Get())
		{
			return !Actor->IsA<MeshPartition::APreviewSection>() && !Actor->IsA<MeshPartition::ACompiledSection>() && !Actor->IsA<MeshPartition::AInteractiveSection>();
		}
	}
	else if (const FActorDescTreeItem* ActorDescItem = InItem.CastTo<FActorDescTreeItem>())
	{
		if (const FWorldPartitionActorDescInstance* ActorDescInstance = ActorDescItem->ActorDescHandle.GetInstance())
		{
			UClass* NativeClass = ActorDescInstance->GetActorNativeClass();
			if (ensure(NativeClass))
			{
				return !NativeClass->IsChildOf<MeshPartition::ACompiledSection>();
			}
		}
	}

	return true;
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE

