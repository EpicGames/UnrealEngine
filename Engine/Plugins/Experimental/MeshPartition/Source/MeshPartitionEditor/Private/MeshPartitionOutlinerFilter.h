// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerStandaloneTypes.h"
#include "Filters/GenericFilter.h"

namespace UE::MeshPartition
{
class FMeshPartitionOutlinerFilter : public FGenericFilter<const ISceneOutlinerTreeItem&>
{
public:
	FMeshPartitionOutlinerFilter(TSharedPtr<FFilterCategory> InCategory);

	virtual bool PassesFilter(const ISceneOutlinerTreeItem& InItem) const override;
};

class FBaseModifierFilter : public FGenericFilter<const ISceneOutlinerTreeItem&>
{
public:
	FBaseModifierFilter(TSharedPtr<FFilterCategory> InCategory);

	virtual bool PassesFilter(const ISceneOutlinerTreeItem& InItem) const override;
	virtual bool IsInverseFilter() const override { return true; }
};

class FBuiltSectionFilter : public FGenericFilter<const ISceneOutlinerTreeItem&>
{
public:
	FBuiltSectionFilter(TSharedPtr<FFilterCategory> InCategory);

	virtual bool PassesFilter(const ISceneOutlinerTreeItem& InItem) const override;
	virtual bool IsInverseFilter() const override { return true; }
};
} // namespace UE::MeshPartition
