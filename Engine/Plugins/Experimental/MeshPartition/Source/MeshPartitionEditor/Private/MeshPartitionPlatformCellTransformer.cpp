// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPlatformCellTransformer.h"

#include "Interfaces/ITargetPlatform.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"

#include "MeshPartitionDataLayerContainer.h"
#include "MeshPartitionEditorUtils.h"

namespace UE::MeshPartition
{

bool UPlatformCellTransformer::ShouldStripRuntimeCell(const UWorldPartitionRuntimeCell* InCell) const
{
	// A cell carrying MeshPartition-owned data layers represents one build variant of compiled sections
	// (potentially platform-specific LODs, Nanite toggles, server-only vs rendering-only, etc.).
	// If none of those data layers are relevant to the current cook target, the entire cell — including
	// any derived HLODs — must be dropped so it doesn't end up in the packaged data or pollute streaming.
	// If the cell has no MeshPartition data layers it is not our concern and we always retain.
	// #TODO [roey]: How to handle cooks with -TargetPlatform=WindowsServer+Linux+MacOS?

	const TArray<const UDataLayerInstance*> CellDataLayers = InCell->GetDataLayerInstances();
	const TArray<ITargetPlatform*> TargetPlatforms = UE::MeshPartition::EditorUtils::GetTargetPlatforms();

	bool bHasPlatformDataLayers = false;
	bool bHasRelevantPlatformDataLayer = false;

	for (const UDataLayerInstance* Instance : CellDataLayers)
	{
		const UDataLayerAsset* Asset = Instance->GetAsset();
		if (!Asset)
		{
			continue;
		}

		const MeshPartition::AMeshPartitionDataLayerContainer* Container = Asset->GetTypedOuter<MeshPartition::AMeshPartitionDataLayerContainer>();
		if (!Container || !Container->IsDataLayerOwnedByContainer(Asset))
		{
			continue;
		}

		bHasPlatformDataLayers = true;
		for (ITargetPlatform* TargetPlatform : TargetPlatforms)
		{
			if (Container->IsDataLayerRelevantForPlatform(Asset, TargetPlatform))
			{
				bHasRelevantPlatformDataLayer = true;
				break;
			}
		}

		if (bHasRelevantPlatformDataLayer)
		{
			break;
		}
	}

	const bool bRetain = !bHasPlatformDataLayers || bHasRelevantPlatformDataLayer;
	return !bRetain;
}

void UPlatformCellTransformer::TransformRuntimeCell(UWorldPartitionRuntimeCell* InCell)
{
	// Whenever a build variant set of CompiledSections are generated, they are assigned a unique data layer each.
	// A "build variant" is a set of compiled sections which can be enabled or disabled for specific target platforms.
	// sections per build variant may also have completely different bounds and generate completely different numbers of compiled section actors.
	// (you might want compiled sections with nanite enabled for PC and a set with nanite disabled and traditional LODs for switch).
	// (for servers we have a "Common" build variant which contains only the runtime gameplay data and rendering is split off into their own variants)
	//
	// This data layer is used to allow us to separate the compiled sections (and derived HLODs) into their own streaming cells. This prevents all compiled
	// sections across build variants being included in a single HLOD and it prevents the larger high end platform actor sizes from polluting the streaming
	// grids of lower end platforms.
	//
	// During TransformRuntimeCell we strip these data layers so that we can achieve normal data layer loading properties for compiled sections.
	// This is necessary to solve the following cases:
	// 1. User assigned a "classic" runtime data layer to their compiled section actors and triggered it to load.
	//		this would enable loading of _all_ mesh partition build variants due to the OR nature of DLs.
	// 2. User assigned a "classic" runtime data layer to their compiled section actors and expects it to remain unloaded (Eg. AncientGame nighttime scene)
	//		When the mesh partition system tries to load the active platform data layer it would unintentionally load data which was intended to be unloaded.

	TArray<const UDataLayerInstance*> CellDataLayers = InCell->GetDataLayerInstances();

	for (auto It = CellDataLayers.CreateIterator(); It; ++It)
	{
		const UDataLayerInstance* Instance = *It;
		if (const UDataLayerAsset* Asset = Instance->GetAsset())
		{
			if (const MeshPartition::AMeshPartitionDataLayerContainer* Container = Asset->GetTypedOuter<MeshPartition::AMeshPartitionDataLayerContainer>())
			{
				if (Container->IsDataLayerOwnedByContainer(Asset))
				{
					It.RemoveCurrentSwap();
				}
			}
		}
	}

	InCell->SetDataLayers(CellDataLayers);
}

}
