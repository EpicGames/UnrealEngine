// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPreviewComponents.h"
#include "MeshPartitionPreviewSceneProxy.h"

#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "MeshPartitionMaterialCacheCommon.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionEditorModule.h"
#include "Engine/CollisionProfile.h"
#include "ProfilingDebugging/ScopedTimers.h"

namespace UE::MeshPartition
{
static TAutoConsoleVariable<bool> CVarCustomMeshGeneratePhysics(TEXT("MegaMesh.Preview.CustomPreviewMeshGeneratePhysics"),
	true,
	TEXT("Generate physics data for Mesh Terrain preview meshes, to aid editor interaction."));

static TAutoConsoleVariable<bool> CVarCustomMeshAsyncPhysics(TEXT("MegaMesh.Preview.CustomPreviewMeshAsyncPhysics"),
	true, 
	TEXT("Allow async generation of physics data for Mesh Terrain preview meshes."));
	
static TAutoConsoleVariable<int32> CVarPreviewMaterialCacheTileCount(TEXT("MegaMesh.Preview.MaterialCache.TileCountXY"),
	512,
	TEXT("The number of tiles (x/y) allocated for material cache textures for preview meshes"));

HHitProxy* UStaticMeshPreviewComponent::CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) const
{
	return new MeshPartition::HPreviewProxy(this);
}

UPreviewMeshComponent::UPreviewMeshComponent()
{
	MeshData = MakeShared<const FMeshData>();
}

void UPreviewMeshComponent::GetResourceSizeEx(FResourceSizeEx& Size)
{
	Super::GetResourceSizeEx(Size);

	Size.AddDedicatedSystemMemoryBytes(MeshData ? MeshData->GetByteCount() : 0);
}

void UPreviewMeshComponent::SetMeshData(TSharedRef<const MeshPartition::FMeshData> InMeshData)
{
	MeshData = InMeshData;
	
	MarkRenderStateDirty();
	UpdateBounds();
	{
		double Time = 0.0;
		FDurationTimer Timer(Time);
		Timer.Start();

		RebuildPhysicsData();

		Timer.Stop();
		UE_LOGF(LogMegaMeshEditor, Verbose, "UPreviewMeshComponent::SetMeshData created physics state in %.2f seconds", Time);
	}
}

void UPreviewMeshComponent::SetMeshData(FMeshData&& InMeshData)
{
	SetMeshData(MakeShared<const FMeshData>(MoveTemp(InMeshData)));
}

FBoxSphereBounds UPreviewMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBox Result;

	for (int VertexID : MeshData->VertexIndicesItr())
	{
		Result += MeshData->GetVertex(VertexID);
	}
	if (!Result.IsValid)
	{
		// If bbox is empty, set a very small bbox to avoid log spam/etc in other engine systems.
		// The check used is generally IsNearlyZero(), which defaults to KINDA_SMALL_NUMBER, so set 
		// a slightly larger box here to be above that threshold
		constexpr double Extent = (double)(KINDA_SMALL_NUMBER + SMALL_NUMBER);
		Result = FBox(FVector(-Extent), FVector(Extent));
	}
	return FBoxSphereBounds(Result).TransformBy(LocalToWorld);
}

void UPreviewMeshComponent::OnRegister()
{
	Super::OnRegister();
}

void UPreviewMeshComponent::OnUnregister()
{
	Super::OnUnregister();

	for (TObjectPtr VirtualTexture : MaterialCacheTextures)
	{
		VirtualTexture->Unregister();
		VirtualTexture->ReleaseResource();
	}

	MaterialCacheTextures.Empty();

	AbortPendingPhysicsCook();
}

void UPreviewMeshComponent::RebuildPhysicsData()
{
	if (AsyncBodySetup)
	{
		AsyncBodySetup->AbortPhysicsMeshAsyncCreation();
		AsyncBodySetup = nullptr;
	}

	if (!CVarCustomMeshAsyncPhysics.GetValueOnGameThread() || !CVarCustomMeshGeneratePhysics.GetValueOnGameThread() ||
		!MeshData.IsValid() || MeshData->VertexCount() <= 0)
	{
		MeshBodySetup = nullptr;
		// Generate synchronously
		RecreatePhysicsState();
		return;
	}

	AsyncBodySetup = CreateBodySetupHelper();
	UE_LOGF(LogMegaMeshEditor, Verbose, "UPreviewMeshComponent::RebuildPhysicsData enqueuing async physics generation.  guid=%ls", *AsyncBodySetup->BodySetupGuid.ToString());

	AsyncBodySetup->CreatePhysicsMeshesAsync(FOnAsyncPhysicsCookFinished::CreateUObject(this, &UPreviewMeshComponent::FinishPhysicsAsyncCook, AsyncBodySetup.Get()));
}

void UPreviewMeshComponent::FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup)
{
	if (FinishedBodySetup != AsyncBodySetup)
	{
		UE_LOGF(LogMegaMeshEditor, Display, "UPreviewMeshComponent::FinishPhysicsAsyncCook discarding cancelled UBodySetup generation.  guid=%ls", *FinishedBodySetup->BodySetupGuid.ToString());
		return;
	}

	// Received the UBodySetup containing the cooked physics data.  Apply it.
	UE_LOGF(LogMegaMeshEditor, Verbose, "UPreviewMeshComponent::FinishPhysicsAsyncCook applying new UBodySetup.  guid=%ls", *FinishedBodySetup->BodySetupGuid.ToString());
	MeshBodySetup = FinishedBodySetup;
	AsyncBodySetup = nullptr;

	RecreatePhysicsState();
}

void UPreviewMeshComponent::AbortPendingPhysicsCook()
{
	if (AsyncBodySetup)
	{
		UE_LOGF(LogMegaMeshEditor, Display, "UPreviewMeshComponent::AbortPendingPhysicsCook aborting stale UBodySetup generation.  guid=%ls", *AsyncBodySetup->BodySetupGuid.ToString());
		AsyncBodySetup->AbortPhysicsMeshAsyncCreation();
		AsyncBodySetup = nullptr;
	}
}

UBodySetup* UPreviewMeshComponent::GetBodySetup()
{
	if (!MeshBodySetup)
	{
		MeshBodySetup = CreateBodySetupHelper();
	}

	return MeshBodySetup;
}

UBodySetup* UPreviewMeshComponent::CreateBodySetupHelper()
{
	UBodySetup* NewBodySetup = NewObject<UBodySetup>(this, NAME_None, RF_NoFlags);
	NewBodySetup->BodySetupGuid = FGuid::NewGuid();

	NewBodySetup->bGenerateMirroredCollision = false;
	NewBodySetup->bDoubleSidedGeometry = true;
	NewBodySetup->CollisionTraceFlag = CTF_UseComplexAsSimple;

	NewBodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	NewBodySetup->bSupportUVsAndFaceRemap = false;

	return NewBodySetup;
}

bool UPreviewMeshComponent::GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData)
{
	if (MeshData.IsValid() && CVarCustomMeshGeneratePhysics.GetValueOnGameThread())
	{
		return MeshData->ConvertToTriMeshCollisionData(CollisionData);
	}
	return false; 
}

bool UPreviewMeshComponent::ContainsPhysicsTriMeshData(bool /*InUseAllTriData*/) const
{
	return CVarCustomMeshGeneratePhysics.GetValueOnGameThread() && MeshData.IsValid() && MeshData->VertexCount() > 0;
}

bool UPreviewMeshComponent::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool /*InUseAllTriData*/) const
{
	if (MeshData.IsValid())
	{
		OutTriMeshEstimates.VerticeCount = MeshData->VertexCount();
	}
	else
	{
		OutTriMeshEstimates.VerticeCount = 0;
	}
	return true;
}

void UPreviewMeshComponent::GetMeshId(FString& OutMeshId)
{
	// This is the opportunity to mark the physics inputs with an ID for ddc to cache the physics body.
}

UMaterialCacheVirtualTexture* GetMaterialCacheVirtualTexture(MeshPartition::UPreviewMeshComponent* Component, FGuid TagGuid)
{
	// Linear search
	for (TObjectPtr MaterialCacheVirtualTexture : Component->MaterialCacheTextures)
	{
		UMaterialCacheVirtualTextureTag* TagHandle = MaterialCacheVirtualTexture->Tag.Get();
		if ((TagHandle ? TagHandle->Guid : FGuid()) == TagGuid)
		{
			return MaterialCacheVirtualTexture;
		}
	}

	return nullptr;
}

void UPreviewMeshComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	Super::SetMaterial(ElementIndex, Material);

	if (!ensure(Material != nullptr))
	{
		return;
	}
	
	if (IsMaterialCacheEnabled(GetWorld()))
	{
		int32 TileCount = CVarPreviewMaterialCacheTileCount->GetInt();
		UpdateMaterialCacheTextures(this, Material, FIntPoint(TileCount, TileCount), MaterialCacheTextures);
	}
}

void UPreviewMeshComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	Super::GetUsedMaterials(OutMaterials, bGetDebugMaterials);
}

FPrimitiveSceneProxy* UPreviewMeshComponent::CreateSceneProxy()
{
	if (MeshData->VertexCount() == 0)
	{
		return nullptr;
	}
	return new FMegaMeshCustomPreviewSceneProxy(this);
}

FMegaMeshCustomPreviewSceneProxy* UPreviewMeshComponent::GetCustomSceneProxy() const
{
	return static_cast<FMegaMeshCustomPreviewSceneProxy*>(SceneProxy);
}

HPreviewProxy::HPreviewProxy(const UActorComponent* InComponent)
	: HComponentVisProxy(InComponent, EHitProxyPriority::HPP_World)
	, Actor(InComponent ? InComponent->GetOwner() : nullptr)
{}

IMPLEMENT_HIT_PROXY(MeshPartition::HPreviewProxy, HComponentVisProxy);
} // namespace UE::MeshPartition