// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPreviewSceneProxy.h"

#include "MeshPartitionPreviewComponents.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionMaterialCacheCommon.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"
#include "MaterialCache/MaterialCacheVirtualTextureRenderProxy.h"
#include "MaterialCachedData.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "Engine/Engine.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "SceneInterface.h"
#include "SceneView.h"

#if RHI_RAYTRACING
#include "RayTracingInstance.h"
#endif // RHI_RAYTRACING

namespace UE::MeshPartition
{

FMegaMeshCustomPreviewSceneProxy::FMegaMeshCustomPreviewSceneProxy(UPreviewMeshComponent* InComponent)
	: FPrimitiveSceneProxy(InComponent)
	, VertexFactory(GetScene().GetFeatureLevel(), "MegaMeshCustomPreviewVertexFactory")
	, MaterialRelevance(InComponent->GetMaterialRelevance(GetScene().GetShaderPlatform()))
{
	if (ensure(InComponent->GetMeshData()->VertexCount() > 0))
	{
		InitResources(*InComponent->GetMeshData());

		if (!InComponent->MaterialCacheTextures.IsEmpty())
		{
			for (UMaterialCacheVirtualTexture* MaterialCacheTexture : InComponent->MaterialCacheTextures)
			{
				if (FMaterialCacheVirtualTextureRenderProxy* RenderProxy = MaterialCacheTexture->CreateRenderProxy(
					FBox2f(FVector2f::Zero(), FVector2f::One()),
					0
				))
				{
					MaterialCacheRenderProxies.Emplace(RenderProxy);
				}
			}

			MaterialCacheDescriptor = PackMaterialCacheDescriptor(
				UINT16_MAX,
				0
			);
		}

		ENQUEUE_RENDER_COMMAND(FMegaMeshCustomPreviewSceneProxyInitialize)(
			[&](FRHICommandListImmediate& RHICmdList)
		{
			PositionVertexBuffer.InitResource(RHICmdList);
			StaticMeshVertexBuffer.InitResource(RHICmdList);
			ColorVertexBuffer.InitResource(RHICmdList);

			FLocalVertexFactory::FDataType Data;
			PositionVertexBuffer.BindPositionVertexBuffer(&VertexFactory, Data);
			StaticMeshVertexBuffer.BindTangentVertexBuffer(&VertexFactory, Data);
			StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&VertexFactory, Data);
			ColorVertexBuffer.BindColorVertexBuffer(&VertexFactory, Data);

			VertexFactory.SetData(RHICmdList, Data);

			VertexFactory.InitResource(RHICmdList);


			PositionVertexBuffer.InitResource(RHICmdList);
			StaticMeshVertexBuffer.InitResource(RHICmdList);
			ColorVertexBuffer.InitResource(RHICmdList);
			IndexBuffer.InitResource(RHICmdList);

#if RHI_RAYTRACING
			InitRayTracingGeometry(RHICmdList);
#endif // RHI_RAYTRACING
		});
	}

	Material = InComponent->GetMaterial(0);

	bSupportsRuntimeVirtualTexture = true;

	if (FMaterial* MaterialResource = Material->GetMaterialResource(GShaderPlatformForFeatureLevel[GetScene().GetFeatureLevel()]))
	{
		bSupportsMaterialCache = MaterialResource->SamplesMaterialCache();
	}
}

FMegaMeshCustomPreviewSceneProxy::~FMegaMeshCustomPreviewSceneProxy()
{
	VertexFactory.ReleaseResource();
	PositionVertexBuffer.ReleaseResource();
	StaticMeshVertexBuffer.ReleaseResource();
	ColorVertexBuffer.ReleaseResource();
	IndexBuffer.ReleaseResource();
#if RHI_RAYTRACING
	RayTracingGeometry.ReleaseResource();
#endif // RHI_RAYTRACING
}

void FMegaMeshCustomPreviewSceneProxy::InitResources(const FMeshData& Mesh)
{
	const int32 NumTexCoords = Mesh.GetNumUVChannels();

	TArray<uint32>& Indices = IndexBuffer.Indices;

	Tasks::FTask BuildIndexBuffer = Tasks::Launch(TEXT("BuildIndexBuffer"),
		[&]()
		{
			Indices.Reserve(Mesh.MaxTriangleID() * 3);

			TArray<TArray<int>> IndexGroups;
			ParallelForWithTaskContext(IndexGroups, Mesh.MaxTriangleID(),
			[&](TArray<int>& OutIndexGroup, int TriangleID)
			{
				if (!Mesh.IsTriangle(TriangleID))
				{
					return;
				}
				const Geometry::FIndex3i Triangle = Mesh.GetTriangle(TriangleID);
				OutIndexGroup.Add(Triangle[0]);
				OutIndexGroup.Add(Triangle[1]);
				OutIndexGroup.Add(Triangle[2]);
			});

			for (TArray<int>& IndexGroup : IndexGroups)
			{
				Indices.Append(MoveTemp(IndexGroup));
			}
		});

	Tasks::FTask BuildVertexBuffers = Tasks::Launch(TEXT("BuildVertexBuffers"),
		[&]()
		{
			StaticMeshVertexBuffer.SetUseFullPrecisionUVs(true);
			StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(true);

			PositionVertexBuffer.Init(Mesh.MaxVertexID());
			StaticMeshVertexBuffer.Init(Mesh.MaxVertexID(), NumTexCoords);
			ColorVertexBuffer.Init(Mesh.MaxVertexID());

			ParallelFor(Mesh.MaxVertexID(), [&](int VertexID)
			{
				if (!Mesh.IsVertex(VertexID))
				{
					return;
				}

				const FVector3f Normal = FVector3f(Mesh.GetVertexNormal(VertexID));
				PositionVertexBuffer.VertexPosition(VertexID) = FVector3f(Mesh.GetVertex(VertexID));
				StaticMeshVertexBuffer.SetVertexTangents(VertexID, FVector3f(1, 0, 0), FVector3f(0, 1, 0), Normal);
				ColorVertexBuffer.VertexColor(VertexID) = FColor::Black;

				StaticMeshVertexBuffer.SetVertexUV(VertexID, FMeshData::CHANNEL_UV_INDEX, Mesh.GetChannelUV(VertexID));

				for (int32 ChannelIdx = 0; ChannelIdx < Mesh.GetNumSourceUVChannels(); ++ChannelIdx)
				{
					StaticMeshVertexBuffer.SetVertexUV(VertexID, ChannelIdx + FMeshData::SOURCE_UV_OFFSET, FVector2f(Mesh.GetVertexUV(VertexID, ChannelIdx)));
				}
			});
		});

	Tasks::Wait( MakeArrayView({ BuildIndexBuffer, BuildVertexBuffers }));
}

void FMegaMeshCustomPreviewSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	FMeshBatch BaseMeshBatch;
	BaseMeshBatch.VertexFactory = &VertexFactory;
	BaseMeshBatch.MaterialRenderProxy = Material ? Material->GetRenderProxy() : UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	BaseMeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
	BaseMeshBatch.Type = PT_TriangleList;
	BaseMeshBatch.DepthPriorityGroup = SDPG_World;
	BaseMeshBatch.LODIndex = 0;
	BaseMeshBatch.SegmentIndex = 0;
	BaseMeshBatch.MeshIdInPrimitive = 0;
	BaseMeshBatch.LCI = nullptr;
	BaseMeshBatch.CastShadow = true;
	BaseMeshBatch.bUseForMaterial = true;
	BaseMeshBatch.bDitheredLODTransition = false;
	BaseMeshBatch.bUseForDepthPass = true;
	BaseMeshBatch.bUseAsOccluder = ShouldUseAsOccluder();

	FMeshBatchElement& BatchElement = BaseMeshBatch.Elements[0];

	BatchElement.IndexBuffer = &IndexBuffer;
	BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
	BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
	BatchElement.FirstIndex = 0;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = PositionVertexBuffer.GetNumVertices() - 1;

	PDI->DrawMesh(BaseMeshBatch, FLT_MAX);

	{
		FMeshBatch VTMeshBatch(BaseMeshBatch);
		VTMeshBatch.CastShadow = false;
		VTMeshBatch.bUseAsOccluder = false;
		VTMeshBatch.bUseForDepthPass = false;
		VTMeshBatch.bUseForMaterial = false;
		VTMeshBatch.bDitheredLODTransition = false;
		VTMeshBatch.bRenderToVirtualTexture = true;
		for (ERuntimeVirtualTextureMaterialType MaterialType : RuntimeVirtualTextureMaterialTypes)
		{
			VTMeshBatch.RuntimeVirtualTextureMaterialType = (uint32)MaterialType;
			PDI->DrawMesh(VTMeshBatch, FLT_MAX);
		}
	}
}

void FMegaMeshCustomPreviewSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;
	const bool bWireframe = (AllowDebugViewmodes() && EngineShowFlags.Wireframe);
	const bool bIsSelected = IsSelected();

	FMaterialRenderProxy* WireframeMaterialProxy = nullptr;
	if (bWireframe)
	{
		const FLinearColor WireframeColorOverride = bIsSelected ? GEngine->GetSelectedMaterialColor() : GetWireframeColor();
		WireframeMaterialProxy = new FColoredMaterialRenderProxy(GEngine->WireframeMaterial->GetRenderProxy(), WireframeColorOverride);
		Collector.RegisterOneFrameMaterialProxy(WireframeMaterialProxy);
	}

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			FMaterialRenderProxy* MaterialProxy = Material ? Material->GetRenderProxy() : UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();

			FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
			FPrimitiveUniformShaderParametersBuilder Builder;
			BuildUniformShaderParameters(Builder);
			DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

			if (IndexBuffer.Indices.Num() > 0)
			{
				FMeshBatch& MeshBatch = Collector.AllocateMesh();
				FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
				BatchElement.IndexBuffer = &IndexBuffer;
				MeshBatch.bWireframe = bWireframe;
				MeshBatch.bDisableBackfaceCulling = bWireframe;
				MeshBatch.VertexFactory = &VertexFactory;
				MeshBatch.MaterialRenderProxy = bWireframe ? WireframeMaterialProxy : MaterialProxy;
				MeshBatch.bRenderToVirtualTexture = true;

				BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

				BatchElement.FirstIndex = 0;
				BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
				BatchElement.MinVertexIndex = 0;
				BatchElement.MaxVertexIndex = PositionVertexBuffer.GetNumVertices() - 1;
				MeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
				MeshBatch.Type = PT_TriangleList;
				MeshBatch.DepthPriorityGroup = SDPG_World;
				Collector.AddMesh(ViewIndex, MeshBatch);
			}
		}
	}
}

#if RHI_RAYTRACING
void FMegaMeshCustomPreviewSceneProxy::InitRayTracingGeometry(FRHICommandListBase& RHICmdList)
{
	if (!IsRayTracingEnabled())
	{
		return;
	}

	RayTracingGeometry.ReleaseResource();

	FRayTracingGeometryInitializer Initializer;
	Initializer.IndexBuffer = IndexBuffer.IndexBufferRHI;
	Initializer.TotalPrimitiveCount = IndexBuffer.Indices.Num() / 3;
	Initializer.GeometryType = RTGT_Triangles;
	Initializer.bFastBuild = true;
	Initializer.bAllowUpdate = false;

	FRayTracingGeometrySegment Segment;
	Segment.VertexBuffer = PositionVertexBuffer.VertexBufferRHI;
	Segment.NumPrimitives = Initializer.TotalPrimitiveCount;
	Segment.MaxVertices = PositionVertexBuffer.GetNumVertices();

	Initializer.Segments.Add(Segment);

	RayTracingGeometry.SetInitializer(MoveTemp(Initializer));
	RayTracingGeometry.InitResource(RHICmdList);
}

bool FMegaMeshCustomPreviewSceneProxy::IsRayTracingRelevant() const
{
	return true;
}

bool FMegaMeshCustomPreviewSceneProxy::HasRayTracingRepresentation() const
{
	return true;
}

void FMegaMeshCustomPreviewSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

	FRayTracingInstance RayTracingInstance;
	RayTracingInstance.Geometry = &RayTracingGeometry;
	RayTracingInstance.InstanceTransforms.Add(GetLocalToWorld());

	FMeshBatch MeshBatch;

	MeshBatch.VertexFactory = &VertexFactory;
	MeshBatch.SegmentIndex = 0;
	MeshBatch.MaterialRenderProxy = Material ? Material->GetRenderProxy() : UMaterial::GetDefaultMaterial(MD_Surface)->GetRenderProxy();
	MeshBatch.Type = PT_TriangleList;
	MeshBatch.DepthPriorityGroup = SDPG_World;
	MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);

	FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
	BatchElement.IndexBuffer = &IndexBuffer;
	BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
	BatchElement.FirstIndex = 0;
	BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = PositionVertexBuffer.GetNumVertices() - 1;

	RayTracingInstance.Materials.Add(MeshBatch);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if ((VisibilityMap & (1 << ViewIndex)) == 0)
		{
			continue;
		}

		Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
	}
}
#endif // RHI_RAYTRACING

FPrimitiveViewRelevance FMegaMeshCustomPreviewSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	const FEngineShowFlags& EngineShowFlags = View->Family->EngineShowFlags;

	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bStaticRelevance = !EngineShowFlags.Wireframe;
	Result.bDynamicRelevance = EngineShowFlags.Wireframe;
	Result.bEditorVisualizeLevelInstanceRelevance = IsEditingLevelInstanceChild();
	Result.bEditorStaticSelectionRelevance = (IsSelected() || IsHovered());
	Result.bRenderInMainPass = ShouldRenderInMainPass();
	Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
	Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
	Result.bRenderCustomDepth = ShouldRenderCustomDepth();

	MaterialRelevance.SetPrimitiveViewRelevance(Result);

	Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

	return Result;
}

SIZE_T FMegaMeshCustomPreviewSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

HHitProxy* FMegaMeshCustomPreviewSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies)
{
	MeshPartition::HPreviewProxy* HProxy = new MeshPartition::HPreviewProxy(Component);
	OutHitProxies.Add(HProxy);
	return HProxy;
}

uint32 FMegaMeshCustomPreviewSceneProxy::GetMemoryFootprint(void) const { return(sizeof(*this) + GetAllocatedSize()); }
uint32 FMegaMeshCustomPreviewSceneProxy::GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }

} // namespace UE::MeshPartition
