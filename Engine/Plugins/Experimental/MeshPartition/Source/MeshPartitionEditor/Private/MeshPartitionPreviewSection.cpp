// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPartitionPreviewSection.h"

#include "MeshPartitionPreviewComponents.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshPartition.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionChannel.h"
#include "RenderUtils.h"
#include "MaterialCachedData.h"
#include "MeshPartitionMaterialCacheCommon.h"
#include "MeshPartitionStaticMeshDescriptor.h"
#include "MaterialCache/MaterialCache.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"

namespace UE::MeshPartition
{
APreviewSection::APreviewSection()
	: Parent(nullptr)
{
#if WITH_EDITORONLY_DATA
	bLockLocation = true;
#endif // WITH_EDITORONLY_DATA

	PrimaryActorTick.bCanEverTick = false;

	USceneComponent* SceneComponent = CreateDefaultSubobject<USceneComponent>(TEXT("SceneComponent"));
	SceneComponent->SetMobility(EComponentMobility::Static);
	RootComponent = SceneComponent;
	
	const FName CollisionProfileName = UCollisionProfile::BlockAll_ProfileName;

	PreviewMeshComponent = CreateDefaultSubobject<MeshPartition::UPreviewMeshComponent>(TEXT("PreviewMeshComponent"));
	PreviewMeshComponent->SetCollisionProfileName(CollisionProfileName);
	PreviewMeshComponent->SetupAttachment(RootComponent);
	PreviewMeshComponent->SetMobility(EComponentMobility::Static);

	FarFieldMeshComponent = CreateDefaultSubobject<MeshPartition::UMeshPartitionStaticMeshComponent>(TEXT("FarFieldMeshComponent"));
	FarFieldMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	FarFieldMeshComponent->SetupAttachment(RootComponent);
	FarFieldMeshComponent->SetMobility(EComponentMobility::Static);
	FarFieldMeshComponent->SetGenerateOverlapEvents(false);
	FarFieldMeshComponent->SetHiddenInGame(true);
	FarFieldMeshComponent->SetVisibility(false);
	FarFieldMeshComponent->bAffectIndirectLightingWhileHidden = true;
	FarFieldMeshComponent->CastShadow = false;
	FarFieldMeshComponent->bAllowCullDistanceVolume = false;
	FarFieldMeshComponent->bNeverDistanceCull = true;
	FarFieldMeshComponent->bRayTracingFarField = true;
}

void APreviewSection::BeginDestroy()
{
	DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);

	Super::BeginDestroy();
}

void APreviewSection::PostUnregisterAllComponents()
{
	Super::PostUnregisterAllComponents();
	
	for (TObjectPtr VirtualTexture : MaterialCacheTextures)
	{
		VirtualTexture->Unregister();
		VirtualTexture->ReleaseResource();
	}
	
	MaterialCacheTextures.Empty();
}

void APreviewSection::PostRegisterAllComponents()
{
	// Note: If Parent is valid, it should always have a valid TransientSectionAttachAnchor (as assigned in Parent's constructor)
	if (Parent.IsValid() && ensure(Parent->GetTransientSectionAttachAnchor()))
	{
		AttachToComponent(Parent->GetTransientSectionAttachAnchor(), FAttachmentTransformRules::SnapToTargetIncludingScale);
	}

	Super::PostRegisterAllComponents();
}

TArray<TObjectPtr<UStaticMesh>> APreviewSection::GetMeshes() const
{
	TArray<TObjectPtr<UStaticMesh>> StaticMeshes;

	StaticMeshes.SetNum(MeshComponents.Num());

	for (int32 Index = 0; Index < MeshComponents.Num(); ++Index)
	{
		StaticMeshes[Index] = MeshComponents[Index]->GetStaticMesh();
	}

	return StaticMeshes;
}

//todo(luc.eygasier): should we try to externally add the component to let the transformer do it?
void APreviewSection::AddMesh(UStaticMesh* InStaticMesh, const FStaticMeshDescriptor& InDescriptor)
{
	Modify(true);

	MeshPartition::UStaticMeshPreviewComponent* MeshComponent = NewObject<MeshPartition::UStaticMeshPreviewComponent>(this, MakeUniqueObjectName(this, MeshPartition::UStaticMeshPreviewComponent::StaticClass(), TEXT("MeshComponent")));
	
	MeshComponent->SetStaticMesh(InStaticMesh);
	MeshComponent->SetupAttachment(RootComponent);
	MeshComponent->SetMobility(EComponentMobility::Static);
	MeshComponent->SetCollisionProfileName(InDescriptor.CollisionProfileName);
	MeshComponent->SetVisibility(false);
	MeshComponent->SetCanEverAffectNavigation(InDescriptor.bCanEverAffectNavigation);
	MeshComponent->MaterialCacheTextures = MaterialCacheTextures;

	// TODO: Restore later
	// MeshComponent->MaterialCacheUVRegion = Descriptor.UVRegion;
	
	// When creating these new MeshComponents, assign the custom primitive data required for rendering via the channel textures.
	FChannelPacking::SetCustomPrimitiveData(MeshComponent, ChannelTable, ChannelTexcoordDesc);

	MeshComponents.Emplace(MeshComponent);

	// #todo: it would be better for the mesh component to be instanced so it shows up in the details panel.
	// However, currently the details panel will block the static mesh compilation in order to display the details.
	//AddInstanceComponent(MeshComponent);
	MeshComponent->RegisterComponent();

	if (UseNanite(GMaxRHIShaderPlatform))
	{
		UStaticMeshComponent* VirtualTextureFallbackMeshComponent = NewObject<UStaticMeshComponent>(this, MakeUniqueObjectName(this, UStaticMeshComponent::StaticClass(), TEXT("VirtualTextureFallbackMeshComponent")));
		VirtualTextureFallbackMeshComponent->OnComponentCreated();
		// #todo: it would be better for the mesh component to be instanced so it shows up in the details panel.
		// However, currently the details panel will block the static mesh compilation in order to display the details.
		//AddInstanceComponent(VirtualTextureFallbackMeshComponent);

		VirtualTextureFallbackMeshComponent->SetStaticMesh(InStaticMesh);
		VirtualTextureFallbackMeshComponent->SetupAttachment(RootComponent);
		VirtualTextureFallbackMeshComponent->SetMobility(EComponentMobility::Static);
		VirtualTextureFallbackMeshComponent->SetForceDisableNanite(true);
		VirtualTextureFallbackMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
		
		VirtualTextureFallbackMeshComponent->SetRenderInMainPass(false);
		VirtualTextureFallbackMeshComponent->SetRenderInDepthPass(false);
		VirtualTextureFallbackMeshComponent->SetVisibleInRayTracing(false);
		VirtualTextureFallbackMeshComponent->SetCastShadow(false);
		VirtualTextureFallbackMeshComponent->bSelectable = false;

		FChannelPacking::SetCustomPrimitiveData(VirtualTextureFallbackMeshComponent, ChannelTable, ChannelTexcoordDesc);

		VirtualTextureFallbackMeshComponent->RegisterComponent();

		VirtualTextureFallbackMeshComponents.Emplace(VirtualTextureFallbackMeshComponent);
	}

	InStaticMesh->OnPostMeshBuild().AddUObject(this, &APreviewSection::OnStaticMeshBuild);
}

void APreviewSection::AddCollisionComponent(UMeshPartitionCollisionComponent* InCollisionComponent)
{
	if (InCollisionComponent == nullptr)
	{
		return;
	}

	//todo(luc.eygasier): is this really necessary? Doesn't seem used
	CollisionComponents.Emplace(InCollisionComponent);
	AddInstanceComponent(InCollisionComponent);
	InCollisionComponent->RegisterComponent();
}

void APreviewSection::SetFarFieldMesh(UStaticMesh* InStaticMesh)
{
	Modify(true);

	FarFieldMeshComponent->SetStaticMesh(InStaticMesh);

	FChannelPacking::SetCustomPrimitiveData(FarFieldMeshComponent, ChannelTable, ChannelTexcoordDesc);

	InStaticMesh->OnPostMeshBuild().AddUObject(this, &APreviewSection::OnFarFieldStaticMeshBuild);
}

TSharedPtr<const MeshPartition::FMeshData> APreviewSection::GetPreviewMesh() const
{
	return PreviewMeshComponent->GetMeshData();
}

void APreviewSection::SetPreviewMesh(TSharedRef<const MeshPartition::FMeshData> InMeshData, const FName& InCollisionProfileName, const bool bInCanEverAffectNavigation)
{
	PreviewMeshComponent->SetMeshData(InMeshData);

	if (UMaterialInterface* MegaMeshMaterial = GetPreviewMaterial())
	{
		PreviewMeshComponent->SetMaterial(0, MegaMeshMaterial);
	}

	PreviewMeshBounds = PreviewMeshComponent->Bounds.GetBox();

	PreviewMeshComponent->SetCanEverAffectNavigation(bInCanEverAffectNavigation);
	PreviewMeshComponent->SetCollisionProfileName(InCollisionProfileName);
}

AMeshPartition* APreviewSection::GetParent() const
{
	return Parent.Get();
}

void APreviewSection::SetParent(AMeshPartition* InMegaMesh)
{
	Modify(true);
	DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
	Parent = InMegaMesh;
	
	// Note: If Parent is valid, it should always have a valid TransientSectionAttachAnchor (as assigned in Parent's constructor)
	if (Parent.IsValid() && ensure(Parent->GetTransientSectionAttachAnchor()))
	{
		AttachToComponent(Parent->GetTransientSectionAttachAnchor(), FAttachmentTransformRules::SnapToTargetIncludingScale);
	}
}

void APreviewSection::SetBuildPerfStats(MeshPartition::FBuildPerfStats&& InBuildPerfStats)
{
	BuildPerfStats = InBuildPerfStats;
}

const MeshPartition::FBuildPerfStats& APreviewSection::GetBuildPerfStats() const
{
	return BuildPerfStats;
}

void APreviewSection::InvalidateRenderStates()
{
	ForAllPrimitiveComponents([](UPrimitiveComponent* PrimitiveComponent)
	{
		PrimitiveComponent->MarkRenderStateDirty();
		return true;
	});
}

void APreviewSection::SetMaterialInstance(UMaterialInstanceDynamic* InMaterialInstance)
{
	// new material instance
	MaterialInstance = InMaterialInstance;

	// reassign the self channel texture to the correct parameter in the material
	if (MaterialInstance && ChannelTexture)
	{
		SetChannelTexture(ChannelTexture);
	}

	// Need to reapply the new material instance to all the sub primitives of this section
	ForAllPrimitiveComponents([InMaterialInstance](UPrimitiveComponent* PrimitiveComponent)
	{
		PrimitiveComponent->SetMaterial(0, InMaterialInstance);
		return true;
	});
}

void APreviewSection::SetChannelData(TConstArrayView<uint8> InChannelTable, const FVector2f& InTexcoordDesc)
{
	// Channel table is the channel index to texture slice indirection for the section
	// unused channels (in this section) are marked -1
	// used channels have a valid index matching a slice of the section's channel texture array
	ChannelTable = InChannelTable;
	ChannelTexcoordDesc = InTexcoordDesc;

	// Channel Table is assigned to all the primitive components of the Section as a custom primitive data used by the Material
	ForAllPrimitiveComponents([&](UPrimitiveComponent* PrimitiveComponent)
	{
		FChannelPacking::SetCustomPrimitiveData(PrimitiveComponent, ChannelTable, ChannelTexcoordDesc);
		return true;
	});
}

void APreviewSection::SetChannelTexture(UTexture* InChannelTexture)
{
	ChannelTexture = InChannelTexture;

	UMaterialInstanceDynamic* SectionMID = Cast<UMaterialInstanceDynamic>(GetPreviewMaterial());

	if (SectionMID)
	{
		SectionMID->SetTextureParameterValue(UE::MeshPartition::ChannelTextureParameterName, InChannelTexture);
	}
}

void APreviewSection::SetMaterialCacheTileCount(const FIntPoint& TileCount)
{
	MaterialCacheTileCount = TileCount;
}

void APreviewSection::OnMeshPartitionDefinitionChanged(const UMeshPartitionDefinition* InDefinition)
{
	UMaterialInterface* MegaMeshMaterial = GetPreviewMaterial();

	if (!ensureMsgf(MegaMeshMaterial != nullptr, TEXT("Null material assigned to the MegaMeshPreviewSection. This should always at least be an engine fallback material")))
	{
		return;
	}

	ForAllPrimitiveComponents([MegaMeshMaterial](UPrimitiveComponent* PrimitiveComponent)
	{
		PrimitiveComponent->SetMaterial(0, MegaMeshMaterial);
		return true;
	});
}

void APreviewSection::OnMeshPartitionDefinitionModified(const UMeshPartitionDefinition* InDefinition, const FName& InPropertyName)
{
	OnMeshPartitionDefinitionChanged(InDefinition);
}

void APreviewSection::RecreateMaterialCacheTextures()
{
	if (!MaterialInstance)
	{
		return;
	}
	
	if (IsMaterialCacheEnabled(GetWorld()))
	{
		UpdateMaterialCacheTextures(RootComponent, MaterialInstance, MaterialCacheTileCount, MaterialCacheTextures);
		
		// Propagate to relevant components
		for (TObjectPtr<UStaticMeshComponent> MeshComponent : GetMeshComponents())
		{
			MeshComponent->MaterialCacheTextures = MaterialCacheTextures;
			MeshComponent->MarkRenderStateDirty();
		}
	}
}

void APreviewSection::AddBaseModifier(const TWeakObjectPtr<MeshPartition::UModifierComponent>& InBaseModifier)
{
	if (!InBaseModifier.IsValid())
	{
		return;
	}

	BaseModifiers.Emplace(InBaseModifier);	
}

void APreviewSection::RemoveBaseModifier(const TWeakObjectPtr<MeshPartition::UModifierComponent>& InBaseModifier)
{
	if (!InBaseModifier.IsValid())
	{
		return;
	}

	BaseModifiers.Remove(InBaseModifier);	
}

void APreviewSection::AddModifier(const TWeakObjectPtr<MeshPartition::UModifierComponent>& InModifier)
{
	if (!InModifier.IsValid())
	{
		return;
	}

	Modifiers.Emplace(InModifier);	
}

UMeshPartitionEditorComponent* APreviewSection::GetMegaMeshEditorComponent() const
{
	return Cast<UMeshPartitionEditorComponent>(Parent->GetMeshPartitionComponent());
}

void APreviewSection::SetRuntimeVirtualTextures(const TArray<TObjectPtr<URuntimeVirtualTexture>>& InRVTs)
{
	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		MeshComponent->RuntimeVirtualTextures = InRVTs;
	}

	PreviewMeshComponent->RuntimeVirtualTextures = InRVTs;
	
	for (UStaticMeshComponent* VirtualTextureFallbackMeshComponent : VirtualTextureFallbackMeshComponents)
	{
		VirtualTextureFallbackMeshComponent->RuntimeVirtualTextures = InRVTs;
	}

	InvalidateRenderStates();
}

void APreviewSection::OnStaticMeshBuild(UStaticMesh* InStaticMesh)
{
	ensure(!InStaticMesh->IsCompiling());
	// First pass to check if all resources are ready before continuing.
	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		UStaticMesh* StaticMesh = MeshComponent->GetStaticMesh();

		if ((StaticMesh != nullptr) && (StaticMesh->IsCompiling()))
		{
			return;
		}
	}
	
	for (TObjectPtr<UMaterialCacheVirtualTexture> MaterialCacheVirtualTexture : MaterialCacheTextures)
	{
		MaterialCacheVirtualTexture->RecreateAllocation();
	}

	PreviewMeshComponent->SetMeshData(MeshPartition::FMeshData{});

	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		MeshComponent->SetVisibility(GetRootComponent()->GetVisibleFlag());
		MeshComponent->MarkRenderStateDirty();
	}
}

void APreviewSection::OnFarFieldStaticMeshBuild(UStaticMesh* InStaticMesh)
{
	ensure(!InStaticMesh->IsCompiling());

	FarFieldMeshComponent->UpdateMaterialCacheTextures();
	FarFieldMeshComponent->MarkRenderStateDirty();
}

UMaterialInterface* APreviewSection::GetPreviewMaterial() const
{
	if (IsValid(MaterialInstance))
	{
		return MaterialInstance;
	}

	return UMaterial::GetDefaultMaterial(MD_Surface);
}

void APreviewSection::ForAllPrimitiveComponents(TFunctionRef<bool(UPrimitiveComponent*)> InFunc) const
{
	auto HandleComponent = [InFunc](UPrimitiveComponent* Component)
	{
		if (Component == nullptr)
		{
			return true;
		}

		return InFunc(Component);
	};

	if (!HandleComponent(PreviewMeshComponent))
	{
		return;
	}

	for (UStaticMeshComponent* MeshComponent : MeshComponents)
	{
		if (!HandleComponent(MeshComponent))
		{
			return;
		}
	}

	if (!HandleComponent(FarFieldMeshComponent))
	{
		return;
	}

	for (UStaticMeshComponent* VirtualTextureFallbackMeshComponent : VirtualTextureFallbackMeshComponents)
	{
		if (!HandleComponent(VirtualTextureFallbackMeshComponent))
		{
			return;
		}
	}
}
} // namespace UE::MeshPartition