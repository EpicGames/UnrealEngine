// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPreviewVisualizers.h"

#include "Components/PrimitiveComponent.h"
#include "Editor.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelEditor.h"
#include "LevelEditorContextMenu.h"
#include "MeshPartition.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionPreviewComponents.h"
#include "MeshPartitionPreviewSection.h"
#include "ScopedTransaction.h"
#include "SEditorViewport.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "MegaMeshPreviewVisualizers"

namespace UE::MeshPartition
{
namespace MegaMeshPreviewVisualizersLocals
{
	void PerformSelection(UTypedElementSelectionSet& SelectionSet, const TArray<AActor*>& Actors,
		bool bAddModifier, bool bToggleModifier)
	{
		const FScopedTransaction Transaction(LOCTEXT("SelectionTransaction", "Change Selection"));

		if (!bToggleModifier && !bAddModifier)
		{
			SelectionSet.ClearSelection(FTypedElementSelectionOptions());
		}

		for (AActor* Actor : Actors)
		{
			if (!IsValid(Actor))
			{
				continue;
			}

			FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor);
			if (!ElementHandle.IsSet())
			{
				continue;
			}

			bool bAlreadySelected = SelectionSet.IsElementSelected(ElementHandle, FTypedElementIsSelectedOptions());

			if (bAlreadySelected && bAddModifier)
			{
				// Nothing to do in this case
				continue;
			}

			if (bAlreadySelected && bToggleModifier)
			{
				SelectionSet.DeselectElement(ElementHandle, FTypedElementSelectionOptions());
			}
			else
			{
				SelectionSet.SelectElement(ElementHandle, FTypedElementSelectionOptions());
			}
		}

		SelectionSet.NotifyPendingChanges();
	};

	// Used to create a submenu per relevant actor when right clicking a megamesh location
	void CreateSubMenuForActor(const FComponentVisualizer& Visualizer, UToolMenu* ActorSubMenu, AActor* Actor, UTypedElementSelectionSet& SelectionSet)
	{
		if (!ensure(ActorSubMenu && Actor))
		{
			return;
		}

		FToolMenuSection& Section = ActorSubMenu->AddSection(NAME_None);

		Section.AddSubMenu(NAME_None,
			LOCTEXT("RightClick_label", "Right Click Menu"),
			LOCTEXT("RightClick_tooltip", "Clicking this changes selection to the given actor and opens its right click menu"),
			FNewToolMenuDelegate::CreateSPLambda(Visualizer.AsShared(),
				[Actor, &SelectionSet](UToolMenu* Menu)
				{
					// We want the regular right click menu to appear for the given actor. However currently this requires
					//  the actor to be selected, so when a user clicks this sub menu label, change selection and open the
					//  sub menu.
					PerformSelection(SelectionSet, { Actor }, false, false);
					FLevelEditorModule* LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
					if (!LevelEditorModule)
					{
						return;
					}
					TWeakPtr<ILevelEditor> WeakLevelEditor = LevelEditorModule->GetLevelEditorInstance();
					FTypedElementHandle ElementHandle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor);
					FLevelEditorContextMenu::InitMenuContext(Menu->Context, WeakLevelEditor,
						ELevelEditorMenuContext::Viewport, ElementHandle);
					UToolMenu* OtherMenu = UToolMenus::Get()->GenerateMenu(FName("LevelEditor.ActorContextMenu"), Menu->Context);
					if (!OtherMenu)
					{
						return;
					}
					Menu->Sections.Append(OtherMenu->Sections);
				}
			), 
			// Require clicking the actor to actually make the actor menu appear.
			/*bOpenSubMenuOnClick*/ true);

	}

	// Creates the menu that should appear when right clicking a megamesh location
	bool SummonRightClickMenu(const FComponentVisualizer& Visualizer, 
		FEditorViewportClient* ViewportClient, const FViewportClick& Click,
		const UPrimitiveComponent* PreviewSectionComponent,
		UTypedElementSelectionSet& SelectionSet,
		const UMeshPartitionEditorComponent& MegaMesh, const TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>>& BaseMeshes,
		const FVector3d& HitLocation,
		bool bAddModifier, bool bToggleModifier)
	{
		if (!ensure(ViewportClient && ViewportClient->Viewport))
		{
			return false;
		}

		TSharedPtr<SWidget> ViewportWidget = ViewportClient->GetEditorViewportWidget();
		if (!ensure(ViewportWidget.IsValid()))
		{
			return false;
		}

		UToolMenus* ToolMenus = UToolMenus::Get();
		static const FName MenuName("MegaMesh.SelectionMenu");
		if (!ToolMenus->IsMenuRegistered(MenuName))
		{
			ToolMenus->RegisterMenu(MenuName);
		}

		FToolMenuContext Context;
		UToolMenu* Menu = ToolMenus->GenerateMenu(MenuName, Context);

		TSet<AActor*> AddedActors;
		auto AddOwnerToMenu = [&Visualizer, &SelectionSet, &AddedActors, bAddModifier, bToggleModifier]
			(FToolMenuSection& Section, const UActorComponent* Component)
		{
			AActor* ParentActor = Component->GetOwner();
			if (!ParentActor)
			{
				return;
			}
			bool bAlreadyAdded = false;
			AddedActors.Add(ParentActor, &bAlreadyAdded);
			if (bAlreadyAdded)
			{
				return;
			}

			// We want the user to be able to click the section itself to complete the selection action, so this
			//  action will be bound to the section click. However we also want to be able to display a submenu
			//  similar to one that one would get from right clicking an actor normally in the viewport or outliner.
			FToolUIAction Action;
			Action.ExecuteAction = FToolMenuExecuteAction::CreateSPLambda(Visualizer.AsShared(), 
				[&SelectionSet, ParentActor, bAddModifier, bToggleModifier](const FToolMenuContext& InContext)
			{
				PerformSelection(SelectionSet, { ParentActor }, bAddModifier, bToggleModifier);
			});

			Section.AddSubMenu(NAME_None,
				FText::FromString(ParentActor->GetActorNameOrLabel()),
				FText::FromString(ParentActor->GetName()),
				FNewToolMenuDelegate::CreateSPLambda(Visualizer.AsShared(),
					[&Visualizer, &SelectionSet, ParentActor](UToolMenu* Menu)
					{
						CreateSubMenuForActor(Visualizer, Menu, ParentActor, SelectionSet);
					}
				),
				FToolUIActionChoice(Action), EUserInterfaceActionType::Button, false);
		};

		// Gather relevant modifiers. We want to display these before the sections because it's more likely
		//  that the user is right clicking to select these than the base sections (which they will frequently
		//  successfully select with left click).
		// TODO: Would be nice to have some way to do this spatially, without going through all loaded modifiers.
		//  But probably ok for now for something that only runs once on right click.
		TArray<MeshPartition::UModifierComponent*> RelevantModifiers = MegaMesh.GetModifiersFiltered(
			[&HitLocation](const MeshPartition::UModifierComponent* Modifier)
			{
				if (Modifier->IsBase())
				{
					return false;
				}
				for (const FBox& Box : Modifier->ComputeBounds())
				{
					if (Box.IsInside(HitLocation))
					{
						return true;
					}
				}
				return false;
			});

		if (!RelevantModifiers.IsEmpty())
		{
			// Sort the modifiers in order of application. The sort we use when applying is based
			//  on modifier descriptors, so we'll convert to those, sort the indices based on the 
			//  descriptors, and then use the indices to get back the modifier components.
			TArray<FModifierDesc> ModifierDescriptors;
			TArray<int32> Indices;
			for (int32 i = 0; i < RelevantModifiers.Num(); ++i)
			{
				Indices.Add(i);
				ModifierDescriptors.Emplace(*RelevantModifiers[i]);
			}

			UMeshPartitionDefinition* Definition = MegaMesh.GetMegaMeshDefinition();
			TConstArrayView<FName> TypePriorities = Definition ? Definition->GetModifierTypePriorities() : TConstArrayView<FName>();
			Indices.Sort([&TypePriorities, &ModifierDescriptors](int32 IndexA, int32 IndexB)
			{
				return MeshPartition::FModifierGroup::ShouldApplyModifierBefore(TypePriorities, ModifierDescriptors[IndexA], ModifierDescriptors[IndexB]);
			});

			FText ModifierSectionName = bAddModifier ? LOCTEXT("ClickModifier_Shift", "(Shift) Click Modifier")
				: bToggleModifier ? LOCTEXT("ClickModifier_Ctrl", "(Ctrl) Click Modifier")
				: LOCTEXT("ClickModifier", "Click Modifier");

			FToolMenuSection& ModifierMenuSection = Menu->AddSection(NAME_None, ModifierSectionName);

			for (int32 Index : Indices)
			{
				AddOwnerToMenu(ModifierMenuSection, RelevantModifiers[Index]);
			}
		}

		FText BaseSectionName = bAddModifier ? LOCTEXT("ClickBaseSection_Shift", "(Shift) Click Base Sections")
			: bToggleModifier ? LOCTEXT("ClickBaseSection_Ctrl", "(Ctrl) Click Base Section")
			: LOCTEXT("ClickBaseSection", "Click Base Section");
		
		FToolMenuSection& BaseSectionMenuSection = Menu->AddSection(NAME_None, BaseSectionName);

		for (const TWeakObjectPtr<MeshPartition::UModifierComponent>& BaseMesh : BaseMeshes)
		{
			if (!BaseMesh.IsValid())
			{
				continue;
			}

			AddOwnerToMenu(BaseSectionMenuSection, BaseMesh.Get());
		}

		FText PreviewSectionName = bAddModifier ? LOCTEXT("ClickPreviewSection_Shift", "(Shift) Click Preview Sections")
			: bToggleModifier ? LOCTEXT("ClickPreviewSection_Ctrl", "(Ctrl) Click Preview Section")
			: LOCTEXT("ClickPreviewSection", "Click Preview Section");
		FToolMenuSection& PreviewSectionMenuSection = Menu->AddSection(NAME_None, PreviewSectionName);

		AddOwnerToMenu(PreviewSectionMenuSection, PreviewSectionComponent);
		
		FSlateApplication::Get().PushMenu(
			ViewportWidget.ToSharedRef(),
			FWidgetPath(),
			ToolMenus->GenerateWidget(Menu),
			// Click.GetCursorPos gives us position within the viewport, wherease this wants position on screen.
			//  Easiest option is to just get it from slate.
			FSlateApplication::Get().GetCursorPos(),
			FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));

		return true;
	}

	// Helper to get the component and actor out of the hit proxy
	bool GetPreviewSection(HComponentVisProxy* VisProxy, 
		const UPrimitiveComponent*& ComponentOut, const MeshPartition::APreviewSection*& PreviewActorOut)
	{
		if (!ensure(VisProxy && VisProxy->IsA(MeshPartition::HPreviewProxy::StaticGetType())))
		{
			return false;
		}

		MeshPartition::HPreviewProxy* CastProxy = static_cast<MeshPartition::HPreviewProxy*>(VisProxy);

		ComponentOut = Cast<UPrimitiveComponent>(CastProxy->Component.Get());
		if (!ComponentOut)
		{
			return false;
		}

		// Need the preview actor for finding the contributing modifiers
		PreviewActorOut = Cast<MeshPartition::APreviewSection>(CastProxy->Actor.Get());
		if (!PreviewActorOut)
		{
			return false;
		}

		return true;
	}

	bool HandlePreviewClick(
		const FComponentVisualizer& Visualizer, FEditorViewportClient* ViewportClient, const FViewportClick& Click,
		const UPrimitiveComponent* PreviewSectionComponent, const UMeshPartitionEditorComponent* MeshPartition, 
		const TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>>& BaseMeshes)
	{
		if (!ensure(PreviewSectionComponent && MeshPartition && ViewportClient))
		{
			return false;
		}

		if (Click.GetKey() == EKeys::MiddleMouseButton)
		{
			// We don't do anything with middle mouse button, currently.
			return false;
		}

		// We'll need the selection set to be able to change selection
		UTypedElementSelectionSet* ElementSelectionSet = (ViewportClient && ViewportClient->GetModeTools()) ? 
			ViewportClient->GetModeTools()->GetEditorSelectionSet() : nullptr;
		if (!ElementSelectionSet)
		{
			return false;
		}

		// Get the location we actually clicked in world space by raycasting the preview mesh again
		FHitResult Result;
		bool bHit = const_cast<UPrimitiveComponent*>(PreviewSectionComponent)->LineTraceComponent(Result,
			Click.GetOrigin(),
			Click.GetOrigin() + Click.GetDirection() * HALF_WORLD_MAX,
			FCollisionQueryParams(SCENE_QUERY_STAT(HitTest), true));
		if (!bHit)
		{
			return false;
		}

		const FVector3d& HitLocation = Result.ImpactPoint;

		bool bAddModifier = Click.IsShiftDown();
		bool bToggleModifier = Click.IsControlDown();
		
		if (Click.GetKey() == EKeys::RightMouseButton)
		{
			// Only summon the custom right click menu if Alt is pressed.
			if (!Click.IsAltDown())
			{
				return false;
			}
			return SummonRightClickMenu(Visualizer, ViewportClient, Click, 
				PreviewSectionComponent,
				*ElementSelectionSet, 
				*MeshPartition, BaseMeshes, HitLocation, bAddModifier, bToggleModifier);
		}

		// See what base section is closest to the hit location. Currently we just pick the base section
		//  whose bounds are closest to the hit. Note that we can't just look for containment because the
		//  base section might be flat, whereas you may be clicking a point that was pulled upward by
		//  a modifier.
		AActor* HitBaseMeshActor = nullptr;
		double ClosestDistSquared = TNumericLimits<double>::Max();
		for (const TWeakObjectPtr<MeshPartition::UModifierComponent>& BaseMesh : BaseMeshes)
		{
			if (!BaseMesh.IsValid())
			{
				continue;
			}
			AActor* OwnerActor = BaseMesh->GetOwner();
			if (!OwnerActor)
			{
				continue;
			}
			for (const FBox& Box : BaseMesh->ComputeBounds())
			{
				double DistSquared = Box.ComputeSquaredDistanceToPoint(Result.ImpactPoint);
				if (DistSquared < ClosestDistSquared)
				{
					HitBaseMeshActor = OwnerActor;
					ClosestDistSquared = DistSquared;
				}
			}
		}

		if (!HitBaseMeshActor)
		{
			return false;
		}

		// Here we have a base mesh to add/remove to/from our selection. 
		PerformSelection(*ElementSelectionSet, { HitBaseMeshActor }, bAddModifier, bToggleModifier);
		return true;
	}
}

bool FMeshPreviewVisualizer::VisProxyHandleClick(
	FEditorViewportClient* ViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	using namespace MegaMeshPreviewVisualizersLocals;

	// Get the preview actor here because we need to be able to access its internals while we're inside the friended visualizer.
	const UPrimitiveComponent* Component = nullptr;
	const MeshPartition::APreviewSection* PreviewActor = nullptr;
	if (!GetPreviewSection(VisProxy, Component, PreviewActor))
	{
		return false;
	}
	// Check mesh partition mesh validity
	if (!PreviewActor->Parent.IsValid())
	{
		return false;
	}

	return HandlePreviewClick(*this, ViewportClient, Click,
		Component,
		Cast<UMeshPartitionEditorComponent>(PreviewActor->Parent->GetMeshPartitionComponent()),
		PreviewActor->BaseModifiers);
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
