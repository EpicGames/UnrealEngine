// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"

namespace UE::MeshPartition
{
	/**
	 * Visualizer intended to handle clicks on mega mesh preview components, to route the selection to
	 *  the corresponding base section.
	 */
	class FMeshPreviewVisualizer : public FComponentVisualizer
	{
	public:
		virtual bool VisProxyHandleClick(FEditorViewportClient* ViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click);
		virtual bool ShouldAutoSelectElementOnHandleClick() const { return false; } // we handle selection in VisProxyHandleClick
	};

} // namespace UE::MeshPartition