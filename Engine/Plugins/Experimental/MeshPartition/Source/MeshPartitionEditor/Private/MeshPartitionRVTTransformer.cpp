// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionRVTTransformer.h"

#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionPreviewComponents.h"
#include "VT/RuntimeVirtualTexture.h"

namespace UE::MeshPartition
{

namespace MeshPartitionRVTTransformerLocals
{
} // namespace MeshPartitionRVTTransformerLocals

bool FRVTTransformer::Execute(MeshPartition::UPreviewMeshComponent& InPreviewMeshComponent, const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InBuildVariant) const
{
	InPreviewMeshComponent.RuntimeVirtualTextures = RuntimeVirtualTextures;
	InPreviewMeshComponent.MarkRenderStateDirty();

	return true;
}

bool FRVTTransformer::Execute(MeshPartition::FTransformerContext& InTransformerContext) const
{
	UE::Tasks::FTask SetRuntimeVirtualTexturesTask = UE::Tasks::Launch(TEXT("FRVTTransformer::SetRuntimeVirtualTextures"), [this,
		&InTransformerContext]() mutable
		{
			SetRuntimeVirtualTextures(InTransformerContext);
		},
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);

	SetRuntimeVirtualTexturesTask.Wait();

	return true;
}

void FRVTTransformer::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	for (URuntimeVirtualTexture* RuntimeVirtualTexture : RuntimeVirtualTextures)
	{
		if (RuntimeVirtualTexture == nullptr)
		{
			continue;
		}

		// We are just assigning the Virtual Textures, not doing something based on their content.
		InDependencies += RuntimeVirtualTexture->GetPathName();
	}
}

void FRVTTransformer::SetRuntimeVirtualTextures(const MeshPartition::FTransformerContext& InTransformerContext) const
{
	TSet<AActor*> Sections;
	
	for (const MeshPartition::FTransformerUnit& TransformerUnit : InTransformerContext.TransformerUnits)
	{
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if (Section == nullptr)
		{
			continue;
		}

		Sections.Emplace(Section);
	}
	
	for (AActor* Section : Sections)
	{
		if (MeshPartition::APreviewSection* PreviewSection = Cast<MeshPartition::APreviewSection>(Section))
		{
			PreviewSection->SetRuntimeVirtualTextures(RuntimeVirtualTextures);
		}
		else if (MeshPartition::ACompiledSection* CompiledSection = Cast<MeshPartition::ACompiledSection>(Section))
		{
			CompiledSection->SetRuntimeVirtualTextures(RuntimeVirtualTextures);
		}
		else
		{
			checkf(false, TEXT("This should never happen. Transformers should only be applied on Preview or Compiled sections at the moment."));
		}
	}
}

} // namespace UE::MeshPartition
