// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionRectangleGenerator.h"
#include "ActorFactories/ActorFactory.h"
#include "Builders/CubeBuilder.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Generators/MeshShapeGenerator.h"
#include "HAL/FileManager.h"
#include "LocationVolume.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionEditorUtils.h"
#include "Misc/ScopedSlowTask.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "WorldPartition/WorldPartition.h"
#include "UObject/SavePackage.h"

#define LOCTEXT_NAMESPACE "MegaMeshRectangle"

namespace UE::MeshPartition
{

ALocationVolume* FRectangleGeneratorUtils::CreateLocationVolume(UWorld* const World, AMeshPartition* const MegaMesh, const FString& Label, const FBox& Bounds)
{
	checkf(World, TEXT("World must not be nullptr."));

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FRectangleGeneratorUtils::CreateLocationVolume);


	FActorSpawnParameters SpawnParameters;
	SpawnParameters.Name = *Label;
	SpawnParameters.NameMode = FActorSpawnParameters::ESpawnActorNameMode::Requested;

	ALocationVolume* LocationVolume = World->SpawnActor<ALocationVolume>(Bounds.GetCenter(), {}, SpawnParameters);
	if (LocationVolume)
	{
		LocationVolume->SetActorLabel(Label);
		LocationVolume->GetRootComponent()->SetMobility(EComponentMobility::Movable);
		LocationVolume->AttachToActor(MegaMesh, FAttachmentTransformRules::KeepWorldTransform);
		LocationVolume->SetActorScale3D(Bounds.GetSize());
		LocationVolume->Tags.Add(FName("MeshTerrainLocationVolume"));

		UCubeBuilder* Builder = NewObject<UCubeBuilder>();
		Builder->X = 1.0f;
		Builder->Y = 1.0f;
		Builder->Z = 1.0f;
		UActorFactory::CreateBrushForVolumeActor(LocationVolume, Builder);
	}

	return LocationVolume;
}


void FRectangleGeneratorUtils::SpawnLocationVolumes(const FVector& MeshSize, const FVector2d& MeshWorldOffset, const FInt32Point& LocationVolumesResolution, UWorld* const World, AMeshPartition* const MegaMesh, FSlowTask* SlowTask)
{

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FRectangleGeneratorUtils::SpawnLocationVolumes);

	

	// find a single number that we can append to each label to avoid duplicating any existing volume name on this mesh partition .
	int32 SuffixInt = [MegaMesh, &LocationVolumesResolution] {
			// collect existing ActorLabels
			TSet<FString> ActorLabels;
			{
				TArray<AActor*> AttachedActors;
				MegaMesh->GetAttachedActors(AttachedActors);
				ActorLabels.Reserve(AttachedActors.Num());
				for (AActor* Actor : AttachedActors)
				{
					ActorLabels.Add(Actor->GetActorLabel());
				}
			}
			int32 Disambiguation = 0;
			bool bFoundDuplicate;
			do
			{ 
				bFoundDuplicate = false;
				for (int32 y = 0; y < LocationVolumesResolution.Y; ++y)
				{
					for (int32 x = 0; x < LocationVolumesResolution.X; ++x)
					{
						const FString SuggestedLabel = FString::Format(TEXT("LocationVolume_X{0}-Y{1}_{2}"), { x, y, Disambiguation });
						if (ActorLabels.Contains(SuggestedLabel))
						{
							bFoundDuplicate = true;
							break;
						}
					}
					if (bFoundDuplicate) break;
				}
				
				if (bFoundDuplicate)
				{
					Disambiguation++;
				}
			} while(bFoundDuplicate);

			return Disambiguation;
		}();


	const FVector3d VolumeScale = {
			MeshSize.X / FMath::Max(LocationVolumesResolution.X, 1), 
			MeshSize.Y / FMath::Max(LocationVolumesResolution.Y, 1), 
			MeshSize.Z
	};

	for (int32 VolumeY = 0; VolumeY < LocationVolumesResolution.Y; ++VolumeY)
	{
		for (int32 VolumeX = 0; VolumeX < LocationVolumesResolution.X; ++VolumeX)
		{
			if (SlowTask) SlowTask->TickProgress();

			const FVector3d BoundsMin = { MeshWorldOffset.X + VolumeScale.X * VolumeX, MeshWorldOffset.Y + VolumeScale.Y * VolumeY, 0.0 };
			const FVector3d BoundsMax = { MeshWorldOffset.X + VolumeScale.X * (VolumeX + 1), MeshWorldOffset.Y + VolumeScale.Y * (VolumeY + 1), VolumeScale.Z };

			const FString& Label = FString::Format(TEXT("LocationVolume_X{0}-Y{1}_{2}"), { VolumeX, VolumeY, SuffixInt });

			FRectangleGeneratorUtils::CreateLocationVolume(World, MegaMesh, Label, { BoundsMin, BoundsMax });
		}
	}
}

// this treats MeshResolution as Quads.
FInt32Point FRectangleGeneratorUtils::ComputeSectionResolution(const FInt32Point MeshResolution, const int32 MaxTrianglesPerSection)
{
	FInt32Point Resolution(0);

	if (MeshResolution.X > 0 && MeshResolution.Y > 0 && MaxTrianglesPerSection > 0)
	{
		const int32 MaxQuadsPerSection = FMath::Max(FMath::DivideAndRoundDown(MaxTrianglesPerSection, 2), 1);
		const int32 SquareQuadsPerSectionEdge = FMath::Sqrt(static_cast<float>(MaxQuadsPerSection));

		const int32 ShortEdgeIndex = MeshResolution[0] <= MeshResolution[1] ? 0 : 1;
		const int32 ShortEdgeNumSections = FMath::DivideAndRoundUp(MeshResolution[ShortEdgeIndex], SquareQuadsPerSectionEdge);
		const int32 ShortEdgeQuadsPerSection = FMath::DivideAndRoundUp(MeshResolution[ShortEdgeIndex], ShortEdgeNumSections);
		const int32 LongEdgeQuadsPerSection = FMath::Max(MaxQuadsPerSection / ShortEdgeQuadsPerSection, 1); 

		const int32 LongEdgeNumSections = FMath::DivideAndRoundUp(MeshResolution[1 - ShortEdgeIndex], LongEdgeQuadsPerSection);

		Resolution = ShortEdgeIndex == 0 ? FInt32Point(ShortEdgeNumSections, LongEdgeNumSections) : FInt32Point(LongEdgeNumSections, ShortEdgeNumSections);
	}

	return Resolution;
}

TArray<FRectangleGeneratorUtils::FSectionInfo> FRectangleGeneratorUtils::ComputeSectionInfos(const FInt32Point MeshResolution, const FInt32Point SectionsResolution)
{

	const int32 NumSections = SectionsResolution.X * SectionsResolution.Y;

	TArray<FSectionInfo> SectionInfos;
	SectionInfos.Reserve(NumSections);

	if (SectionsResolution.X >0 && SectionsResolution.Y > 0 && MeshResolution.X > 0 && MeshResolution.Y > 0)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FRectangleGeneratorUtils::Create_ComputeSectionInfos);

		const FInt32Point MaxSectionResolution = FMath::DivideAndRoundUp(MeshResolution, SectionsResolution);
		check(MaxSectionResolution.X > 0 && MaxSectionResolution.Y > 0);

		FInt32Point SectionStart = { 0 };
		FInt32Point RemainingMeshResolution = MeshResolution;
		FInt32Point SectionResolution = { 0 };

		for (int32 SectionY = 0; SectionY < SectionsResolution.Y; ++SectionY)
		{
			SectionResolution.Y = FMath::Min(MaxSectionResolution.Y, RemainingMeshResolution.Y);
			RemainingMeshResolution.Y -= SectionResolution.Y;
			RemainingMeshResolution.X = MeshResolution.X;

			SectionStart.X = 0;

			for (int32 SectionX = 0; SectionX < SectionsResolution.X; ++SectionX)
			{
				SectionResolution.X = FMath::Min(MaxSectionResolution.X, RemainingMeshResolution.X);
				RemainingMeshResolution.X -= SectionResolution.X;

				const FInt32Point SectionEnd = SectionStart + SectionResolution;

				const FVector2d MinUV = { static_cast<double>(SectionStart.X) / MeshResolution.X, static_cast<double>(SectionStart.Y) / MeshResolution.Y };
				const FVector2d MaxUV = { static_cast<double>(SectionEnd.X) / MeshResolution.X, static_cast<double>(SectionEnd.Y) / MeshResolution.Y };

				SectionInfos.Add({ {SectionX, SectionY}, SectionResolution, MinUV, MaxUV });
				SectionStart.X += SectionResolution.X;
			}
			SectionStart.Y += SectionResolution.Y;
		}

#if UE_BUILD_DEBUG
		int64 NumQuads = 0;
		for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
		{
			const FSectionInfo& SectionInfo = SectionInfos[SectionIndex];
			NumQuads += SectionInfo.Resolution.X * SectionInfo.Resolution.Y;
		}
		ensure(NumQuads == MeshResolution.X * MeshResolution.Y);
#endif
	}

	return MoveTemp(SectionInfos);
}

TArray<MeshPartition::UMeshProviderModifier*> FRectangleGeneratorUtils::SpawnBaseModifiers(const TArray<FSectionInfo>& SectionInfos, const FVector& MeshSize, const FVector2d& MeshWorldOffset, const FTransform& ToWorld, const bool bRegisterModifers, UWorldPartition* WorldPartition, UMeshPartitionEditorComponent* MeshPartitionEditorComponent, FSlowTask* SlowTask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FRectangleGeneratorUtils::SpawnBaseModifiers);

	TArray<MeshPartition::UMeshProviderModifier*> Providers;
	Providers.Reserve(SectionInfos.Num());

	// find a single number that we can append to each label to avoid duplicating any existing volume name on this megamesh.
	const int32 SuffixInt = [MeshPartitionEditorComponent, &SectionInfos] {
		// collect existing ActorLabels
		TSet<FString> ActorLabels;
		{
			AMeshPartition* MegaMesh = Cast<AMeshPartition>(MeshPartitionEditorComponent->GetOwner());
			TArray<AActor*> AttachedActors;
			MegaMesh->GetAttachedActors(AttachedActors);
			ActorLabels.Reserve(AttachedActors.Num());
			for (AActor* Actor : AttachedActors)
			{
				ActorLabels.Add(Actor->GetActorLabel());
			}
		}
	
		int32 Disambiguation = 0;
		bool bFoundDuplicate;
		do
		{
			bFoundDuplicate = false;
			for (const FSectionInfo& SectionInfo : SectionInfos)
			{
				const FString SuggestedLabel = FString::Format(TEXT("Section_X{0}-Y{1}_{2}"), { SectionInfo.IndexXY.X, SectionInfo.IndexXY.Y,  Disambiguation });
				if (ActorLabels.Contains(SuggestedLabel))
				{
					bFoundDuplicate = true;
					break;
				}
			}

			if (bFoundDuplicate)
			{
				Disambiguation++;
			}
		} while (bFoundDuplicate);

		return Disambiguation;
		}();


	for (const FSectionInfo& SectionInfo : SectionInfos)
	{
		if (SlowTask) SlowTask->TickProgress();


		const FVector2d SectionCenterUV = (SectionInfo.MinUV + SectionInfo.MaxUV) * 0.5;
		const FVector2d SectionCenter = MeshWorldOffset + SectionCenterUV * FVector2d(MeshSize);

		const FTransform PivotTransform = FTransform(FVector(SectionCenter.X, SectionCenter.Y, 0.0));
		const FTransform SectionToWorld = PivotTransform * ToWorld;
		AActor* BaseModifier = MeshPartitionEditorComponent->SpawnBaseModifier({}, {}, SectionToWorld, bRegisterModifers);

		const FString ActorLabel = FString::Format(TEXT("Section_X{0}-Y{1}_{2}"), { SectionInfo.IndexXY.X, SectionInfo.IndexXY.Y, SuffixInt });
		BaseModifier->SetActorLabel(ActorLabel); // NOTE: this causes re-registration as a side effect

		MeshPartition::UMeshProviderModifier* MeshProviderComponent = BaseModifier->FindComponentByClass<MeshPartition::UMeshProviderModifier>();
		Providers.Add(MeshProviderComponent);

		// tell the preview section builder to ignore this while we build it -- we will clear at the end
		MeshProviderComponent->SetIgnoreChanged(true);
	}

	return MoveTemp(Providers);
}

void FRectangleGeneratorUtils::GenerateSectionMesh(FDynamicMesh3& Mesh, const FSectionInfo& SectionInfo, const FVector& MeshSize, TFunction<double(FVector2f) >& GetHeight)
{
	Mesh.EnableVertexUVs(FVector2f(0, 0));
	Mesh.EnableAttributes();
	Geometry::FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->PrimaryUV();

	const FVector2d SectionScale = (SectionInfo.MaxUV - SectionInfo.MinUV) * FVector2d{ MeshSize };

	const FVector2d V00 = { -SectionScale.X / 2.f, -SectionScale.Y / 2.f };
	const FVector2d V01 = { SectionScale.X / 2.f, -SectionScale.Y / 2.f };
	const FVector2d V11 = { SectionScale.X / 2.f, SectionScale.Y / 2.f };
	const FVector2d V10 = { -SectionScale.X / 2.f, SectionScale.Y / 2.f };

	{
		// @todo convert these from Begin/EndUnsafe inserts to just use Append() since all the inserts are sequential anyhow 

		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FRectangleGeneratorUtils::GenerateSectionMesh_BuildSectionMesh_AppendVertices);

		const FVector2f UV00(SectionInfo.MinUV);
		const FVector2f UV01(SectionInfo.MinUV.X, SectionInfo.MaxUV.Y);
		const FVector2f UV11(SectionInfo.MaxUV);
		const FVector2f UV10(SectionInfo.MaxUV.X, SectionInfo.MinUV.Y);

		
		Mesh.BeginUnsafeVerticesInsert();
		int32 VertexId = 0;
		for (int32 IndexY = 0; IndexY <= SectionInfo.Resolution.Y; ++IndexY)
		{
			// by construction SectionInfo.Resolution.Y > 0 to reach here.
			const double Yt = static_cast<double>(IndexY) / static_cast<double>(SectionInfo.Resolution.Y);
			for (int32 IndexX = 0; IndexX <= SectionInfo.Resolution.X; ++IndexX)
			{
				// by construction SectionInfo.Resolution.X > 0 to reach here.
				const double Xt = static_cast<double>(IndexX) / static_cast<double>(SectionInfo.Resolution.X);

				const FVector2d VertexXY = Geometry::FMeshShapeGenerator::BilinearInterp(V00, V01, V11, V10, Xt, Yt);
				const FVector2f UV = Geometry::FMeshShapeGenerator::BilinearInterp(UV00, UV10, UV11, UV01, Xt, Yt);
				const double Height = (GetHeight) ? GetHeight(UV) : 0.;

				Geometry::FVertexInfo VertexInfo({ VertexXY.X, VertexXY.Y, Height * MeshSize.Z });
				Mesh.InsertVertex(VertexId, VertexInfo, true);
				UVOverlay->AppendElement(UV);
				Mesh.SetVertexUV(VertexId++, UV);
			}
		}
		Mesh.EndUnsafeVerticesInsert();
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FRectangleGeneratorUtils::GenerateSectionMesh_BuildSectionMesh_AppendTriangles);
		Mesh.BeginUnsafeTrianglesInsert();
		int32 TriangleId = 0;
		for (int32 IndexY = 0; IndexY < SectionInfo.Resolution.Y; ++IndexY)
		{
			for (int32 IndexX = 0; IndexX < SectionInfo.Resolution.X; ++IndexX)
			{
				const int32 Quad00 = IndexY * (SectionInfo.Resolution.X + 1) + IndexX;
				const int32 Quad10 = (IndexY + 1) * (SectionInfo.Resolution.X + 1) + IndexX;
				const int32 Quad01 = Quad00 + 1;
				const int32 Quad11 = Quad10 + 1;
				const Geometry::FIndex3i Triangle0(Quad00, Quad11, Quad01);
				const Geometry::FIndex3i Triangle1(Quad00, Quad10, Quad11);

				Mesh.InsertTriangle(TriangleId, Triangle0, 0, true);
				UVOverlay->SetTriangle(TriangleId++, Geometry::FIndex3i(Quad00, Quad11, Quad01));

				Mesh.InsertTriangle(TriangleId, Triangle1, 0, true);
				UVOverlay->SetTriangle(TriangleId++, Geometry::FIndex3i(Quad00, Quad10, Quad11));
			}
		}
		Mesh.EndUnsafeTrianglesInsert();
	}

	Geometry::FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->GetNormalLayer(0);
	Geometry::FMeshNormals::InitializeOverlayToPerVertexNormals(NormalOverlay);
}


void  FMegaMeshGeneratorBase::SaveActorPackages(const TArray<TWeakObjectPtr<AActor>>& InActorsToSave)
{
	if (InActorsToSave.IsEmpty())
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(FMegaMeshGeneratorBase::SaveActorPackages);

	FSavePackageArgs SaveArgs;
	SaveArgs.TopLevelFlags = RF_Standalone;

	TArray<FPackageSaveInfo> PackageSaveInfos;
	TArray<FSavePackageResultStruct> SaveResults;
	PackageSaveInfos.Reserve(InActorsToSave.Num());
	SaveResults.Reserve(InActorsToSave.Num());

	for (const TWeakObjectPtr<AActor>& Actor : InActorsToSave)
	{
		if (!Actor.IsValid()) continue;

		UPackage* PackageToSave = Actor->GetPackage();
		FPackageSaveInfo SaveInfo;
		SaveInfo.Asset = nullptr;
		SaveInfo.Package = PackageToSave;
		SaveInfo.Filename = SourceControlHelpers::PackageFilename(PackageToSave);
		PackageSaveInfos.Add(SaveInfo);
	}

	UPackage::SaveConcurrent(PackageSaveInfos, SaveArgs, SaveResults);

	for (const TWeakObjectPtr<AActor>& Actor : InActorsToSave)
	{
		if (!Actor.IsValid()) continue;
		SavedPackages.Add(SourceControlHelpers::PackageFilename(Actor->GetPackage()));
	}
}

void  FMegaMeshGeneratorBase::UnloadActors(const TArray<TWeakObjectPtr<AActor>>& InActorsToUnload, UWorld* World) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FMegaMeshGeneratorBase::UnloadActors);

	if (!World) return;

	UWorldPartition* WorldPartition = World->GetWorldPartition();

	if (!WorldPartition) return;

	// just make sure the object has no standalone flags, then it should get garbage collected
	for (const TWeakObjectPtr<AActor>& Actor : InActorsToUnload)
	{
		
		if (!Actor.IsValid())
		{
			continue; // already unloaded
		}

		// unpin if necessary
		if (!ensure(!WorldPartition->IsActorPinned(Actor->GetActorGuid())))
		{
			WorldPartition->UnpinActors({ Actor->GetActorGuid() });
		}

		ForEachObjectWithPackage(Actor->GetPackage(), [](UObject* Object)
			{
				Object->ClearFlags(RF_Standalone);
				return true;
			});
	}

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);

	// Double check that they were actually unloaded
	EditorUtils::ValidateObjectsAreUnloaded(InActorsToUnload);
}

void  FMegaMeshGeneratorBase::CancelAndWait(UWorld* World)
{
	if (!World) return;

	TRACE_CPUPROFILER_EVENT_SCOPE(FMegaMeshGeneratorBase::CancelAndWait);

	bIsCancelled = true;

	// Outstanding tasks may try to reference the mesh provider components directly, so we have to wait here until all outstanding tasks join before we can clean up any uobjects:
	for (const TPair<int, Tasks::FTask>& TaskInFlight : MeshTasks)
	{
		[[maybe_unused]] bool bCompleted = TaskInFlight.Value.Wait();
	}

	// Destroy any mesh providers they haven't already been saved
	{
		// Using TWeakObjectPtr so that we can double check that Actors were unloaded after GC
		TArray<TWeakObjectPtr<AActor>> ActorsToUnload;
		ActorsToUnload.Reserve(MegaMeshMeshProviders.Num());

		Algo::TransformIf(MegaMeshMeshProviders, ActorsToUnload,
			[](const MeshPartition::UMeshProviderModifier* MeshProvider)
			{
				return MeshProvider != nullptr;
			},
			[](const MeshPartition::UMeshProviderModifier* MeshProvider)
			{
				return MeshProvider->GetOwner();
			});

		UnloadActors(ActorsToUnload, World);
	}

	// Destroy the parent mega mesh actor after all loaded sections are destroyed and GC'd
	if (MegaMesh)
	{
		// It's important here to destroy -> save -> GC in order for WP to pick up that this is now an empty package and remove the actor desc:
		// (note this is different from what UnloadActors does) 
		World->DestroyActor(MegaMesh);

		FSavePackageArgs SaveArgs;
		SaveArgs.TopLevelFlags = RF_Standalone;
		UPackage::Save(MegaMesh->GetPackage(), nullptr, *USourceControlHelpers::PackageFilename(MegaMesh->GetPackage()), SaveArgs);

		ForEachObjectWithPackage(MegaMesh->GetPackage(), [](UObject* Object)
			{
				Object->ClearFlags(RF_Standalone);
				return true;
			});

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);
	}

	// Finally, delete all the saved OFPA actor files directly off the disk so we don't need to load them in the editor which would be a very slow operation
	for (FString SavedPackage : SavedPackages)
	{
		IFileManager::Get().Delete(*SavedPackage, false, true);
	}

	SavedPackages.Empty();
}

Tasks::FTask FMegaMeshRectangleGenerator::LaunchBuildSectionTask(const FSectionInfo& SectionInfo, FDynamicMesh3& Mesh)
{
	return Tasks::Launch(TEXT("Build Section Mesh"),
		[&bIsCancelled = bIsCancelled,
		&Mesh, &SectionInfo, MeshSize = Params.MeshSize]
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMegaMeshRectangleGenerator::LaunchBuildSectionMesh);

			TFunction<double(FVector2f) > GetHeight(nullptr);
			if (bIsCancelled)
			{
				return;
			}
			FVector MeshSize3d(MeshSize, 0.);
			FRectangleGeneratorUtils::GenerateSectionMesh(Mesh, SectionInfo, MeshSize3d, GetHeight);

		});
}

bool FMegaMeshRectangleGenerator::Generate(FPackageSourceControlHelper* InSourceControlHelper)
{
	const FVector MeshSize3d(Params.MeshSize, 0.);

	const int32 NumSections = Params.SectionsResolution.X * Params.SectionsResolution.Y;
	const int32 NumLocationVolumes = Params.LocationVolumesResolution.X * Params.LocationVolumesResolution.Y;

	FScopedSlowTask SlowTask(NumSections + NumLocationVolumes, LOCTEXT("RectangleCreate_SlowTask", "Creating sections..."));
	SlowTask.MakeDialog(true);

	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;
	// if needed, spawn and save a new MegaMesh actor 
	if (!ExistingMegaMesh)
	{
		MegaMesh = World->SpawnActor<AMeshPartition>(AMeshPartition::StaticClass(), FTransform::Identity);
		MeshPartitionEditorComponent = NewObject<UMeshPartitionEditorComponent>(MegaMesh, UMeshPartitionEditorComponent::StaticClass(), TEXT("MegaMeshEditorComponent"), RF_Transactional);
		MegaMesh->SetMeshPartitionComponent(MeshPartitionEditorComponent);
		MeshPartitionEditorComponent->RegisterComponent();
		
		if (Params.bSaveAndUnload)
		{
			SaveActorPackages({ MegaMesh });
		}
	}
	else
	{
		MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(ExistingMegaMesh->GetMeshPartitionComponent());
		if (!MeshPartitionEditorComponent) return false;
	}

	// Upper bound on how many bytes each section will require to avoid allocating too much memory at once
	const int32 MaxTasksInFlight = [&MeshResolution = Params.MeshResolution, NumSections]{
		constexpr int32 EstimatedBytesPerVertex = 350;
		constexpr float MemoryMargin = 0.1f;
		const int32 EstimatedBytesPerSection = EstimatedBytesPerVertex * MeshResolution.X * MeshResolution.Y / NumSections;
		const uint64 AvailablePhysicalMemory = FPlatformMemory::GetStats().AvailablePhysical * (1. - MemoryMargin);
		return  FMath::Max<int32>(AvailablePhysicalMemory / EstimatedBytesPerSection, 1);
	}();
	

	MeshTasks.Reserve(MaxTasksInFlight);
	SavedPackages.Reserve(SavedPackages.Num() + NumSections);


	// compute the partition of the MegaMesh into sections in the form of section info
	const TArray<FRectangleGeneratorUtils::FSectionInfo> SectionInfos = FRectangleGeneratorUtils::ComputeSectionInfos(Params.MeshResolution, Params.SectionsResolution);
	

	// center the entire generated mega mesh around the local origin
	const FVector2d MeshWorldOffset =  -Params.MeshSize  / 2.0;

	// disable non-dirty tracking (which normally will try to Pin newly created non-dirty actors)
	// - if we are saving and unloading, we don't want them to pin
	UWorldPartition* WorldPartition = World->GetWorldPartition();
	UWorldPartition::FDisableNonDirtyActorTrackingScope DisableNonDirtyActorTrackingScope(WorldPartition, Params.bSaveAndUnload);

	// Spawn all the base modifiers ahead of time so we can operate directly on their internal FDynamicMesh to avoid copies.
	// if we are unloading, don't bother registering
	const bool bRegisterModifiers = !Params.bSaveAndUnload;
	const FTransform XForm = (ExistingMegaMesh) ? ToWorld : FTransform();
	MegaMeshMeshProviders = FRectangleGeneratorUtils::SpawnBaseModifiers(SectionInfos, MeshSize3d, MeshWorldOffset, XForm, bRegisterModifiers, WorldPartition, MeshPartitionEditorComponent, &SlowTask);

	
	// Spawn location volumes if requested
	if (NumLocationVolumes > 0)
	{
		FRectangleGeneratorUtils::SpawnLocationVolumes(MeshSize3d, MeshWorldOffset, Params.LocationVolumesResolution, World, MegaMesh, &SlowTask);
	}

	
	if (SlowTask.ShouldCancel())
	{
		CancelAndWait(World);
		return false;
	}

	// When this many base modifier actors have finished being built, we save them all simultaneously and then unload them.
	constexpr uint32 NumActorsBeforeSave = 32;

	TArray<TWeakObjectPtr<AActor>> ActorsToSave;
	ActorsToSave.Reserve(NumActorsBeforeSave);

	Meshes.SetNum(NumSections);

	for (int32 SectionIndex = 0; SectionIndex < NumSections; ++SectionIndex)
	{
		if (SlowTask.ShouldCancel())
		{
			CancelAndWait(World);
			return false;
		}

		const FRectangleGeneratorUtils::FSectionInfo& SectionInfo = SectionInfos[SectionIndex];
		FDynamicMesh3& SectionMesh = Meshes[SectionIndex];

		// launch task
		Tasks::FTask BuildSectionMeshTask = LaunchBuildSectionTask(SectionInfo, SectionMesh);
		
		MeshTasks.Add({ SectionIndex, BuildSectionMeshTask });

		const bool bHasMoreWork = SectionIndex != NumSections - 1;
		// If we've reached the max concurrency, start processing results to keep memory consumption down:
		while ((bHasMoreWork && MeshTasks.Num() > MaxTasksInFlight) || (!bHasMoreWork && MeshTasks.Num() > 0))
		{
			// process any completed mesh tasks
			for (auto It = MeshTasks.CreateIterator(); It; ++It)
			{
				if (!It->Value.IsCompleted())
				{
					continue;
				}

				const int32 ProviderIndex = It->Key;
				MeshPartition::UMeshProviderModifier* MeshProvider = MegaMeshMeshProviders[ProviderIndex];
				MeshProvider->SetMesh(MoveTemp(Meshes[ProviderIndex]));

				ActorsToSave.Add(MeshProvider->GetOwner());

				// remove from the mesh tasks list
				It.RemoveCurrentSwap();

				// Stop popping completed tasks and launch more when we go below this percentage of MaxTasksInFlight:
				constexpr float TaskFillPercent = 0.75;
				if (bHasMoreWork && MeshTasks.Num() < MaxTasksInFlight * TaskFillPercent)
				{
					break;
				}
			}
		}

		if (!bHasMoreWork || ActorsToSave.Num() >= NumActorsBeforeSave)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FMegaMeshRectangleGenerator::Create_SaveGeneratedSections);
			SlowTask.EnterProgressFrame(ActorsToSave.Num());

			if (Params.bSaveAndUnload)
			{
				SaveActorPackages(ActorsToSave);
				UnloadActors(ActorsToSave, World);
			}

			ActorsToSave.Reset();
		}
	}

	// if we created this MegaMesh, set the actor transform to reflect user intent
	if (MegaMesh)
	{
		MegaMesh->SetActorTransform(ToWorld);
	}

	if (InSourceControlHelper && Params.bSaveAndUnload)
	{
		[[maybe_unused]] bool bAddedToSourceControl = InSourceControlHelper->AddToSourceControl(SavedPackages);
	}

	// If save and unload is disabled, the ignore changed flag must be reset and bounds must be updated to trigger preview mesh generation:
	if (!Params.bSaveAndUnload)
	{
		TArray<FBox> Bounds;
		for (MeshPartition::UMeshProviderModifier* MeshProvider : MegaMeshMeshProviders)
		{
			MeshProvider->SetIgnoreChanged(false);
			Bounds.Append(MeshProvider->ComputeBounds());
		}
		MeshPartitionEditorComponent->OnBoundsChanged(Bounds, UE::MeshPartition::EChangeType::StateChange);
	}

	

	return true;
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
