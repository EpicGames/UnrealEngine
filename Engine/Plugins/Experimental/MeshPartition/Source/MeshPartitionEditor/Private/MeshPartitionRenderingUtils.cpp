// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionRenderingUtils.h"

#include "Engine/Engine.h"
#include "EngineGlobals.h"
#include "IRenderCaptureProvider.h"
#include "HAL/IConsoleManager.h"

namespace UE::MeshPartition
{

FRenderCaptureManager& FRenderCaptureManager::Get()
{
	static FRenderCaptureManager Instance;
	static FAutoConsoleCommand CCmdRenderDocCaptureFrame = FAutoConsoleCommand(
		TEXT("MeshPartition.Channels.CaptureNextBatch"),
		TEXT("Captures the next Job Batch and launches RenderDoc"),
		FConsoleCommandDelegate::CreateLambda([]()
			{
				FRenderCaptureManager::CaptureNextBatch();
			}));

	return Instance;
}

void FRenderCaptureManager::CaptureNextBatch()
{
	FRenderCaptureManager::Get().NextCaptureCounter = 1;
}

void FRenderCaptureManager::BeginCapture(FRDGBuilder& InGraphBuilder)
{
	if (FRenderCaptureManager::Get().NextCaptureCounter > 0)
	{
		FRenderCaptureManager::Get().CurrentCapture = MakeUnique<RenderCaptureInterface::FScopedCapture>(true, InGraphBuilder, *FString("MeshPartition"));
		FRenderCaptureManager::Get().NextCaptureCounter = 0;
	}
}

void FRenderCaptureManager::EndCapture()
{
	if (FRenderCaptureManager::Get().CurrentCapture)
	{
		FRenderCaptureManager::Get().CurrentCapture.Reset();
	}
}

} // namespace UE::MeshPartition