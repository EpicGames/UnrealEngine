// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionRuntimeCellTransformer.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionModifierActor.h"
#include "MeshPartitionDependencyContext.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionWorldUpdater.h"
#include "MeshPartition.h"

#if WITH_EDITOR
#include "CoreMinimal.h"
#include "Editor.h"
#include "GameFramework/Actor.h"
#include "Engine/Level.h"
#include "Engine/StaticMesh.h"
#include "WorldPartition/WorldPartitionRuntimeCellInterface.h"
#include "EngineUtils.h"
#include "MeshPartitionEditorUtils.h"
#include "AssetRegistry/AssetRegistryModule.h"
#endif

#define LOCTEXT_NAMESPACE "MegaMeshRuntimeCellTransformer"

#define MEGAMESH_DEBUG_LOG(...) UE_LOG(LogMegaMeshEditor, Verbose, __VA_ARGS__)

namespace UE::MeshPartition
{
TAutoConsoleVariable<bool> CVarMeshPartitionPIEStripOutOfDate(TEXT("MeshPartition.PIE.StripOutOfDateSections"),
	true,
	TEXT("Whether to remove out-of-date compiled sections when playing in editor."));

URuntimeCellTransformer::URuntimeCellTransformer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

#if WITH_EDITOR

void URuntimeCellTransformer::PreTransform(ULevel* InLevel)
{
	check(InLevel);

	const IWorldPartitionCell* Cell = InLevel->GetWorldPartitionRuntimeCell();
	UWorld* OuterWorld = Cell->GetOuterWorld();
	const bool bInPIE = OuterWorld->IsPlayInEditor();

	MEGAMESH_DEBUG_LOG(TEXT("%p PRE-Transforming Cell %p %s (%d Actors, PIE:%d) World: %p %s"), this, InLevel, *InLevel->GetName(), InLevel->Actors.Num(), bInPIE, OuterWorld, *OuterWorld->GetName());
	CellsPreTransformed++;

	if (!WorldUpdater.IsValid())
	{
		if (bInPIE)
		{
			// use the world updater specified by the editor subsystem, if there is one (because the editor world updater may better reflect in-memory changes from the editor)
			UMeshPartitionEditorSubsystem* EditorSubsystem = UMeshPartitionEditorSubsystem::Get();
			WorldUpdater = EditorSubsystem->GetPIEWorldUpdaterForWorld(OuterWorld);
		}
		if (WorldUpdater == nullptr)
		{
			// otherwise create an updater from the current world (in either PIE or cook mode; cell transformers are not invoked during compile)
			FMeshPartitionWorldUpdater::EUpdateMode UpdateMode = bInPIE ? FMeshPartitionWorldUpdater::EUpdateMode::ForPIE : FMeshPartitionWorldUpdater::EUpdateMode::ForCook;
			WorldUpdater = MakeShared<FMeshPartitionWorldUpdater>(OuterWorld, UpdateMode);
		}
		check(WorldUpdater.IsValid());
	}
}

void URuntimeCellTransformer::Transform(ULevel* InLevel)
{
	using namespace EditorUtils;

	check(InLevel);
	check(IsInGameThread());
	
	CellsTransformed++;

	const IWorldPartitionCell* Cell = InLevel->GetWorldPartitionRuntimeCell();
	UWorld* OuterWorld = Cell->GetOuterWorld();
	const bool bIsPIE = OuterWorld->IsPlayInEditor();

	MEGAMESH_DEBUG_LOG(TEXT("%p Transforming Cell %p %s (%d Actors, PIE:%d) World: %p %s"), this, InLevel, *InLevel->GetName(), InLevel->Actors.Num(), InLevel->GetWorld()->IsPlayInEditor(), OuterWorld, *OuterWorld->GetName());

	const bool bStripOutOfDate = bIsPIE && CVarMeshPartitionPIEStripOutOfDate.GetValueOnGameThread();

	TArray<MeshPartition::UModifierComponent*> BaseModifiers;

	// strip any actors we don't want : i.e. the megamesh source data and old compiled sections
	InLevel->Actors.RemoveAll([this, &BaseModifiers, bIsPIE, bStripOutOfDate](TObjectPtr<AActor> Actor)
		{
			if (!Actor)
			{
				return true;
			}

			// strip compiled sections
			if (MeshPartition::ACompiledSection* CompiledSection = Cast<MeshPartition::ACompiledSection>(Actor))
			{
				const FMeshPartitionUpdater* MeshPartitionUpdater = WorldUpdater->GetMeshPartitionUpdaterFor(CompiledSection);

				const FCompiledSectionStatus* Status = nullptr;
				if (ensure(MeshPartitionUpdater != nullptr))
				{
					Status = MeshPartitionUpdater->GetCompiledSectionStatus(CompiledSection);
				}

				// it's possible that newly created PIE placeholder compiled sections will not have a status
				if (Status == nullptr)
				{
					if (bIsPIE && CompiledSection->IsPlaceholder() && (CompiledSection->GetBuildInfo().BuildKey == WorldUpdater->GetBuildKey()))
					{
						MEGAMESH_DEBUG_LOG(TEXT("    FOUND Placeholder %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						return false;
					}

					// otherwise warn that we are somehow getting unknown compiled sections, and remove it
					UE_LOGF(LogMegaMeshEditor, Warning, "Encountered unknown compiled section, removing it: %.6ls (%p %ls)", *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
					if (MeshPartitionUpdater)
					{
						MeshPartitionUpdater->RemovedUnknown++;
					}
					return true;
				}
				check(Status->bChecked);

				if (Status->bReuse)
				{
					// keep this compiled section, it should be reused
					MEGAMESH_DEBUG_LOG(TEXT("    REUSING UpToDate %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
					MeshPartitionUpdater->ReusedCompiledSections++;
					return false;
				}

				if (Status->bIsPlaceholder)
				{
					if (!bIsPIE)
					{
						// in a cook, remove all placeholders
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING Placeholder for cook %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						MeshPartitionUpdater->RemovedOutOfDate++;
						return true;
					}
					else
					{
						// for PIE, remove any placeholders (if they were added for this PIE session they won't have a Status and are handled above)
						check(CompiledSection->GetBuildInfo().BuildKey != WorldUpdater->GetBuildKey());
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING OLD Placeholder %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						MeshPartitionUpdater->RemovedOutOfDate++;
						return true;
					}
				}

				if (bStripOutOfDate)
				{
					// look for reasons it may be out of date
					if (Status->bNonTargetVariant)
					{
						// built for a non-target build variant
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING NonTargetVariant %.6s (%p %s) Variant:%s"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel(),
							*CompiledSection->GetBuildInfo().BuildVariantName.ToString());
						MeshPartitionUpdater->RemovedNonTargetBuildVariant++;
						return true;
					}

					if (Status->bMismatchedModifiers)
					{
						// modifier set hash does not match any target group (set hash is derived from the original (non-PIE) modifier paths)
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING MismatchedModifiers %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						MeshPartitionUpdater->RemovedOutOfDate++;
						return true;
					}

					if (Status->bIsDuplicate)
					{
						// is a duplicate of another compiled section (and this one should not be used)
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING Duplicate %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						MeshPartitionUpdater->RemovedOutOfDate++;
						return true;
					}

					if (!Status->bBuildVariantHashMatches)
					{
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING BuildVariantHashFail %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						MeshPartitionUpdater->RemovedOutOfDate++;
						return true;
					}

					if (!Status->bPackageHashMatches)
					{
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING PackageHashFail %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						MeshPartitionUpdater->RemovedOutOfDate++;
						return true;
					}

					if (!Status->bClassHashMatches)
					{
						MEGAMESH_DEBUG_LOG(TEXT("    REMOVING ClassHashFail %.6s (%p %s)"), *CompiledSection->GetActorGuid().ToString(), Actor.Get(), *Actor->GetActorNameOrLabel());
						MeshPartitionUpdater->RemovedOutOfDate++;
						return true;
					}
				}
				else
				{
					// if not stripped, warn that they are out of date, but continue to use the out-of-date compiled section
					UE_LOGF(LogMegaMeshEditor, Display, "Mesh Partition '%ls' has an out-of-date compiled section '%ls' - which may show an old appearance.",
						*CompiledSection->GetBuildInfo().MegaMeshPath.ToString(),
						*CompiledSection->GetActorNameOrLabel());
					return false;
				}
			}

			// strip megamesh modifier components off of all actors, they are not for runtime
			TArray<MeshPartition::UModifierComponent*> ModifiersToRemove;
			TInlineComponentArray<MeshPartition::UModifierComponent*> ActorModifiers(Actor);
			for (MeshPartition::UModifierComponent* Modifier : ActorModifiers)
			{
				// record all of the base modifiers we see
				if (Modifier->IsBase())
				{
					BaseModifiers.Add(Modifier);
				}

				// Remove the component from the actor
				Actor->RemoveOwnedComponent(Modifier);
				Modifier->DestroyComponent();
			}

			// Remove any MegaMeshModifierActors
			if (Actor->IsA<MeshPartition::AModifierActor>())
			{
				MEGAMESH_DEBUG_LOG(TEXT("    REMOVING MeshPartition::AModifierActor (%p %s)"), Actor.Get(), *Actor->GetActorNameOrLabel());
				return true;
			}

			MEGAMESH_DEBUG_LOG(TEXT("    KEEPING actor (%p %s)"), Actor.Get(), *Actor->GetActorNameOrLabel());

			return false;
		});
}

void URuntimeCellTransformer::PostTransform(ULevel* InLevel)
{
	check(InLevel);

	MEGAMESH_DEBUG_LOG(TEXT("%p POST-Transforming Cell %p %s (%d Actors, PIE:%d) %d total cells transformed"), this, InLevel, *InLevel->GetName(), InLevel->Actors.Num(), InLevel->GetWorld()->IsPlayInEditor(), CellsTransformed);

	CellsPostTransformed++;

	if (WorldUpdater)
	{
		WorldUpdater->ReportStats();
	}
}
} // namespace UE::MeshPartition

#endif // WITH_EDITOR
#undef MEGAMESH_DEBUG_LOG
#undef LOCTEXT_NAMESPACE