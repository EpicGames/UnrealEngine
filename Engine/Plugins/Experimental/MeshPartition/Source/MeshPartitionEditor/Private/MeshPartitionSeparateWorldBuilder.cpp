// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionSeparateWorldBuilder.h"

#include "Algo/IsSorted.h"
#include "MeshPartition.h"
#include "MeshPartitionChannelCollection.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionDescriptorCache.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionWorldPartitionHelpers.h"
#include "EditorWorldUtils.h"
#include "EngineUtils.h"
#include "MaterialCache/MaterialCache.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionLevelHelper.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "Misc/PackagePath.h"
#include "Editor/EditorEngine.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/ContentBundle/ContentBundlePaths.h"
#include "UObject/UObjectGlobalsInternal.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Engine/LevelStreaming.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"

extern UNREALED_API UEditorEngine* GEditor;

static TAutoConsoleVariable<bool> CVarFillModifiersToProcess(
	TEXT("MeshPartition.SeparateWorldBuilder.FillModifiersToProcess"),
	true,
	TEXT("When false, skips filling ModifiersToProcess during PIE build (debug only)."));

namespace UE::MeshPartition
{

struct FModifierGroupBuildCacheKey
{
	FGuid MeshPartitionGUID;
	FName BuildVariantName;
	TArray<FSoftObjectPath> BaseModifierPaths;

	bool operator==(const FModifierGroupBuildCacheKey& InOther) const
	{
		return MeshPartitionGUID == InOther.MeshPartitionGUID
			&& BuildVariantName == InOther.BuildVariantName
			&& BaseModifierPaths == InOther.BaseModifierPaths;
	}

	friend uint32 GetTypeHash(const FModifierGroupBuildCacheKey& InKey)
	{
		return HashCombine(HashCombine(GetTypeHash(InKey.MeshPartitionGUID), GetTypeHash(InKey.BuildVariantName)), HashBaseModifierPaths(InKey.BaseModifierPaths));
	}

	// Compute a stable hash from the sorted base modifier paths on the BuildInfo.
	static uint32 HashBaseModifierPaths(const TArray<FSoftObjectPath>& InBaseModifierPaths)
	{
		// HashCombine is order-dependent; paths must be sorted for deterministic results.
		ensure(Algo::IsSorted(InBaseModifierPaths, [](const FSoftObjectPath& A, const FSoftObjectPath& B) { return A.GetAssetPath().Compare(B.GetAssetPath()) < 0; }));

		uint32 Hash = 0;
		for (const FSoftObjectPath& Path : InBaseModifierPaths)
		{
			Hash = HashCombine(Hash, GetTypeHash(Path));
		}
		return Hash;
	}
};

struct FCachedModifierGroupBuild
{
	TSharedPtr<const FSectionChannels> Channels;
	TMap<FIntVector, TSharedPtr<const FMeshData>> CellMeshes;		// Keyed by absolute world-space grid coordinate
	FModifierGroup ModifierGroup;
	FGuid ModifiersHash;										// Computed once on cache miss, reused for subsequent siblings
};

struct FModifierGroupBuildCache
{
	TMap<FModifierGroupBuildCacheKey, FCachedModifierGroupBuild> Entries;
};

// Resolver that remaps modifier paths from the editor world package to the transient
// BuilderPackage so that ResolveModifierPtrs finds the instanced actors loaded there.
class FBuilderPackageModifierResolver : public MeshPartition::IModifierResolver
{
public:
	FBuilderPackageModifierResolver(FName InOriginalPackageName, FName InBuilderPackageName)
		: OriginalPackageName(InOriginalPackageName)
		, BuilderPackageName(InBuilderPackageName)
	{}

	virtual MeshPartition::UModifierComponent* ResolveModifier(const MeshPartition::FModifierDesc& ModifierDesc) override
	{
		FSoftObjectPath RemappedPath = ModifierDesc.ModifierPath;
		FName OldPackageName = RemappedPath.GetAssetPath().GetPackageName();
		if (OldPackageName == OriginalPackageName)
		{
			RemappedPath.RemapPackage(OldPackageName, BuilderPackageName);
		}
		UObject* Resolved = RemappedPath.ResolveObject();
		return Cast<MeshPartition::UModifierComponent>(Resolved);
	}

private:
	FName OriginalPackageName;
	FName BuilderPackageName;
};

namespace MeshPartitionSeparateWorldBuilderLocals
{

void ApplyGridCellToSection(ACompiledSection*					InCompiledSection,
							const FCachedModifierGroupBuild&	InCachedBuild,
							UMeshPartitionEditorComponent*		InEditorComponent)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::SeparateWorldBuilder::ApplyGridCellToSection);

	const FCompiledSectionBuildInfo& BuildInfo = InCompiledSection->GetBuildInfo();
	const FIntVector& GridCellCoord = BuildInfo.GridCellCoord;

	const TSharedPtr<const FMeshData>* CellMeshPtr = InCachedBuild.CellMeshes.Find(GridCellCoord);

	if (!CellMeshPtr || !*CellMeshPtr)
	{
		UE_LOGF(LogMegaMeshEditor, Verbose, "No geometry for grid cell (%d,%d,%d) — leaving as placeholder.", GridCellCoord.X, GridCellCoord.Y, GridCellCoord.Z);
		return;
	}

	if (!ensure(InCachedBuild.Channels.IsValid()))
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "No section channels for grid cell (%d,%d,%d) — leaving as placeholder.", GridCellCoord.X, GridCellCoord.Y, GridCellCoord.Z);
		return;
	}

	InEditorComponent->ApplyBuiltMeshToSection(InCompiledSection, *CellMeshPtr, *InCachedBuild.Channels, InCachedBuild.ModifierGroup);
	InCompiledSection->SetIsPlaceholder(false);

	UE_LOGF(LogMegaMeshEditor, Verbose, "Built grid cell (%d,%d,%d): %p %ls (%d vertices, %d tris)",
		   GridCellCoord.X, GridCellCoord.Y, GridCellCoord.Z, InCompiledSection, *InCompiledSection->GetActorNameOrLabel(),
		   (*CellMeshPtr)->VertexCount(), (*CellMeshPtr)->TriangleCount());
}

// Splits the full mesh into grid cells, stores the result in the build cache, and
// returns a pointer to the cached entry. Returns nullptr if the split could not be
// performed (caller falls through to single-section path).
FCachedModifierGroupBuild* SplitAndCacheGridBuild(const FCompiledSectionBuildInfo&				InBuildInfo,
												  const FBuildTaskHandle&						InTaskHandle,
												  FModifierGroup&&								InModifierGroup,
												  FModifierGroupBuildCache&						InCache)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::SeparateWorldBuilder::SplitAndCacheGridBuild);

	// Read from BuildInfo — set by the WorldUpdater when creating placeholders.
	// ResolveGridFromPipeline requires access to WP RuntimeHash, which is not
	// available in the PIE world; the editor world resolved and stamped it at placeholder creation.
	const FGridSettings& GridSettings = InBuildInfo.GridSettings;
	TSharedPtr<const FMeshData> ProcessedMesh = InTaskHandle.GetTask()->GetMesh();

	UE_LOGF(LogMegaMeshEditor, Verbose, "Grid-split build: GridCellCoord=(%d,%d,%d) CellSize=%u bIs2D=%d MeshBounds=[%.1f,%.1f,%.1f]-[%.1f,%.1f,%.1f]",
			InBuildInfo.GridCellCoord.X, InBuildInfo.GridCellCoord.Y, InBuildInfo.GridCellCoord.Z, GridSettings.CellSize, GridSettings.bIs2D ? 1 : 0,
		   ProcessedMesh ? ProcessedMesh->GetBounds().Min.X : 0.0, ProcessedMesh ? ProcessedMesh->GetBounds().Min.Y : 0.0, ProcessedMesh ? ProcessedMesh->GetBounds().Min.Z : 0.0,
		   ProcessedMesh ? ProcessedMesh->GetBounds().Max.X : 0.0, ProcessedMesh ? ProcessedMesh->GetBounds().Max.Y : 0.0, ProcessedMesh ? ProcessedMesh->GetBounds().Max.Z : 0.0);

	if (!GridSettings.IsGridSplit() || !ProcessedMesh)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Grid cell size could not be resolved for grid-split build — falling back to single-section.");
		return nullptr;
	}

	// Split the full mesh into grid cells keyed by absolute world-space coordinates.
	// Absolute coordinates decouple this path from the WorldUpdater's placeholder creation:
	// both derive cell coordinates from world position and cell size alone, independent of
	// the bounding box used to enumerate cells. Extra cells (from modifiers growing the mesh
	// beyond ComputeBaseBounds) simply won't match any placeholder and are harmlessly skipped.
	// The task's Transform was populated from BuilderSettings.Transform (= MeshPartition actor's LocalToWorld)
	// when the build task was created -- see MeshPartitionMeshBuilder.cpp:1367. Forwarding it here aligns
	// grid cells with the WP runtime grid when the actor is translated.
	const FTransform& LocalToWorld = InTaskHandle.GetTask()->GetTransform();
	TMap<FIntVector, FMeshData> RawCellMeshes = GridHelpers::BuildGridCellMeshes(*ProcessedMesh, GridSettings, LocalToWorld);

	UE_LOGF(LogMegaMeshEditor, Verbose, "Grid-split produced %d non-empty cells (GridCellCoord=(%d,%d,%d)).",
		   RawCellMeshes.Num(), InBuildInfo.GridCellCoord.X, InBuildInfo.GridCellCoord.Y, InBuildInfo.GridCellCoord.Z);

	FCachedModifierGroupBuild NewCache;
	NewCache.Channels = InTaskHandle.GetTask()->GetSectionChannels();
	NewCache.CellMeshes.Reserve(RawCellMeshes.Num());

	for (TPair<FIntVector, FMeshData>& CellEntry : RawCellMeshes)
	{
		NewCache.CellMeshes.Add(CellEntry.Key, MakeShared<const FMeshData>(MoveTemp(CellEntry.Value)));
	}

	NewCache.ModifierGroup = MoveTemp(InModifierGroup);
	NewCache.ModifiersHash = InBuildInfo.ModifiersHash;

	FModifierGroupBuildCacheKey CacheKey{InBuildInfo.MegaMeshGUID, InBuildInfo.BuildVariantName, InBuildInfo.BaseModifierPaths};
	return &InCache.Entries.Add(CacheKey, MoveTemp(NewCache));
}

// Walks every level-instance child container under InEditorWorldPartition and appends an
// FWorldPartitionRuntimeCellObjectMapping for each non-ALevelInstance actor desc.
static void GatherLevelInstanceActorMappings(
	UWorldPartition*                                 InEditorWorldPartition,
	FName                                            InOriginalPackageName,
	TArray<FWorldPartitionRuntimeCellObjectMapping>& InOutActorPackagesToLoad)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::SeparateWorldBuilder::GatherLevelInstanceActorMappings);

	if (InEditorWorldPartition == nullptr)
	{
		return;
	}

	UActorDescContainerInstance* RootContainer = InEditorWorldPartition->GetActorDescContainerInstance();
	if (RootContainer == nullptr)
	{
		return;
	}

	TFunction<void(UActorDescContainerInstance*)> VisitContainer = [&](UActorDescContainerInstance* Container)
	{
		TSet<FGuid> ChildContainerActorGuids;
		for (const TPair<FGuid, TObjectPtr<UActorDescContainerInstance>>& Child : Container->GetChildContainerInstances())
		{
			ChildContainerActorGuids.Add(Child.Key);
			if (UActorDescContainerInstance* ChildContainer = Child.Value.Get())
			{
				VisitContainer(ChildContainer);
			}
		}

		// Skip the root — its actors are the parent world's own actors; the caller handles those
		// directly through AddActorToLoad.
		if (Container == RootContainer)
		{
			return;
		}

		const FActorContainerID& ContainerID = Container->GetContainerID();
		const FTransform ContainerTransform = Container->GetTransform();
		const FName ContainerPackage = Container->GetContainerPackage();

		for (UActorDescContainerInstance::TIterator<> It(Container); It; ++It)
		{
			FWorldPartitionActorDescInstance* DescInstance = *It;
			if (DescInstance == nullptr)
			{
				continue;
			}

			if (ChildContainerActorGuids.Contains(DescInstance->GetGuid()))
			{
				continue;
			}

			// Skip nested level instances. We don't need to handle them here as they will be
			// included in their parent's container map.
			UClass* NativeClass = DescInstance->GetActorNativeClass();
			if (NativeClass && NativeClass->IsChildOf(ALevelInstance::StaticClass()))
			{
				continue;
			}
			
			// Only consider actor descs inside level instances which have modifier components:
			bool bHasModifierComponent = false;
			const FWorldPartitionActorDesc* ActorDesc = DescInstance->GetActorDesc();
			for (const TUniquePtr<FWorldPartitionComponentDesc>& ComponentDesc : ActorDesc->GetComponentDescs())
			{
				if (!ensure(ComponentDesc.IsValid()))
				{
					continue;
				}

				UClass* ComponentDescClass = ComponentDesc->GetComponentNativeClass();
				if (ComponentDescClass->IsChildOf<MeshPartition::UModifierComponent>())
				{
					bHasModifierComponent = true;
					break;
				}
			}
			
			if (!bHasModifierComponent)
			{
				continue;
			}

			FWorldPartitionRuntimeCellObjectMapping Mapping(
				DescInstance->GetActorPackage(),
				FName(DescInstance->GetActorSoftPath().ToString()),
				DescInstance->GetBaseClass(),
				DescInstance->GetNativeClass(),
				ContainerID,
				ContainerTransform,
				FTransform::Identity,
				ContainerPackage,
				InOriginalPackageName,
				ContainerID.GetActorGuid(DescInstance->GetGuid()),
				false);

			TArray<FWorldPartitionRuntimeCellPropertyOverride> PropertyOverrides;
			Container->GetPropertyOverridesForActor(ContainerID, DescInstance->GetGuid(), PropertyOverrides);
			if (PropertyOverrides.Num())
			{
				Mapping.PropertyOverrides = MoveTemp(PropertyOverrides);
			}

			InOutActorPackagesToLoad.Add(MoveTemp(Mapping));
		}
	};

	VisitContainer(RootContainer);
}

static bool LoadAndInitializeModifiers(
	MeshPartition::FModifierGroup& InOutModifierGroup,
	const FGuid& MeshPartitionGuid,
	UPackage* InBuilderPackage,
	UWorld* InBuilderWorld)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FSeparateWorldBuilder::LoadAndInitializeModifiers);

	UWorld* EditorWorld = GEditor ? GEditor->GetEditorWorldContext().World() : nullptr;
	check(EditorWorld);

	const FName OriginalPackageName = EditorWorld->GetPackage()->GetFName();
	const FName BuilderPackageName = InBuilderPackage->GetFName();

	UWorldPartition* WorldPartition = EditorWorld->GetWorldPartition();
	if (!WorldPartition)
	{
		return false;
	}
	

	// Gather actor packages for each unique modifier owner from the editor world's WorldPartition.
	TArray<FWorldPartitionRuntimeCellObjectMapping> ActorPackagesToLoad;

	auto AddActorToLoad = [WorldPartition, &OriginalPackageName, &ActorPackagesToLoad](const FGuid& ActorGuid)
	{
		const FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid);
		if (ActorDescInstance == nullptr)
		{
			UE_LOGF(LogMegaMeshEditor, Warning, "No actor descriptor found for modifier owner guid: %ls", *ActorGuid.ToString());
			return;
		}

		const UActorDescContainerInstance* Container = ActorDescInstance->GetContainerInstance();
		if (!Container)
		{
			UE_LOGF(LogMegaMeshEditor, Warning, "No actor descriptor container found for modifier owner guid: %ls", *ActorGuid.ToString());
			return;
		}
		const FActorContainerID& ContainerID = Container->GetContainerID();

		FWorldPartitionRuntimeCellObjectMapping Mapping = FWorldPartitionRuntimeCellObjectMapping(
			ActorDescInstance->GetActorPackage(),
			FName(ActorDescInstance->GetActorSoftPath().ToString()),
			ActorDescInstance->GetBaseClass(),
			ActorDescInstance->GetNativeClass(),
			ContainerID,
			Container->GetTransform(),
			FTransform::Identity,
			Container->GetPackage()->GetFName(),
			OriginalPackageName,
			ContainerID.GetActorGuid(ActorDescInstance->GetGuid()),
			false
		);

		// Note: We do not need to set up any property overrides on the FWPRuntimeCellObjectMapping for actors not inside of level instances.

		ActorPackagesToLoad.Add(MoveTemp(Mapping));
	};

	AddActorToLoad(MeshPartitionGuid);

	TSet<FGuid> SeenOwnerGuids;
	SeenOwnerGuids.Add(MeshPartitionGuid);
	for (const MeshPartition::FModifierDesc& ModifierDesc : InOutModifierGroup.AllModifierDescs())
	{
		bool bAlreadyAdded = false;
		SeenOwnerGuids.Add(ModifierDesc.OwnerGuid, &bAlreadyAdded);
		if (bAlreadyAdded)
		{
			continue;
		}

		AddActorToLoad(ModifierDesc.OwnerGuid);
	}

	// Remap the package and into the BuilderPackage so they are instanced and loaded actors resolve under our transient world.
	FLinkerInstancingContext InstancingContext;
	InstancingContext.SetSoftObjectPathRemappingEnabled(false);
	InstancingContext.AddPackageMapping(OriginalPackageName, BuilderPackageName);

	GatherLevelInstanceActorMappings(WorldPartition, OriginalPackageName, ActorPackagesToLoad);

	for (const FWorldPartitionRuntimeCellObjectMapping& Mapping : ActorPackagesToLoad)
	{
		if (ContentBundlePaths::IsAContentBundlePath(Mapping.ContainerPackage.ToString()) ||
			FExternalDataLayerHelper::IsExternalDataLayerPath(Mapping.ContainerPackage.ToString()))
		{
			check(Mapping.ContainerPackage != Mapping.WorldPackage);
			if (InstancingContext.RemapPackage(Mapping.ContainerPackage) == Mapping.ContainerPackage)
			{
				InstancingContext.AddPackageMapping(Mapping.ContainerPackage, BuilderPackageName);
			}
		}
	}

	// Load actors with DestLevel=nullptr so the helper leaves each actor in its instanced external package.
	// Without this, the actors would load into the main world package and end up getting trashed on subsequent loads.
	// This mirrors what happens during streaming generation.
	TArray<int32> AsyncLoadIDs;
	FWorldPartitionLevelHelper::FPackageReferencer PackageReferencer;

	FWorldPartitionLevelHelper::LoadActors(
		FWorldPartitionLevelHelper::FLoadActorsParams()
			.SetOuterWorld(InBuilderWorld)
			.SetDestLevel(nullptr)
			.SetActorPackages(ActorPackagesToLoad)
			.SetPackageReferencer(&PackageReferencer)
			.SetLoadAsync(true, &AsyncLoadIDs)
			.SetCompletionCallback([](bool) {})
			.SetInstancingContext(MoveTemp(InstancingContext)));

	FlushAsyncLoading(AsyncLoadIDs);

	// Migrate loaded actors from their instanced external packages into the builder world's persistent level.
	// Collapses the external packages and trashes them when empty.
	TArray<UPackage*> ModifiedPackages;
	FWorldPartitionLevelHelper::MoveExternalActorsToLevel(ActorPackagesToLoad, InBuilderWorld->PersistentLevel, ModifiedPackages);

	// UpdateWorldComponents allows component registration which is required to initialize modifier components to a valid state (eg. ComponentToWorld transform).
	InBuilderWorld->UpdateWorldComponents(false, true);

	AMeshPartition* BuilderMeshPartition = nullptr;
	for (AMeshPartition* MeshPartition : TActorRange<AMeshPartition>(InBuilderWorld))
	{
		if (MeshPartition->GetActorGuid() == MeshPartitionGuid)
		{
			BuilderMeshPartition = MeshPartition;
			break;
		}
	}
	
	if (!ensure(BuilderMeshPartition && InBuilderWorld->PersistentLevel))
	{
		return false;
	}

	// Discover modifiers that came in from level-instance actors.
	// Those actors are now in BuilderWorld->PersistentLevel because GatherLevelInstanceActorMappings added them to
	// ActorPackagesToLoad above. Walk them here and add their modifiers into the group.
	{
		TConstArrayView<FName> ModifierTypePriorities = BuilderMeshPartition->GetMeshPartitionDefinition()
			? BuilderMeshPartition->GetMeshPartitionDefinition()->GetModifierTypePriorities()
			: TConstArrayView<FName>();

		const FBox BaseBounds = InOutModifierGroup.ComputeBaseBounds();

		for (AActor* Actor : InBuilderWorld->PersistentLevel->Actors)
		{
			if (Actor == nullptr)
			{
				continue;
			}
			
			TInlineComponentArray<MeshPartition::UModifierComponent*> ActorModifiers(Actor);
			for (MeshPartition::UModifierComponent* Modifier : ActorModifiers)
			{
				// All modifiers must point to the same affected mesh partition, otherwise they would not have been loaded.
				// Since their affected mesh partition pointer is currently assigned to the mesh partition in the PIE world,
				// (or null in the case of modifiers in level instances), reassign it here so we are in a consistent world state.
				Modifier->SetAffectedMeshPartition(BuilderMeshPartition);
				
				if (!Modifier->IntersectsAnyBounds({ BaseBounds }))
				{
					continue;
				}

				// Skip any actors which were seen in the initial loading pass. This filters the loop down to only those actors which were not in the modifier group initially.
				// These can only be the ones loaded from level instances.
				if (!SeenOwnerGuids.Contains(Actor->GetActorGuid()))
				{
					if (Modifier->IsBase())
					{
						UE_LOGF(LogMegaMeshEditor, Warning, "Modifier %ls on actor %ls is a BaseModifier and will be ignored; base modifiers are not supported inside level instances.",
							*Modifier->GetName(), *Actor->GetName());
						continue;
					}
					InOutModifierGroup.AddModifierSorted(ModifierTypePriorities, *Modifier);
				}
			}
		}
	}

	CastChecked<UMeshPartitionEditorComponent>(BuilderMeshPartition->GetMeshPartitionComponent())->UpdateModifierList();

	InOutModifierGroup.SetModifierResolver(MakeShared<FBuilderPackageModifierResolver>(OriginalPackageName, BuilderPackageName));
	InOutModifierGroup.RemoveDisabledModifiers();
	InOutModifierGroup.ProgressToState(MeshPartition::FModifierGroup::EState::ModifiersResolved);

	return true;
}

} // namespace MeshPartitionSeparateWorldBuilderLocals

// ---------------------------------------------------------------------------
// FSeparateWorldBuilder
// ---------------------------------------------------------------------------

FSeparateWorldBuilder::FBuilderWorldScope::FBuilderWorldScope(FSeparateWorldBuilder& InOwner, const FString& InWorldPackageName)
	: Owner(InOwner)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::SeparateWorldBuilder::FBuilderWorldScope);

	ensure(Owner.BuilderPackage == nullptr);
	ensure(Owner.BuilderWorld == nullptr);
	check(GEditor);

	Owner.BuilderPackage = CreatePackage(TEXT("/Temp/MeshPartitionBuilderPackage"));

	UWorld* EditorWorld = GEditor->GetEditorWorldContext().World();

	// Create a World + PersistentLevel inside the BuilderPackage so the actor outer
	// chain (Package.WorldName:PersistentLevel.ActorName) resolves during loading.
	UWorld::InitializationValues IVS;
	IVS.InitializeScenes(false)
		.CreateWorldPartition(true)
		.AllowAudioPlayback(false)
		.RequiresHitProxies(false)
		.CreatePhysicsScene(false)
		.CreateNavigation(false)
		.CreateAISystem(false)
		.ShouldSimulatePhysics(false)
		.EnableTraceCollision(false)
		.CreateFXSystem(false);
	
	Owner.BuilderWorld = UWorld::CreateWorld(EWorldType::Inactive, /*bInformEngineOfWorld*/false, *InWorldPackageName, Owner.BuilderPackage, /*bAddToRoot*/false, EditorWorld->GetFeatureLevel(), &IVS, /*bInSkipInitWorld*/false);
	
	check(Owner.BuilderWorld);
	Owner.BuilderWorld->SetFlags(RF_Public | RF_Transient);

	FWorldContext& WorldContext = GEngine->CreateNewWorldContext(EWorldType::Inactive);
	WorldContext.SetCurrentWorld(Owner.BuilderWorld);
}

FSeparateWorldBuilder::FBuilderWorldScope::~FBuilderWorldScope()
{
	// Cache entries reference modifier actors loaded into BuilderWorld and must be
	// released before the world holding them is torn down.
	if (Owner.GroupBuildCache)
	{
		Owner.GroupBuildCache->Entries.Empty();
	}

	check(Owner.BuilderWorld);
	Owner.BuilderWorld->DestroyWorld(false /*bBroadcastWorldDestroyedEvent*/);
	GEngine->DestroyWorldContext(Owner.BuilderWorld);

	TrashObject(Owner.BuilderPackage);

	Owner.BuilderPackage = nullptr;
	Owner.BuilderWorld = nullptr;

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
}

void FSeparateWorldBuilder::BuildCompiledSectionMesh(MeshPartition::ACompiledSection* InCompiledSection, FName InBuildVariantName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::SeparateWorldBuilder::BuildCompiledSectionMesh);

	// NOTE: the compiled section is the actor we need to build the mesh for
	// however, the compiled section does NOT belong to our World (our World is a temp world loaded just for building)
	check(InCompiledSection);
	UWorld* CompiledSectionWorld = InCompiledSection->GetWorld();

	if (CompiledSectionWorld == nullptr)
	{
		// The compiled section's level may have been streamed out. The actor is still
		// reachable via weak pointer but its world is gone — skip it.
		UE_LOGF(LogMegaMeshEditor, Warning, "BuildCompiledSectionMesh: compiled section %ls has no world — skipping (likely streamed out).", *InCompiledSection->GetActorNameOrLabel());
		return;
	}

	MeshPartition::FCompiledSectionBuildInfo BuildInfo = InCompiledSection->GetBuildInfo();

	// Fix PIE paths early so that all downstream uses (cache keys, modifier group building,
	// and world naming) operate on consistent editor-world paths.
	{
		MeshPartition::EditorUtils::FPIEPathFixer& PIEPathFixer = UMeshPartitionEditorSubsystem::GetPIEPathFixer();
		PIEPathFixer.FixInPlace(BuildInfo.MegaMeshPath);
		for (FSoftObjectPath& ModifierPath : BuildInfo.BaseModifierPaths)
		{
			PIEPathFixer.FixInPlace(ModifierPath);
		}

		// Re-sort after PIE-fixing: FixInPlace changes path strings, which can break the
		// lexicographic order that HashBaseModifierPaths requires for deterministic hashing.
		BuildInfo.BaseModifierPaths.Sort([](const FSoftObjectPath& A, const FSoftObjectPath& B) { return A.GetAssetPath().Compare(B.GetAssetPath()) < 0; });
	}

	// Get the mesh partition definition.  Note that the compiled section is in a streaming cell, and it's parent mesh partition may not be loaded
	// So instead we make sure to grab the definition from the path stored on the buildinfo
	FSoftObjectPath MeshPartitionDefinitionPath(BuildInfo.MegaMeshDefinitionPath);
	TSoftObjectPtr<UMeshPartitionDefinition> MeshPartitionDefinition(MeshPartitionDefinitionPath);
	UMeshPartitionDefinition* Definition = MeshPartitionDefinition.Get();

	if (Definition == nullptr)
	{
		Definition = UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
	}

	const MeshPartition::FCompiledSectionBuildVariant& BuildVariant = Definition->GetCompiledSectionBuildVariantByName(InBuildVariantName);

	MeshPartition::FMeshBuilder* Builder = UMeshPartitionEditorSubsystem::GetMeshBuilder();

	if (Builder == nullptr)
	{
		return;
	}

	MeshPartition::FModifierDescriptorCache* DescriptorCache = UMeshPartitionEditorSubsystem::GetDescriptorCache();

	check(DescriptorCache);

	// We search by GUID because the path can be modified by PIE
	TSharedPtr<const MeshPartition::FCachedDescriptors> Descriptors = DescriptorCache->GetCachedDescriptorsByGuid(BuildInfo.MegaMeshGUID);

	if (!Descriptors.IsValid())
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Failed to build compiled section: no cached descriptors for MeshPartition GUID %ls", *BuildInfo.MegaMeshGUID.ToString());
		return;
	}

	// find the mesh partition actor by GUID (NOTE: we can't find it by path, unless we use the PIE world path, otherwise it will get the mesh partition actor in the main editor world)
	FGuid MeshPartitionActorGUID = Descriptors->GetMeshPartitionGUID();
	const AMeshPartition* MeshPartitionActor = nullptr;

	for (AMeshPartition* MeshPartitionActorIt : TActorRange<AMeshPartition>(CompiledSectionWorld))
	{
		if (MeshPartitionActorIt && (MeshPartitionActorIt->GetActorGuid() == MeshPartitionActorGUID))
		{
			MeshPartitionActor = MeshPartitionActorIt;
			break;
		}
	}

	if (MeshPartitionActor == nullptr)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Failed to build compiled section: could not find parent MeshPartition actor %ls in World %ls", *MeshPartitionActorGUID.ToString(), *CompiledSectionWorld->GetName());
		return;
	}

	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartitionActor->GetMeshPartitionComponent());

	check(MeshPartitionEditorComponent);
	check(MeshPartitionEditorComponent->GetWorld() == CompiledSectionWorld);

	// --- Grid-split cache: check for an existing build before doing expensive work ---
	if (BuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid && BuildInfo.GridSettings.IsGridSplit())
	{
		if (!GroupBuildCache)
		{
			GroupBuildCache = MakeShared<FModifierGroupBuildCache>();
		}

		FModifierGroupBuildCacheKey CacheKey{BuildInfo.MegaMeshGUID, InBuildVariantName, BuildInfo.BaseModifierPaths};

		if (FCachedModifierGroupBuild* CachedBuild = GroupBuildCache->Entries.Find(CacheKey))
		{
			UE_LOGF(LogMegaMeshEditor, Verbose, "Grid build cache hit for group %ls — skipping mesh rebuild.", *BuildInfo.ModifierSetHash.ToString());

			// Stamp the PIE-fixed BuildInfo with the cached ModifiersHash so the section
			// carries correct metadata (matching what the first sibling received on cache miss).
			BuildInfo.ModifiersHash = CachedBuild->ModifiersHash;
			InCompiledSection->SetBuildInfo(BuildInfo);

			MeshPartitionSeparateWorldBuilderLocals::ApplyGridCellToSection(InCompiledSection, *CachedBuild, MeshPartitionEditorComponent);
			return;
		}
	}

	// --- Full mesh build (shared between grid miss and non-grid paths) ---

	MeshPartition::FModifierGroup ModifierGroup = Descriptors->BuildModifierGroupForBaseModifierPaths(BuildInfo.BaseModifierPaths);

	if (!ensure(BuildInfo.MegaMeshPath.IsValid()))
	{
		return;
	}

	bool bAllModifiersLoaded = MeshPartitionSeparateWorldBuilderLocals::LoadAndInitializeModifiers(ModifierGroup, BuildInfo.MegaMeshGUID, BuilderPackage, BuilderWorld);
	
	if (!bAllModifiersLoaded)
	{
		return;
	}

	FGuid GroupHash = ModifierGroup.UpdateAndComputeModifierGroupHash();
	BuildInfo.ModifiersHash = GroupHash;

	// Stamp the PIE-fixed, hash-updated BuildInfo on the section so that downstream code
	// (ApplyBuiltMeshToSection) can read it directly via GetBuildInfo().
	InCompiledSection->SetBuildInfo(BuildInfo);

	MeshPartition::FBuilderSettings BuilderSettings;
	{
		BuilderSettings.BuildType = MeshPartition::EBuildType::CompiledSection;
		BuilderSettings.Transform = MeshPartitionActor->GetActorTransform();

		// Fill out modifiers to process
		BuilderSettings.ModifiersToProcess.Empty();

		if (CVarFillModifiersToProcess.GetValueOnGameThread())
		{
			ModifierGroup.ForAllModifiers([&BuilderSettings](MeshPartition::UModifierComponent* InModifierComponent)
			{
				if (ensure(InModifierComponent != nullptr))
				{
					BuilderSettings.ModifiersToProcess.Add(InModifierComponent);
				}

				return true;
			});
		}

		BuilderSettings.TypePriorities = Definition->GetModifierTypePriorities();
		// BuilderSettings.FilterBounds = ;				// optional bounds
		// BuilderSettings.ModifierFilter = ...;		// Optional filter function for the list of modifiers to process.
		BuilderSettings.MaxSectionComplexity = BuildVariant.MaxSectionComplexity;
		BuilderSettings.bRecomputeNormals = false;
		BuilderSettings.bRecomputeTangents = false;		// not yet supported
		BuilderSettings.bCacheResult = true;
		BuilderSettings.bBuildSpatial = false;
		BuilderSettings.bAllowDDCRead = true;			// Should builds with these settings be allowed to use the DDC cache to avoid processing.
		BuilderSettings.bAllowDDCWrite = true;			// Should builds produced with these settings be allowed to store their results in DDC.
		BuilderSettings.TexcoordGenerationOptions = FChannelCollectionUVLayoutOptions::GetFromDefinition(Definition);
		// If builder is using the FastBoxProject channel UV option (intended for preview only), automatically switch it to a better method for compiled sections
		if (BuilderSettings.TexcoordGenerationOptions->UVLayoutMethod == EChannelCollectionUVLayoutMethod::FastBoxProject)
		{
			BuilderSettings.TexcoordGenerationOptions->UVLayoutMethod = EChannelCollectionUVLayoutMethod::ReferenceBoxProject;
		}

		MeshPartition::FBuilderSettings::FChannelRenderSettings ChannelRenderSettings;
		ChannelRenderSettings.ChannelMap = Definition->GetChannelMap();
		ChannelRenderSettings.TexelSize = Definition->GetChannelTexelSize();
		BuilderSettings.ChannelRenderSettings.Emplace(MoveTemp(ChannelRenderSettings));
	}

	constexpr bool bShouldEverAllowDDC = true;
	MeshPartition::FBuildTaskHandle TaskHandle = Builder->Build(BuilderSettings, ModifierGroup, bShouldEverAllowDDC); 
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::SeparateWorldBuilder::BlockOnBuild);
		TaskHandle.Wait();
	}

	check(TaskHandle.IsCompleted());

	// --- Grid-split: split mesh, cache, and apply the requested cell ---
	if (BuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid && BuildInfo.GridSettings.IsGridSplit())
	{
		FCachedModifierGroupBuild* CachedBuild = MeshPartitionSeparateWorldBuilderLocals::SplitAndCacheGridBuild(BuildInfo, TaskHandle, MoveTemp(ModifierGroup), *GroupBuildCache);

		if (CachedBuild != nullptr)
		{
			MeshPartitionSeparateWorldBuilderLocals::ApplyGridCellToSection(InCompiledSection, *CachedBuild, MeshPartitionEditorComponent);
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Warning, "Grid-split build failed for compiled section %ls — section left as placeholder.",
				    *InCompiledSection->GetActorNameOrLabel());
		}

		return;
	}

	// --- Non-grid path (bSplitSectionsToMatchWorldPartitionRuntimeGrid == false) ---
	TSharedPtr<const FMeshData> ProcessedMesh = TaskHandle.GetTask()->GetMesh();
	TSharedPtr<const MeshPartition::FSectionChannels> SectionChannel = TaskHandle.GetTask()->GetSectionChannels();

	if (ensure(ProcessedMesh.IsValid()) && ensure(SectionChannel.IsValid()))
	{
		MeshPartitionEditorComponent->ApplyBuiltMeshToSection(InCompiledSection, ProcessedMesh, *SectionChannel, ModifierGroup);
		InCompiledSection->SetIsPlaceholder(false);

		UE_LOGF(LogMegaMeshEditor, Verbose, "Built Compiled Section: %p %ls (%d bases, %d vertices, %d tris)",
			   InCompiledSection, *InCompiledSection->GetActorNameOrLabel(), BuildInfo.BaseModifierPaths.Num(),
			   ProcessedMesh->VertexCount(), ProcessedMesh->TriangleCount());
	}
}

void FSeparateWorldBuilder::RequestCompiledSectionBuild(MeshPartition::ACompiledSection* InCompiledSection)
{
	if (ensure(InCompiledSection->IsPlaceholder()))
	{
		check(IsInGameThread());
		CompiledSectionsToBuild.Add(InCompiledSection);
	}
}

void FSeparateWorldBuilder::Tick()
{
	check(IsInGameThread());
	MeshPartition::EditorUtils::FPIEPathFixer& PIEPathFixer = UMeshPartitionEditorSubsystem::GetPIEPathFixer();

	// Swap into a local array before iterating. BuildCompiledSectionMesh pumps the game
	// thread (WaitOnGameThread), which can trigger streaming -> new RequestCompiledSectionBuild
	// calls that Add to CompiledSectionsToBuild. Iterating the member directly would
	// invalidate the range-for iterator on reallocation.
	TArray<TWeakObjectPtr<MeshPartition::ACompiledSection>> SectionsToBuild;
	Swap(SectionsToBuild, CompiledSectionsToBuild);

	// Bucket sections by their MeshPartition asset path so the BuilderWorld and
	// GroupBuildCache lifetimes span every section that may share a cached modifier
	// group build. Without this, a sibling grid-cell section that hits the cache
	// would resolve modifier ptrs against a world that was already torn down.
	TMap<FSoftObjectPath, TArray<TWeakObjectPtr<MeshPartition::ACompiledSection>>> SectionsByMeshPartition;
	int32 TotalSections = 0;

	for (const TWeakObjectPtr<MeshPartition::ACompiledSection>& CompiledSectionPtr : SectionsToBuild)
	{
		MeshPartition::ACompiledSection* CompiledSection = CompiledSectionPtr.Get();

		if ((CompiledSection == nullptr) || !CompiledSection->IsPlaceholder())
		{
			// Filter non-placeholders here so a bucket of all-stale sections never
			// pays the BuilderWorld init+teardown cost (CollectGarbage etc.).
			continue;
		}

		FSoftObjectPath MeshPartitionPath = CompiledSection->GetBuildInfo().MegaMeshPath;

		if (!ensure(MeshPartitionPath.IsValid()))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "Compiled section %ls has invalid MeshPartition path in BuildInfo — skipping build.", *CompiledSection->GetActorNameOrLabel());
			continue;
		}

		// PIE-fix during gather so sections from different PIE prefixes for the same
		// editor asset bucket together (cache reuse correct by construction, not by
		// the prose assumption that all sections in a Tick share one PIE prefix).
		PIEPathFixer.FixInPlace(MeshPartitionPath);

		SectionsByMeshPartition.FindOrAdd(MeshPartitionPath).Add(CompiledSectionPtr);
		++TotalSections;
	}

	int32 SectionIndex = 0;

	// TODO: bucketing collapses world-switch cost within a single Tick, but successive
	// Ticks that re-process the same MeshPartition still pay full init+teardown each
	// time. Cross-Tick cache preservation (detaching cached actor refs from the world)
	// would close the gap; tracked as a follow-up.
	for (const TPair<FSoftObjectPath, TArray<TWeakObjectPtr<MeshPartition::ACompiledSection>>>& Bucket : SectionsByMeshPartition)
	{
		FBuilderWorldScope Scope(*this, Bucket.Key.GetAssetPath().GetAssetName().ToString());

		for (const TWeakObjectPtr<MeshPartition::ACompiledSection>& CompiledSectionPtr : Bucket.Value)
		{
			MeshPartition::ACompiledSection* CompiledSection = CompiledSectionPtr.Get();

			UE_LOGF(LogMegaMeshEditor, Verbose, "Building Placeholder Compiled Section (%d/%d) ...", SectionIndex, TotalSections);
			
			if ((CompiledSection != nullptr) && CompiledSection->IsPlaceholder())
			{
				const FName BuildVariantName = CompiledSection->GetBuildInfo().BuildVariantName;
				BuildCompiledSectionMesh(CompiledSection, BuildVariantName);
			}

			UE_LOGF(LogMegaMeshEditor, Verbose, "BUILT Placeholder Compiled Section (%d/%d)", SectionIndex, TotalSections);
			++SectionIndex;
		}
	}
}

void FSeparateWorldBuilder::ClearRequests()
{
	CompiledSectionsToBuild.Empty();
}

FSeparateWorldBuilder::~FSeparateWorldBuilder()
{
	// GroupBuildCache is destroyed automatically via TSharedPtr
}

} // namespace UE::MeshPartition