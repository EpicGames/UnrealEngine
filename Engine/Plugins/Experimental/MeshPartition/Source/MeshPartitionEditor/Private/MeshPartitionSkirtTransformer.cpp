// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionSkirtTransformer.h"

#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionEditorModule.h"

namespace UE::MeshPartition
{

bool FMeshSkirtTransformer::Execute(MeshPartition::FTransformerContext& InTransformerContext) const
{
	UE_LOGF(LogMegaMeshEditor, Display, "FMeshSkirtTransformer is deprecated and now a no-op. Enable Mesh Skirt settings on FStaticMeshTransformer instead; existing FMeshSkirtTransformer settings are ignored.");

	return true;
}

void FMeshSkirtTransformer::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
}

} // namespace UE::MeshPartition
