// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionStaticMeshTransformer.h"

#include "Engine/StaticMesh.h"
#include "Materials/MaterialInstanceConstant.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionEditorUtils.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionMeshSkirt.h"
#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionStaticMeshDescriptor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicMesh/MeshTangents.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "MeshSimplification.h"
#include "MeshSimplificationQuadrics.h"
#include "MeshConstraintsUtil.h"
#include "MeshQueries.h"
#include "ProjectionTargets.h"
#include "DynamicMeshToMeshDescription.h"
#include "SceneManagement.h"

namespace UE::MeshPartition
{

namespace MeshPartitionStaticMeshTransformerLocals
{

namespace Constants
{
	// Reference parameters for converting a world-space geometric deviation into a ScreenSize threshold.
	// Mirrors UStaticMesh::CalculateViewDistance + the FStaticMeshRenderData::ResolveSectionInfo auto-
	// compute path in Source/Runtime/Engine/Private/StaticMesh.cpp so the values we author are
	// comparable to manually-set ScreenSize values from the static mesh editor.
	constexpr float AutoScreenSizeReferenceWidth    = 1920.0f;
	constexpr float AutoScreenSizeReferenceHeight   = 1080.0f;
	constexpr float AutoScreenSizeReferenceHalfFOV  = UE_PI * 0.25f;
}

using FAttributeAwareSimplification = Geometry::TMeshSimplification<Geometry::FAttrBasedQuadricErrorV2d>;

// struct to organize additional QEM simplifier configuration that is not currently exposed to the user
struct FQEMSimplifyMeshOptions
{
	const bool bAllowSeamCollapse = true;
	const bool bAllowSeamSmoothing = true;
	const bool bAllowSeamSplits = true;
	const bool bPreserveVertexPositions = false;
	const bool bRetainQuadricMemory = false;
	const bool bAutoCompact = true;
	const bool bUseTriangleQuadric = false;
};

// Returns {SymmetricMaxDeviation, OneSidedMaxDeviation}:
//   - Get<0>(): symmetric Hausdorff distance (max of the two one-sided vertex-to-surface maxima).
//   - Get<1>(): simplified->original direction only. 
//
TPair<double, double> SimplifyDynamicMesh(UE::Geometry::FDynamicMesh3& EditMesh, const bool bShouldRecomputeNormals, const bool bShouldRecomputeTangents, const FMeshPartitionTransformerSimplificationSettings& SimplifySettings)
{
	using namespace UE::Geometry;

	const bool bUsingGeometricTolerance = SimplifySettings.ErrorTolerance > 0.f;
	const bool bUsingEdgeLength = SimplifySettings.EdgeLength > 0.f;
	const bool bUsingMaxTrianglesCap = SimplifySettings.MaxTrianglesFraction < 1.f;

	if (!bUsingEdgeLength && !bUsingGeometricTolerance && !bUsingMaxTrianglesCap)
	{
		// no simplification requested
		UE_LOGF(LogMegaMeshEditor, Warning, "Static mesh transformer: not applying mesh reduction. EdgeLength, ErrorTolerance, or MaxTrianglesFraction<1 must be set." );
		return { 0.0, 0.0 };
	}

	FQEMSimplifyMeshOptions QEMOptions;

	// undo splitting done by uv layout
	FMergeCoincidentMeshEdges Welder(&EditMesh);
	Welder.Apply();

	// Refresh the per-corner normal overlay when the caller asked for fresh normals - this is what
	// the FAttrBasedQuadricErrorV2d quadric reads, and what flows out to the MeshDescription.
	// No-op when the mesh has no normal overlay.
	if (bShouldRecomputeNormals)
	{
		FMeshNormals::QuickRecomputeOverlayNormals(EditMesh);
	}

	if (bShouldRecomputeTangents)
	{
		FMeshTangentsf::ComputeDefaultOverlayTangents(EditMesh);
	}
	
	using SimplificationType = FAttributeAwareSimplification;

	// Standard simplifier setup
	FAttributeAwareSimplification Simplifier(&EditMesh);

	// PlaneQuadric corresponds to what classic QEM does
	Simplifier.QuadricOptions.QuadricVariant =
		QEMOptions.bUseTriangleQuadric ?
		Geometry::FAttrBasedQuadricErrorV2d::EQuadricVariant::TriangleQuadric :
		Geometry::FAttrBasedQuadricErrorV2d::EQuadricVariant::PlaneQuadric;

	Simplifier.QuadricOptions.NormalAttributeWeight   = SimplifySettings.NormalAttributeWeight;
	Simplifier.QuadricOptions.TangentAttributeWeight  = SimplifySettings.TangentAttributeWeight;
	Simplifier.QuadricOptions.ColorAttributeWeight    = SimplifySettings.ColorAttributeWeight;
	Simplifier.QuadricOptions.TexCoordAttributeWeight = SimplifySettings.TexCoordAttributeWeight;
	Simplifier.QuadricOptions.WeightLayerWeight       = SimplifySettings.WeightLayerWeight;

	// setting this to X will make the simplifier aware that the scale of the object is X times larger
	// than that for which the attribute weights were chosen.
	//
	Simplifier.QuadricOptions.ScaleCorrection = SimplifySettings.ScaleCorrection;
	
	if (QEMOptions.bUseTriangleQuadric)
	{
		// on triangle quadrics, the sigma has a different meaning and scaling (vertex variance).
		Simplifier.RegularizeWeight = SimplifySettings.MeshRegularization;
	}
	else
	{
		Simplifier.QuadricOptions.SigmaN = SimplifySettings.MeshRegularization;
	}

	// projection mesh only used to calculate distances for termination
	Simplifier.ProjectionMode = FAttributeAwareSimplification::ETargetProjectionMode::NoProjection;

	// Always retain a copy of the pre-simplification surface so the caller can measure the
	// achieved deviation regardless of which criterion drives termination. Only feed it back
	// into the simplifier as a projection target when ErrorTolerance is the active criterion.
	TUniquePtr<FDynamicMesh3> ProjectionMesh = MakeUnique<FDynamicMesh3>(EditMesh);
	TUniquePtr<FDynamicMeshAABBTree3> Spatial = MakeUnique<FDynamicMeshAABBTree3>(ProjectionMesh.Get(), true);
	FMeshProjectionTarget ProjectionTarget;
	if (bUsingGeometricTolerance)
	{
		ProjectionTarget.Mesh = ProjectionMesh.Get();
		ProjectionTarget.Spatial = Spatial.Get();
		Simplifier.SetProjectionTarget(&ProjectionTarget);
	}
	
	Simplifier.DEBUG_CHECK_LEVEL = 0;
	Simplifier.bRetainQuadricMemory = QEMOptions.bRetainQuadricMemory;
	Simplifier.bAllowSeamCollapse = QEMOptions.bAllowSeamCollapse;
	
	if (QEMOptions.bAllowSeamCollapse)
	{
		Simplifier.SetEdgeFlipTolerance(1.e-5);
		if (EditMesh.HasAttributes())
		{
			EditMesh.Attributes()->SplitAllBowties();	// eliminate any bowties that might have formed on attribute seams.
		}
	}

	EEdgeRefineFlags MeshBoundaryConstraints   = (EEdgeRefineFlags)((int)EEdgeRefineFlags::CollapseOnly | (int)EEdgeRefineFlags::NoTopologyMerge);
	EEdgeRefineFlags GroupBorderConstraints    = EEdgeRefineFlags::NoConstraint;
	EEdgeRefineFlags MaterialBorderConstraints = (EEdgeRefineFlags)((int)EEdgeRefineFlags::CollapseOnly | (int)EEdgeRefineFlags::NoTopologyMerge);

	FMeshConstraints Constraints;
	FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, EditMesh,
		MeshBoundaryConstraints, GroupBorderConstraints, MaterialBorderConstraints,
		QEMOptions.bAllowSeamSplits, QEMOptions.bAllowSeamSmoothing, QEMOptions.bAllowSeamCollapse);

	constexpr double MeshBoundaryCornerAngleThresholdDegrees = 45.;
	FMeshConstraintsUtil::SetBoundaryConstraintsWithProjection(Constraints, FMeshConstraintsUtil::EBoundaryType::Mesh, EditMesh, MeshBoundaryCornerAngleThresholdDegrees);
	Simplifier.SetExternalConstraints(MoveTemp(Constraints));

	if (QEMOptions.bPreserveVertexPositions)
	{
		Simplifier.CollapseMode = SimplificationType::ESimplificationCollapseModes::MinimalExistingVertexError;
	}

	const float MinTrianglesFraction = FMath::Clamp(SimplifySettings.MinTrianglesFraction, 0.f, 1.f);
	float MaxTrianglesFraction = FMath::Clamp(SimplifySettings.MaxTrianglesFraction, 0.f, 1.f);
	if (MaxTrianglesFraction < MinTrianglesFraction)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Simplification MaxTrianglesFraction (%.3f) is less than MinTrianglesFraction (%.3f); using MinTrianglesFraction as the upper bound.", MaxTrianglesFraction, MinTrianglesFraction);
		MaxTrianglesFraction = MinTrianglesFraction;
	}

	if (bUsingGeometricTolerance)
	{
		Simplifier.GeometricErrorConstraint = SimplificationType::EGeometricErrorCriteria::PredictedPointToProjectionTarget;
		Simplifier.GeometricErrorTolerance = SimplifySettings.ErrorTolerance;

		// MaxTrianglesFraction enforces an upper bound on the result triangle count regardless of
		// error tolerance: while the mesh is still above this count, collapses are allowed even
		// when they would normally exceed the tolerance.
		if (MaxTrianglesFraction < 1.f)
		{
			Simplifier.MaxResultTriangleCount = FMath::CeilToInt(EditMesh.TriangleCount() * MaxTrianglesFraction);
		}
	}

	if (bUsingEdgeLength)
	{
		Simplifier.SimplifyToEdgeLength(SimplifySettings.EdgeLength);
	}
	else if (bUsingGeometricTolerance)
	{
		// MinTrianglesFraction is the floor when ErrorTolerance drives termination; the geometric
		// constraint stops collapses earlier when the tolerance is reached.
		const int TargetTri = EditMesh.TriangleCount() * MinTrianglesFraction;
		Simplifier.SimplifyToTriangleCount(FMath::Max(TargetTri, 1));
	}
	else
	{
		// Pure triangle-count target driven by MaxTrianglesFraction (ErrorTolerance == 0).
		const int TargetTri = FMath::Max(1, FMath::FloorToInt(EditMesh.TriangleCount() * MaxTrianglesFraction));
		Simplifier.SimplifyToTriangleCount(TargetTri);
	}

	// Measure the actual deviation against the retained original surface as a symmetric
	// Hausdorff distance (max of the two one-sided maxima). With ErrorTolerance the achieved
	// value can exceed the authored tolerance (MaxTrianglesFraction may force extra collapses
	// past it), so this is the value callers should use to drive auto ScreenSize, not the
	// authored ErrorTolerance.
	double OneSidedMaxDeviation = 0.0;
	double SymmetricMaxDeviation = 0.0;
	if (ProjectionMesh.IsValid() && Spatial.IsValid())
	{
		double MinDistance = 0.0;
		double AvgDistance = 0.0;
		double RMSDeviation = 0.0;

		// Forward pass: simplified vertices -> original surface.
		TMeshQueries<FDynamicMesh3>::MeshDistanceStatistics(
			EditMesh, *Spatial,
			(const FDynamicMesh3*)nullptr,
			(const FDynamicMeshAABBTree3*)nullptr,
			/*bSymmetric*/ false,
			OneSidedMaxDeviation, MinDistance, AvgDistance, RMSDeviation);

		// Reverse pass: original vertices -> simplified surface. Needs a fresh AABB tree
		// on the post-simplification mesh.
		FDynamicMeshAABBTree3 EditMeshSpatial(&EditMesh, true);
		double ReverseMaxDistance = 0.0;
		TMeshQueries<FDynamicMesh3>::MeshDistanceStatistics(
			*ProjectionMesh, EditMeshSpatial,
			(const FDynamicMesh3*)nullptr,
			(const FDynamicMeshAABBTree3*)nullptr,
			/*bSymmetric*/ false,
			ReverseMaxDistance, MinDistance, AvgDistance, RMSDeviation);

		SymmetricMaxDeviation = FMath::Max(OneSidedMaxDeviation, ReverseMaxDistance);
	}

	return { SymmetricMaxDeviation, OneSidedMaxDeviation };
}

float ComputeAutoScreenSize(double InMaxDeviation, double InBoundsSphereRadius, float InPixelError)
{
	if (InMaxDeviation <= 0.0 || InBoundsSphereRadius <= 0.0)
	{
		return 0.0f;
	}

	// Solve for the view distance at which InMaxDeviation projects to InPixelError pixels under
	// the reference projection. The 960 factor is ScreenWidth/2 * cot(HFOV/2) with HFOV=90 and
	// Width=1920, matching CalculateViewDistance in StaticMesh.cpp.
	const double ViewDistance = (InMaxDeviation * 960.0) / FMath::Max((double)InPixelError, (double)UE_SMALL_NUMBER);

	const FPerspectiveMatrix ProjMatrix(Constants::AutoScreenSizeReferenceHalfFOV, Constants::AutoScreenSizeReferenceWidth, Constants::AutoScreenSizeReferenceHeight, 1.0f);

	// Offset the view distance by SphereRadius so the distance is to the bounds, not the centre -
	// matches the offset applied in FStaticMeshRenderData::ResolveSectionInfo (StaticMesh.cpp:3134).
	return ComputeBoundsScreenSize(
		FVector::ZeroVector,
		static_cast<float>(InBoundsSphereRadius),
		FVector(0.0, 0.0, ViewDistance + InBoundsSphereRadius),
		ProjMatrix);
}

// Inverse of ComputeAutoScreenSize: returns the world-space MaxDeviation that would project to
// InPixelError pixels at the same reference projection when the bounds occupy InScreenSize of
// the screen. Returns 0 when the requested ScreenSize is at/above the FOV-bounded ceiling
// (~2*ScreenMultiple, ~1.78 for 16:9 HFOV=90) since no positive view distance exists outside
// the bounds for those screen sizes.
double ComputeMaxDeviationForScreenSize(double InScreenSize, double InBoundsSphereRadius, float InPixelError)
{
	if (InScreenSize <= 0.0 || InBoundsSphereRadius <= 0.0)
	{
		return 0.0;
	}

	const FPerspectiveMatrix ProjMatrix(Constants::AutoScreenSizeReferenceHalfFOV, Constants::AutoScreenSizeReferenceWidth, Constants::AutoScreenSizeReferenceHeight, 1.0f);
	const double TwoScreenMultiple = 2.0 * FMath::Max(0.5 * (double)ProjMatrix.M[0][0], 0.5 * (double)ProjMatrix.M[1][1]);

	if (InScreenSize >= TwoScreenMultiple)
	{
		return 0.0;
	}

	// Invert ComputeBoundsScreenSize: Dist = 2*ScreenMultiple * R / SS, then subtract R to get
	// ViewDistance (camera offset from bounds, mirroring the +R offset in ComputeAutoScreenSize).
	const double Dist = TwoScreenMultiple * InBoundsSphereRadius / InScreenSize;
	const double ViewDistance = Dist - InBoundsSphereRadius;
	return ViewDistance * (double)InPixelError / 960.0;
}

// Resolve mode-driven derived fields. The user-facing settings declare *intent*; the simplifier
// consumes raw numbers (ErrorTolerance, MaxTrianglesFraction). For modes that derive one of those
// from ScreenSize, compute the derived value here so the downstream simplifier path stays
// mode-agnostic.
FMeshPartitionTransformerSimplificationSettings ResolveSimplifierSettings(
	const FMeshPartitionTransformerSimplificationSettings& InSettings,
	double InBoundsSphereRadius)
{
	FMeshPartitionTransformerSimplificationSettings Resolved = InSettings;

	switch (InSettings.Mode)
	{
	case EMeshPartitionSimplificationMode::AutoErrorToleranceFromScreenSize:
		if (InSettings.ScreenSize > 0.0f && InBoundsSphereRadius > 0.0)
		{
			const double DerivedTolerance = ComputeMaxDeviationForScreenSize(
				InSettings.ScreenSize, InBoundsSphereRadius, InSettings.PixelError);
			Resolved.ErrorTolerance = static_cast<float>(DerivedTolerance);
		}
		break;

	case EMeshPartitionSimplificationMode::TriangleCountFromScreenSize:
		Resolved.ErrorTolerance = 0.f;
		if (InSettings.ScreenSize > 0.0f)
		{
			Resolved.MaxTrianglesFraction = FMath::Clamp(InSettings.ScreenSize, 0.f, 1.f);
		}
		break;

	case EMeshPartitionSimplificationMode::AutoScreenSizeFromError:
	case EMeshPartitionSimplificationMode::Custom:
	default:
		break;
	}

	return Resolved;
}

} // namespace MeshPartitionStaticMeshTransformerLocals

bool FStaticMeshTransformer::Execute(MeshPartition::FTransformerContext& InTransformerContext) const
{
	UE::Tasks::TTask<TArray<UStaticMesh*>> CreateStaticMeshesTask = UE::Tasks::Launch(TEXT("FStaticMeshTransformer::CreateStaticMeshes"), [this,
		&InTransformerContext]() mutable
		{
			return CreateStaticMeshes(InTransformerContext);
		},
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri
	);

	TArray<UE::Tasks::FTask> BuildStaticMeshTasks;

	BuildStaticMeshTasks.SetNum(InTransformerContext.TransformerUnits.Num());

	for (int32 Index = 0; Index < InTransformerContext.TransformerUnits.Num(); ++Index)
	{
		BuildStaticMeshTasks[Index] = UE::Tasks::Launch(TEXT("FStaticMeshTransformer::BuildMesh"), [this,
			&InTransformerContext,
			&CreateStaticMeshesTask,
			Index]() mutable
			{
				BuildStaticMesh(InTransformerContext, CreateStaticMeshesTask, Index);
			}, UE::Tasks::Prerequisites(CreateStaticMeshesTask));
	}

	UE::Tasks::FTask FinalizeStaticMeshesTask = UE::Tasks::Launch(TEXT("FStaticMeshTransformer::FinalizeStaticMeshes"), [this,
		&CreateStaticMeshesTask,
		&InTransformerContext]() mutable
		{
			FinalizeStaticMeshes(InTransformerContext, CreateStaticMeshesTask);
		},
		UE::Tasks::Prerequisites(BuildStaticMeshTasks),
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri
	);

	FinalizeStaticMeshesTask.Wait();

	return true;
}

void FStaticMeshTransformer::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	InDependencies += CollisionProfile.Name;
	
	for (const MeshPartition::FLODSettings& LOD : LODs)
	{
		InDependencies += LOD.bReduceMesh;
		
		if (LOD.bReduceMesh)
		{
			InDependencies.AddUStructDependencyViaReflection(LOD.SimplifierSettings);
		}
	}

	InDependencies += LODStreaming;
	InDependencies += NumStreamedLODs;
	InDependencies += bCanEverAffectNavigation;
	InDependencies += bApplyMeshSkirt;
	if (bApplyMeshSkirt)
	{
		InDependencies += MeshSkirtSettings.Width;
		InDependencies += MeshSkirtSettings.PushDown;
		InDependencies += MeshSkirtSettings.PushMethod;
		InDependencies += MeshSkirtSettings.PushDirection;
		InDependencies += MeshSkirtSettings.VertexSnapTolerance;
		InDependencies += MeshSkirtSettings.BoundaryMinPerimeter;
	}
	InDependencies += bUseNanite;
	InDependencies += NaniteFallbackMode;
	InDependencies += NaniteFallbackTarget;
	InDependencies += NaniteFallbackPercentTriangles;
	InDependencies += NaniteFallbackRelativeError;
}

TArray<UStaticMesh*> FStaticMeshTransformer::CreateStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext) const
{
	check(IsInGameThread());
		   	
	TArray<UStaticMesh*> Results;
		   	
	if (InTransformerContext.bWasCancelled)
	{
		return Results;
	}
		   	
	for (const MeshPartition::FTransformerUnit& TransformerUnit : InTransformerContext.TransformerUnits)
	{
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if (!ensure(TransformerUnit.MeshData.IsValid()) || !ensure(TransformerUnit.MeshData->VertexCount() != 0) || (Section == nullptr))
		{
			// Still add nullptr to preserve 1-1 num between TransformerUnits and StaticMeshes
			Results.Add(nullptr);
			continue;
		}

		UStaticMesh* StaticMesh = MeshPartition::EditorUtils::CreateStaticMesh(Section, TEXT("MeshPartitionStaticMesh"));
		StaticMesh->SetInternalFlags(EInternalObjectFlags::Async);
		   	
		if (!bCanEverAffectNavigation)
		{
			StaticMesh->MarkAsNotHavingNavigationData();
		}

		Results.Add(StaticMesh);
	}
		   	
	return Results;
}

void FStaticMeshTransformer::BuildStaticMesh(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask, const int32 InTransformerUnitIndex) const
{
	if ((InTransformerContext.bWasCancelled) || (InTransformerContext.TransformerUnits.Num() <= InTransformerUnitIndex))
	{
		return;
	}

	UStaticMesh* StaticMesh = nullptr;
	const MeshPartition::FTransformerUnit& TransformerUnit = InTransformerContext.TransformerUnits[InTransformerUnitIndex];
	
	{
		TArray<UStaticMesh*>& StaticMeshes = InCreateStaticMeshesTask.GetResult();
					
		if (InTransformerUnitIndex >= StaticMeshes.Num())
		{
			return;
		}
					
		StaticMesh = StaticMeshes[InTransformerUnitIndex];
	}

	// static mesh can be null if the built section had no data, we would not have created the static mesh for it.
	if (StaticMesh == nullptr)
	{
		return;
	}

	if (!ensure(TransformerUnit.MeshData != nullptr))
	{
		return;
	}
			
	if (!ensure(IsValid(StaticMesh)))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot get StaticMesh.");
		return;
	}

	const FMeshData* MeshDataToConvert = TransformerUnit.MeshData.Get();

	// if a destructive mutation to the transformer mesh data must be performed,
	// we must create a local stack-copy of the mesh for that operation.
	FMeshData ModifiedMeshData;
	const bool bMustModifyTransformerMesh = TransformerUnit.bShouldRecomputeNormals || bApplyMeshSkirt;
	if (bMustModifyTransformerMesh)
	{
		ModifiedMeshData.Copy(*TransformerUnit.MeshData);
		
		if (TransformerUnit.bShouldRecomputeNormals)
		{
			ModifiedMeshData.RecomputeNormals();
		}

		if (bApplyMeshSkirt)
		{
			MeshPartition::AddMeshSkirt(ModifiedMeshData, MeshSkirtSettings);
		}
		
		MeshDataToConvert = &ModifiedMeshData;
	}

	if (LODs.Num() == 0)
	{
		StaticMesh->SetNumSourceModels(1);
		StaticMesh->CreateMeshDescription(0);
		FStaticMeshSourceModel& SourceModel = StaticMesh->GetSourceModel(0);

		// normals already computed before applying the skirt
		MeshPartition::EditorUtils::BuildSourceModel(SourceModel, *MeshDataToConvert, false, TransformerUnit.bShouldRecomputeTangents);
		return;
	}

	const int32 LODCount = LODs.Num();
	check(LODCount > 0);

	StaticMesh->SetNumSourceModels(LODCount);

	// We always author per-LOD ScreenSizes below (manual, deviation-derived, or geometric fallback),
	// so disable the engine's auto-compute path. FStaticMeshRenderData::ResolveSectionInfo otherwise
	// falls back to Pow(0.75, LODIndex) because we never populate FStaticMeshLODResources::MaxDeviation,
	// which would just race our own wire-up.
	StaticMesh->SetAutoComputeLODScreenSize(false);

	// Reference radius used to convert achieved geometric deviation into a ScreenSize threshold.
	// Computed once - bounds don't change per-LOD. Diagonal length / 2 = encompassing sphere radius.
	const UE::Geometry::FAxisAlignedBox3d MeshBounds = MeshDataToConvert->GetBounds();
	const double BoundsSphereRadius = MeshBounds.DiagonalLength() * 0.5;

	// Fill out all of the LOD source models with mesh descriptions and disable engine-side reduction
	// (each LOD already has a fully-baked MeshDescription, either copied from MeshData or simplified).
	for (int32 LODIndex = 0; LODIndex < LODCount; ++LODIndex)
	{
		const MeshPartition::FLODSettings& LODSettings = LODs[LODIndex];
		FStaticMeshSourceModel& LODSourceModel = StaticMesh->GetSourceModel(LODIndex);
		LODSourceModel.ReductionSettings = {};

		// Geometric ScreenSize fallback for LODs where neither the user nor the simplifier supplies a
		// value. Mirrors the engine's auto-compute path (StaticMesh.cpp:3115) - we disable the engine
		// fallback by calling SetAutoComputeLODScreenSize(false), so we must reproduce its progression.
		constexpr float AutoComputeLODPowerBase = 0.75f; // Mirrors UE::StaticMesh::Private::Constants::AutoComputeLODPowerBase.
		const float GeometricFallbackScreenSize = (LODIndex == 0) ? 1.0f : FMath::Pow(AutoComputeLODPowerBase, LODIndex);

		// In modes that derive simplification *from* ScreenSize, an unauthored ScreenSize (== 0) would
		// otherwise leave the resolver with nothing to feed the simplifier - it would bail with
		// "no simplification requested" despite bReduceMesh being set. Substitute the geometric
		// fallback so the resolver has a sensible target, and the final wire-up below echoes the
		// same value back into LODSourceModel.ScreenSize.Default.
		FMeshPartitionTransformerSimplificationSettings ResolverInput = LODSettings.SimplifierSettings;
		const bool bScreenSizeIsDriver =
			ResolverInput.Mode == EMeshPartitionSimplificationMode::AutoErrorToleranceFromScreenSize ||
			ResolverInput.Mode == EMeshPartitionSimplificationMode::TriangleCountFromScreenSize;
		if (bScreenSizeIsDriver && ResolverInput.ScreenSize <= 0.0f)
		{
			ResolverInput.ScreenSize = GeometricFallbackScreenSize;
			UE_LOGF(LogMegaMeshEditor, Display,
				"Static mesh transformer: LOD%d ScreenSize unauthored in ScreenSize-driver mode, using geometric fallback %.6f for simplifier input",
				LODIndex,
				GeometricFallbackScreenSize);
		}

		// Resolve mode-derived fields (e.g. ErrorTolerance from ScreenSize in mode 2,
		// MaxTrianglesFraction from ScreenSize in mode 3) before any consumer reads them.
		const FMeshPartitionTransformerSimplificationSettings ResolvedSettings =
			MeshPartitionStaticMeshTransformerLocals::ResolveSimplifierSettings(ResolverInput, BoundsSphereRadius);

		// bReduceMesh requested + non-empty input is the condition that actually drives the simplifier.
		// Tie BuildSettings to the same predicate so an empty-input reduce request doesn't desync
		// the build flags from the geometry path taken below.
		const bool bWillSimplify = LODSettings.bReduceMesh && MeshDataToConvert->TriangleCount() > 0;

		FMeshBuildSettings& BuildSettings = LODSourceModel.BuildSettings;
		// We provide pre-simplified geometry for reduced LODs, so trust the simplifier's
		// normals/tangents instead of asking the build pipeline to recompute them.

		BuildSettings.bRecomputeNormals  = false; // already applied before the skirt
		BuildSettings.bRecomputeTangents = !bWillSimplify && TransformerUnit.bShouldRecomputeTangents;
		BuildSettings.bGenerateLightmapUVs = false;
		BuildSettings.DistanceFieldResolutionScale = 0.0f;
		BuildSettings.MaxLumenMeshCards = 4;

		// Allocate a fresh empty MeshDescription on the source model - the static mesh has no
		// pre-existing cache or bulk data on creation, so GetOrCacheMeshDescription() would return null.
		FMeshDescription* LODMeshDescription = LODSourceModel.CreateMeshDescription();

		if (!ensure(LODMeshDescription != nullptr))
		{
			continue;
		}

		double AchievedMaxDeviation = 0.0;
		if (bWillSimplify)
		{
			FDynamicMesh3 DynamicMesh;
			MeshDataToConvert->ConvertToDynamicMesh(DynamicMesh);

			const int32 BeforeTriangleCount = DynamicMesh.TriangleCount();

			// Normals were already recomputed before the skirt was added (see ModifiedMeshData above),
			// so don't let the simplifier overwrite them post-weld.
			const TPair<double, double> Deviations = MeshPartitionStaticMeshTransformerLocals::SimplifyDynamicMesh(DynamicMesh, /*bShouldRecomputeNormals*/ false, TransformerUnit.bShouldRecomputeTangents, ResolvedSettings);
			AchievedMaxDeviation = Deviations.Get<0>();
			const double OneSidedDeviation = Deviations.Get<1>();

			const int32 AfterTriangleCount = DynamicMesh.TriangleCount();
			const float RetainedFraction = BeforeTriangleCount > 0 ? (float)AfterTriangleCount / (float)BeforeTriangleCount : 0.f;

			// Report achieved deviation alongside the user-authored ErrorTolerance so tuning the
			// tolerance against the actual outcome is straightforward. Achieved can exceed the
			// authored value when MaxTrianglesFraction forced extra collapses past the tolerance;
			// the retained fraction shows whether the triangle-fraction caps were the binding limit.
			// Symmetric vs one-sided diverging signals features dropped entirely by the simplifier.
			UE_LOGF(LogMegaMeshEditor, Display,
				"Static mesh transformer LOD%d simplified: %d -> %d tris (%.1f%% retained), ErrorTolerance=%.6f, achieved MaxDeviation symmetric=%.6f one-sided=%.6f",
				LODIndex,
				BeforeTriangleCount,
				AfterTriangleCount,
				RetainedFraction * 100.f,
				ResolvedSettings.ErrorTolerance,
				AchievedMaxDeviation,
				OneSidedDeviation);

			FDynamicMeshToMeshDescription Converter;
			Converter.Convert(&DynamicMesh, *LODMeshDescription, true);
		}
		else
		{
			MeshDataToConvert->ConvertToMeshDescription(*LODMeshDescription);
		}

		// Force the mesh description to precalculate indexing data structures in this async task
		// so that when this function is eventually called on the GT it will not require any significant computation.
		LODMeshDescription->BuildIndexers();

		// Always compute the auto value when we have a deviation measurement, so the log can
		// report it alongside any manually-authored ScreenSize for tuning comparisons - even
		// when this mesh isn't opting in to author ScreenSize at all.
		const float AutoScreenSize = (bWillSimplify && AchievedMaxDeviation > 0.0)
			? MeshPartitionStaticMeshTransformerLocals::ComputeAutoScreenSize(AchievedMaxDeviation, BoundsSphereRadius, ResolvedSettings.PixelError)
			: 0.0f;

		// ScreenSize wire-up. Engine auto-compute is disabled above, so every LOD gets a value from
		// here. In AutoScreenSizeFromError mode the authored ScreenSize is greyed out in the UI -
		// prefer the auto value so any stale manual entry doesn't quietly override the derived one.
		// For ScreenSize-driver modes (2/3) where the user left ScreenSize at 0, the wire-up echoes
		// back the same geometric fallback the resolver used so the LOD activates at the ScreenSize
		// it was simplified for.
		const bool bPreferAutoScreenSize = LODSettings.SimplifierSettings.Mode == EMeshPartitionSimplificationMode::AutoScreenSizeFromError;
		const TCHAR* ScreenSizeSource = TEXT("unset");
		if (LODIndex == 0)
		{
			LODSourceModel.ScreenSize.Default = 1.0f;
			ScreenSizeSource = TEXT("LOD0 anchor");
		}
		else if (bPreferAutoScreenSize && AutoScreenSize > 0.0f)
		{
			LODSourceModel.ScreenSize.Default = AutoScreenSize;
			ScreenSizeSource = TEXT("auto");
		}
		else if (LODSettings.SimplifierSettings.ScreenSize > 0.0f)
		{
			LODSourceModel.ScreenSize.Default = LODSettings.SimplifierSettings.ScreenSize;
			ScreenSizeSource = TEXT("manual");
		}
		else if (bScreenSizeIsDriver)
		{
			// Modes 2/3 with unauthored ScreenSize: keep the LOD activation in sync with what the
			// simplifier was actually told to target (ResolverInput.ScreenSize == fallback above).
			LODSourceModel.ScreenSize.Default = GeometricFallbackScreenSize;
			ScreenSizeSource = TEXT("fallback");
		}
		else if (AutoScreenSize > 0.0f)
		{
			LODSourceModel.ScreenSize.Default = AutoScreenSize;
			ScreenSizeSource = TEXT("auto");
		}
		else
		{
			LODSourceModel.ScreenSize.Default = GeometricFallbackScreenSize;
			ScreenSizeSource = TEXT("fallback");
		}

		UE_LOGF(LogMegaMeshEditor, Display,
			"Static mesh transformer: LOD%d ScreenSize=%.6f (source: %ls, calculated (from actual geometric deviation and PixelError): %.6f)",
			LODIndex,
			LODSourceModel.ScreenSize.Default,
			ScreenSizeSource,
			AutoScreenSize);

		// Use the hash as the guid to ensure that given the same input mesh data, we always hit the cache for the static mesh/nanite build.
		LODSourceModel.CommitMeshDescription(/* bUseHashAsGuid */ true);
	}

	StaticMesh->SetLODStreaming(LODStreaming);
	StaticMesh->GetNumStreamedLODs() = NumStreamedLODs;
}

void FStaticMeshTransformer::FinalizeStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask) const
{
	check(IsInGameThread());
		   
	TArray<UStaticMesh*>& StaticMeshes = InCreateStaticMeshesTask.GetResult();

	for (int32 Index = 0; Index < InTransformerContext.TransformerUnits.Num(); ++Index)
	{
		const MeshPartition::FTransformerUnit& TransformerUnit = InTransformerContext.TransformerUnits[Index];
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if ((Section == nullptr) || !ensure(Index < StaticMeshes.Num()))
		{
			continue;
		}

		UStaticMesh* StaticMesh = StaticMeshes[Index];

		if (StaticMesh != nullptr)
		{
			StaticMesh->ClearInternalFlags(EInternalObjectFlags::Async);
			ForEachObjectWithOuter(StaticMesh, [](UObject* Object)
			{
			   Object->ClearInternalFlags(EInternalObjectFlags::Async);
			});
		}

		FMeshDescription* MeshDescription = (StaticMesh != nullptr) ? StaticMesh->GetMeshDescription(0) : nullptr;

		if (!InTransformerContext.bWasCancelled && (MeshDescription != nullptr) && (MeshDescription->Vertices().Num() != 0))
		{
			MeshPartition::EditorUtils::FFinalizeStaticMeshParams Params;

			Params.StaticMesh = StaticMesh;
			Params.CollisionProfile = CollisionProfile;
			Params.NumLODs = LODs.Num();
			Params.bCanEverAffectNavigation = bCanEverAffectNavigation;
			Params.bSetupSections = true;
			Params.bUseNanite = bUseNanite;
			Params.NaniteFallbackMode = NaniteFallbackMode;
			Params.NaniteFallbackTarget = NaniteFallbackTarget;
			Params.NaniteFallbackPercentTriangles = NaniteFallbackPercentTriangles;
			Params.NaniteFallbackRelativeError = NaniteFallbackRelativeError;

			FStaticMeshDescriptor Descriptor;
			Descriptor.CollisionProfileName = CollisionProfile.Name;
			Descriptor.bCanEverAffectNavigation = bCanEverAffectNavigation;
			Descriptor.UVRegion = TransformerUnit.MeshData->GetUVRegion();

			if (MeshPartition::APreviewSection* PreviewSection = Cast<MeshPartition::APreviewSection>(Section))
			{
				Params.Material = PreviewSection->GetMaterialInstance();
				MeshPartition::EditorUtils::FinalizeStaticMesh(Params);

				PreviewSection->AddMesh(StaticMesh, Descriptor);
			}
			else if (MeshPartition::ACompiledSection* CompiledSection = Cast<MeshPartition::ACompiledSection>(Section))
			{
				Params.Material = CompiledSection->GetMaterialInstance();
				MeshPartition::EditorUtils::FinalizeStaticMesh(Params);

				CompiledSection->AddStaticMesh(StaticMesh, Descriptor);
			}
		}
	}
}

} // namespace UE::MeshPartition
