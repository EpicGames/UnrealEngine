// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionSubsectionTransformer.h"

#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionGridSettings.h"
#include "MeshPartitionMeshBuilder.h"

namespace UE::MeshPartition
{

namespace MeshPartitionSubsectionTransformerLocals
{
	void AddSubSections(const MeshPartition::FTransformerUnit& InTransformerUnit, AActor* InSection, const uint32 InSubSectionSize, TArray<MeshPartition::FTransformerUnit>& OutResults)
	{
		const MeshPartition::FMeshData& MeshData = *InTransformerUnit.MeshData;

		if (InSubSectionSize == 0)
		{
			return;
		}

		constexpr bool bFilterEmptyMeshes = true;
		const MeshPartition::FGridSettings GridSettings { .CellSize = InSubSectionSize, .bIs2D = false };
		// Subsection splits operate in mesh-local space with no WP-grid alignment intent — Identity is correct here.
		const UE::MeshPartition::GridHelpers::FGridDimensions Grid = UE::MeshPartition::GridHelpers::ComputeGridDimensions(FBox(MeshData.GetBounds()), GridSettings, FTransform::Identity);
		TArray<MeshPartition::FMeshData> SubSections = UE::MeshPartition::BuildHelpers::BuildSections(MeshData, Grid, bFilterEmptyMeshes);

		OutResults.Reserve(OutResults.Num() + SubSections.Num());  

		for (MeshPartition::FMeshData& SubSection : SubSections)
		{
			OutResults.Add(MeshPartition::MakeTransformerUnit(InSection,
															  MakeShared<const MeshPartition::FMeshData>(MoveTemp(SubSection)),
															  InTransformerUnit.bShouldRecomputeNormals,
															  InTransformerUnit.bShouldRecomputeTangents));
		}
	}
	
} // namespace MeshPartitionSubsectionTransformerLocals

bool FSubsectionTransformer::Execute(MeshPartition::FTransformerContext& InTransformerContext) const
{
	TArray<MeshPartition::FTransformerUnit> Results;

	for (const MeshPartition::FTransformerUnit& TransformerUnit : InTransformerContext.TransformerUnits)
	{
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if (Section == nullptr)
		{
			continue;
		}

		MeshPartitionSubsectionTransformerLocals::AddSubSections(TransformerUnit, Section, SubSectionSize, Results);
	}

	if (!Results.IsEmpty())
	{
		InTransformerContext.TransformerUnits = MoveTemp(Results);
	}

	return true;
}

void FSubsectionTransformer::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	InDependencies += SubSectionSize;
}

} // namespace UE::MeshPartition
