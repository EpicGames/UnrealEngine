// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionWPActorPropertiesTransformer.h"

#include "MeshPartitionDependencyInterface.h"
#include "MeshPartitionEditorModule.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/HLOD/HLODLayer.h"
#include "WorldPartition/DataLayer/ExternalDataLayerHelper.h"

namespace UE::MeshPartition
{

namespace MeshPartitionWPActorPropertiesTransformerLocals
{
} // namespace MeshPartitionWPActorPropertiesTransformerLocals

bool FWPActorPropertiesTransformer::Execute(MeshPartition::FTransformerContext& InTransformerContext) const
{
	UE::Tasks::FTask Dummy;

	UE::Tasks::FTask SetActorPropertiesTask = UE::Tasks::Launch(TEXT("FWPActorPropertiesTransformer::SetActorProperties"), [this,
		&InTransformerContext]() mutable
		{
			SetActorProperties(InTransformerContext);
		},
		UE::Tasks::Prerequisites(Dummy),
		UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);

	SetActorPropertiesTask.Wait();

	return true;
}

void FWPActorPropertiesTransformer::GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const
{
	InDependencies += RuntimeGrid;
	InDependencies += bIsSpatiallyLoaded;

	for (const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset : DataLayerAssets)
	{
		if (DataLayerAsset.IsNull())
		{
			continue;
		}

		InDependencies += DataLayerAsset.GetLongPackageName();
	}
	
	if (!HLODLayer.IsNull())
	{
		InDependencies += HLODLayer.GetLongPackageName();
	}

	InDependencies += bIncludeActorInHLOD;
}

void FWPActorPropertiesTransformer::SetActorProperties(const MeshPartition::FTransformerContext& InTransformerContext) const
{
	TSet<TWeakObjectPtr<AActor>> Sections;

	if (InTransformerContext.bWasCancelled)
	{
		return;
	}

	for (const MeshPartition::FTransformerUnit& TransformerUnit : InTransformerContext.TransformerUnits)
	{
		AActor* Section = MeshPartition::GetSectionChecked(TransformerUnit);

		if (Section == nullptr)
		{
			continue;
		}

		Sections.Emplace(TransformerUnit.Section);
	}

	for (const TWeakObjectPtr<AActor>& SectionPtr : Sections)
	{
		AActor* Section = SectionPtr.Get();

		if (Section == nullptr)
		{
			continue;
		}

		Section->SetIsSpatiallyLoaded(bIsSpatiallyLoaded);
		Section->SetRuntimeGrid(RuntimeGrid);

		UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(Section);

		if (DataLayerManager != nullptr)
		{
			for (const TSoftObjectPtr<UDataLayerAsset>& DataLayerAsset : DataLayerAssets)
			{
				if (DataLayerAsset.IsNull())
				{
					continue;
				}

				DataLayerAsset.LoadSynchronous();

				// LoadSynchronous can tick the GC; re-resolve the section in case undo/level-stream destroyed it.
				Section = SectionPtr.Get();

				if (Section == nullptr)
				{
					break;
				}

				const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromAsset(DataLayerAsset.Get());

				if (DataLayerInstance == nullptr)
				{
					UE_LOGF(LogMegaMeshEditor, Warning, "Cannot retrieve UDataLayerInstance from data layer asset %ls.", *DataLayerAsset.ToString());
					continue;
				}

				if (const UExternalDataLayerInstance* ExternalDataLayer = Cast<UExternalDataLayerInstance>(DataLayerInstance))
				{
					FText FailureReason;
					FExternalDataLayerHelper::MoveActorsToExternalDataLayer( { Section }, { ExternalDataLayer, true }, &FailureReason);
					if (!FailureReason.IsEmpty())
					{
						UE_LOGF(LogMegaMeshEditor, Warning, "Failed to move to external data layer: %ls", *FailureReason.ToString());
					}
				}
				else
				{
					DataLayerInstance->AddActor(Section);
				}
			}
		}

		// The inner loop may have invalidated the section; re-resolve before HLOD work.
		Section = SectionPtr.Get();

		if (Section == nullptr)
		{
			continue;
		}

		if (!HLODLayer.IsNull())
		{
			HLODLayer.LoadSynchronous();

			// LoadSynchronous can tick the GC; re-resolve before further mutation.
			Section = SectionPtr.Get();

			if (Section == nullptr)
			{
				continue;
			}
		}

		Section->SetHLODLayer(HLODLayer.Get());

		Section->bEnableAutoLODGeneration = bIncludeActorInHLOD;
	}
}

} // namespace UE::MeshPartition
