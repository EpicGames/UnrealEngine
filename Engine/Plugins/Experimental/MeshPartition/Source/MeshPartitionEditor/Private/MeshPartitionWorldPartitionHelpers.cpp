// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionWorldPartitionHelpers.h"

#include "EditorBuildUtils.h"
#include "Engine/LevelStreaming.h"
#include "Engine/StaticMesh.h"
#include "MeshPartition.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionGridSettings.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionModifierTaskGraph.h"
#include "MeshPartitionTransformerPipeline.h"
#include "MeshPartitionWPActorPropertiesTransformer.h"
#include "Misc/App.h"
#include "SourceControlHelpers.h"
#include "UObject/SavePackage.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "WorldPartition/WorldPartitionRuntimeHash.h"
#include "Algo/ForEach.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "LevelInstance/LevelInstanceActor.h"

namespace UE::MeshPartition::WorldPartitionHelpers
{
	namespace WorldPartitionHelpersLocals
	{
		MeshPartition::FGridSettings ResolveGridSettings(const UWorld* InWorld, const FName InGridName)
		{
			// NAME_None is valid: WP runtime hashes map it to their first/main grid.

			if (InWorld == nullptr)
			{
				return {};
			}

			const UWorldPartition* WorldPartition = InWorld->GetWorldPartition();

			if (WorldPartition == nullptr)
			{
				return {};
			}

			const UWorldPartitionRuntimeHash* RuntimeHash = WorldPartition->RuntimeHash;

			if (RuntimeHash == nullptr)
			{
				return {};
			}

			const TOptional<FFixedGridInfo> Info = RuntimeHash->GetFixedGridInfo(InGridName);

			if (!Info.IsSet())
			{
				return {};
			}

			// 2D grids ignore Origin.Z downstream (cell coords are X/Y-only, Z column spans the full input range).
			// Zero it here so the stored FGridSettings.WorldOriginOffset always equals the effective Origin -- prevents
			// spurious BuildVariantHash differences when Origin.Z is edited on a 2D grid.
			FVector EffectiveOrigin = Info->Origin;

			if (Info->bIs2D)
			{
				EffectiveOrigin.Z = 0.0;
			}

			return { Info->CellSize, Info->bIs2D, EffectiveOrigin };
		}
	}

	MeshPartition::FGridSettings ResolveGridFromPipeline(const MeshPartition::FCompiledSectionBuildVariant& InBuildVariant, const UWorld* InWorld)
	{
		if (InBuildVariant.TransformerPipeline == nullptr)
		{
			return {};
		}

		// Find the last FWPActorPropertiesTransformer in the pipeline (iterate in reverse)
		const TArray<TInstancedStruct<MeshPartition::FTransformer>>& Transformers = InBuildVariant.TransformerPipeline->GetTransformers();

		for (int32 TransformerIndex = Transformers.Num() - 1; TransformerIndex >= 0; --TransformerIndex)
		{
			if (Transformers[TransformerIndex].GetScriptStruct() == MeshPartition::FWPActorPropertiesTransformer::StaticStruct())
			{
				const FName GridName = Transformers[TransformerIndex].Get<MeshPartition::FWPActorPropertiesTransformer>().GetRuntimeGrid();

				if (MeshPartition::FGridSettings Resolved = WorldPartitionHelpersLocals::ResolveGridSettings(InWorld, GridName); Resolved.IsGridSplit())
				{
					return Resolved;
				}

				// Fallback: if the named grid was not found, try the main grid
				if (!GridName.IsNone())
				{
					if (MeshPartition::FGridSettings MainResolved = WorldPartitionHelpersLocals::ResolveGridSettings(InWorld, NAME_None); MainResolved.IsGridSplit())
					{
						UE_LOGF(LogMegaMeshEditor, Warning, "WP grid '%ls' not found -- falling back to main grid (cell size %u, origin %ls).",
								*GridName.ToString(), MainResolved.CellSize, *MainResolved.WorldOriginOffset.ToString());

						return MainResolved;
					}
				}

				UE_LOGF(LogMegaMeshEditor, Warning, "Could not resolve WP grid cell size for grid '%ls' — skipping grid-aligned splitting.", *GridName.ToString());

				return {};
			}
		}

		return {};
	}

	TArray<FGridCellEstimate> EstimateGridCells(const FBox& InGroupBounds, const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld)
	{
		TArray<FGridCellEstimate> Cells;

		if (!InGridSettings.IsGridSplit() || !InGroupBounds.IsValid)
		{
			return Cells;
		}

		const GridHelpers::FGridDimensions Grid = GridHelpers::ComputeGridDimensions(InGroupBounds, InGridSettings, InLocalToWorld);

		Cells.Reserve(Grid.TotalCells);

		UE_LOGF(LogMegaMeshEditor, Verbose, "EstimateGridCells: Bounds [%.1f,%.1f,%.1f]-[%.1f,%.1f,%.1f] CellSize=%u bIs2D=%d -> %dx%dx%d = %d cells",
			   InGroupBounds.Min.X, InGroupBounds.Min.Y, InGroupBounds.Min.Z,
			   InGroupBounds.Max.X, InGroupBounds.Max.Y, InGroupBounds.Max.Z,
			   InGridSettings.CellSize, InGridSettings.bIs2D ? 1 : 0, Grid.CellNumber.X, Grid.CellNumber.Y, Grid.CellNumber.Z, Grid.TotalCells);

		for (int32 SectionIndex = 0; SectionIndex < Grid.TotalCells; ++SectionIndex)
		{
			const int32 IndexX = SectionIndex % Grid.CellNumber.X;
			const int32 IndexY = (SectionIndex / Grid.CellNumber.X) % Grid.CellNumber.Y;
			const int32 IndexZ = SectionIndex / (Grid.CellNumber.X * Grid.CellNumber.Y);

			const FIntVector CellCoord = Grid.OriginCoord + FIntVector(IndexX, IndexY, IndexZ);
			const FVector CellMin = Grid.SnappedMin + FVector(IndexX, IndexY, IndexZ) * Grid.CellExtent;
			const FVector CellMax = CellMin + Grid.CellExtent;

			Cells.Add({ CellCoord, FBox(CellMin, CellMax) });
		}

		return Cells;
	}

	bool EnsureLevelInstanceActorIsRegistered(AActor* Actor, UWorld* World)
	{
		if (ALevelInstance* LevelInstance = Cast<ALevelInstance>(Actor))
		{
			check(LevelInstance->GetWorld() == World);
			if (ULevelInstanceSubsystem* LIS = World->GetSubsystem<ULevelInstanceSubsystem>())
			{
				ensure(LevelInstance->HasActorRegisteredAllComponents());
				return true;
			}
		}
		return false;
	}

	TSet<ULevelStreaming*> LoadAllLevelInstances(UWorldPartition* WorldPartition, TSet<FWorldPartitionReference>& InOutActorRefs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LoadAllLevelInstances);

		check(WorldPartition);
		UWorld* World = WorldPartition->GetWorld();
		check(World);

		TSet<ULevelStreaming*> ProcessedLevelStreamings;

		// first make sure we've registered all existing level instance actors..
		for (FWorldPartitionReference& WPRef : InOutActorRefs)
		{
			EnsureLevelInstanceActorIsRegistered(WPRef.GetActor(), World);
		}

		bool bProcessedNewActors = true;
		while (bProcessedNewActors)
		{
			bProcessedNewActors = false;

			// Make sure all Level Instances loads have completed
			World->BlockTillLevelStreamingCompleted();

			// For each Streaming Level, make sure to load its actors if it is a World Partition
			for (ULevelStreaming* LevelStreaming : World->GetStreamingLevels())
			{
				if (!ProcessedLevelStreamings.Contains(LevelStreaming))
				{
					ProcessedLevelStreamings.Add(LevelStreaming);

					if (ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel())
					{
						if (UWorldPartition* LoadedLevelWorldPartition = FWorldPartitionHelpers::GetWorldPartition(LoadedLevel))
						{
							// TODO [chris.tchou] : this loads ALL actors within the level instance.  If we wanted to handle the complexity, we could load just the actors we care about.
							TArray<FWorldPartitionReference> ActorReferences;
							LoadedLevelWorldPartition->LoadAllActors(ActorReferences);
							UE_LOGF(LogMegaMeshEditor, Verbose, "  LoadedLevel %d '%ls' '%ls' : %d streaming actors", ProcessedLevelStreamings.Num(), *LevelStreaming->GetName(), *LoadedLevel->GetName(), ActorReferences.Num());

							// check if any of the loaded actors were level instances, and if so, check that they are registered
							for (FWorldPartitionReference& WPRef : ActorReferences)
							{
								EnsureLevelInstanceActorIsRegistered(WPRef.GetActor(), World);
							}

							InOutActorRefs.Append(MoveTemp(ActorReferences));
							bProcessedNewActors = true;
						}
						else
						{
							UE_LOGF(LogMegaMeshEditor, Verbose, "  LoadedLevel %d '%ls' '%ls' - no world partition", ProcessedLevelStreamings.Num(), *LevelStreaming->GetName(), *LoadedLevel->GetName());
						}
					}
					else
					{
						UE_LOGF(LogMegaMeshEditor, Verbose, "  LoadedLevel %d '%ls' - no loaded level", ProcessedLevelStreamings.Num(), *LevelStreaming->GetName());
					}
				}
			}
		}

		return ProcessedLevelStreamings;
	}

	void LoadAllActorsFromStreamingLevels(
		MeshPartition::FModifierGroup& InGroup,
		const AMeshPartition* InMegaMesh,
		UWorldPartition* WorldPartition,
		TSet<FWorldPartitionReference>& InOutActorRefs,
		TFunctionRef<void(AActor* Modifier, bool bIsInLevelInstance)> PerActorCallback,
		TFunctionRef<void(MeshPartition::UModifierComponent* Modifier, bool bIsInLevelInstance)> PerModifierCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LoadAllActorsFromStreamingLevels);

		// Note that this currently adds ALL actors within the child level instances to InOutActorRefs (even if not megamesh related)
		TSet<ULevelStreaming*> ProcessedLevelStreamings = LoadAllLevelInstances(WorldPartition, InOutActorRefs);

		const bool bIsInLevelInstance = true;

		// By now, all level instances which are relevant to this group are loaded, and actors inside of these level instances will be loaded.
		// The builder can now search through the newly processed level streamings and check if there are modifiers which should be affecting this group.
		for (ULevelStreaming* LevelStreaming : ProcessedLevelStreamings)
		{
			ULevel* LoadedLevel = LevelStreaming->GetLoadedLevel();
			
			if (LoadedLevel == nullptr)
			{
				// The level may not be loaded (e.g. it was streamed out between
				// LoadAllLevelInstances and this iteration).
				continue;
			}

			for (AActor* Actor : LoadedLevel->Actors)
			{
				// Ignore actors that failed to load for some reason
				if (Actor == nullptr)
				{
					continue;
				}

				TInlineComponentArray<MeshPartition::UModifierComponent*> ActorModifiers(Actor);
				int32 FoundModifierCount = 0;
				for (MeshPartition::UModifierComponent* Modifier : ActorModifiers)
				{
					if (Modifier->GetAffectedMeshPartition() != InMegaMesh)
					{
						continue;
					}
					if (Modifier->IsBase())
					{
						UE_LOGF(LogMegaMeshEditor, Warning, "Modifier %ls in LevelInstanced Level %ls is a BaseModifier and will be ignored.  BaseModifiers are not supported inside of LevelInstances.",
							   *Modifier->GetName(),
							   *LoadedLevel->GetName());
						continue;
					}

					// If this (level-instanced) modifier intersects this group, add it to the modifiers list
					if (Modifier->IntersectsAnyBounds({ InGroup.ComputeBaseBounds() }))
					{
						TConstArrayView<FName> ModifierTypePriorities = InMegaMesh && InMegaMesh->GetMeshPartitionDefinition() ? InMegaMesh->GetMeshPartitionDefinition()->GetModifierTypePriorities() : TConstArrayView<FName>();
						InGroup.AddModifierSorted(ModifierTypePriorities, *Modifier);
					}

					PerModifierCallback(Modifier, bIsInLevelInstance);
					FoundModifierCount++;
				}

				if (FoundModifierCount > 0)
				{
					PerActorCallback(Actor, bIsInLevelInstance);
				}
			}
		}
	}

	void LoadGroupModifiersViaWorldPartitionRef(MeshPartition::FModifierGroup& InGroup, UWorldPartition* InWorldPartition, TSet<FWorldPartitionReference>& OutActorRefs)
	{
		// Load the modifiers by constructing WorldPartitionRefs to the actors containing them (guarantees the actor stays loaded until we free the refs)
		auto LoadOwnerActor = [InWorldPartition, &OutActorRefs](const MeshPartition::FModifierDesc& Descriptor)
			{
				const FWorldPartitionReference ActorRef(InWorldPartition, Descriptor.OwnerGuid);
				AActor* Actor = ActorRef.GetActor();
				if (Actor)
				{
					OutActorRefs.Add(ActorRef);
				}
				else
				{
					UE_LOGF(LogMegaMeshEditor, Warning, " FAILED to load WP actor with guid [%ls] class:%ls", *Descriptor.OwnerGuid.ToString(), *Descriptor.ClassPath.GetAssetPath().GetAssetName().ToString());
				}
			};

		Algo::ForEach(InGroup.AllModifierDescs(), LoadOwnerActor);
	}

	void LoadAllRelevantMegaMeshActors(MeshPartition::FModifierGroup& InGroup, const AMeshPartition* InMegaMesh, TSet<FWorldPartitionReference>& OutActorRefs, TFunctionRef<void(AActor* Modifier, bool bIsInLevelInstance)> PerActorCallback, TFunctionRef<void(MeshPartition::UModifierComponent* Modifier, bool bIsInLevelInstance)> PerModifierCallback)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(LoadAllRelevantMegaMeshActors);

		if (InMegaMesh == nullptr)
		{
			return;
		}

		UWorld* World = InMegaMesh->GetWorld();
		UWorldPartition* WorldPartition = World->GetWorldPartition();

		LoadGroupModifiersViaWorldPartitionRef(InGroup, WorldPartition, OutActorRefs);

		// Build LookupTable and update ref counts
		const bool bIsNOTInLevelInstance = false;
		for (const FWorldPartitionReference& ActorRef : OutActorRefs)
		{
			AActor* Actor = ActorRef.GetActor();
			if (Actor == nullptr)
			{
				continue;
			}

			TInlineComponentArray<MeshPartition::UModifierComponent*> ActorModifiers(Actor);
			int32 FoundModifierCount = 0;
			for (MeshPartition::UModifierComponent* Modifier : ActorModifiers)
			{
				if (Modifier->GetAffectedMeshPartition() != InMegaMesh)
				{
					continue;
				}

				PerModifierCallback(Modifier, bIsNOTInLevelInstance);
				FoundModifierCount++;
			}
			if (FoundModifierCount > 0)
			{
				PerActorCallback(Actor, bIsNOTInLevelInstance);
			}
		}

		// By this point, all actors owning a modifier component that are part of this group will have been loaded.		
		// Some of these actors may be LevelInstances.  If so, we need to make sure that they pull in their instanced level.
		// Since streamed in levels will not have actor descriptors that are easily iterable as the modifier groups were being created, they must be searched while loaded for their modifiers:
		LoadAllActorsFromStreamingLevels(InGroup, InMegaMesh, WorldPartition, OutActorRefs, PerActorCallback, PerModifierCallback);

		InGroup.RemoveDisabledModifiers();

		// Now that all modifiers are loaded, resolved all of the modifier pointers (and ensure they are resolved in the target world of our mega mesh)
		InGroup.ProgressToState(MeshPartition::FModifierGroup::EState::ModifiersResolved);

		// check that all modifier descriptors were resolved in the same World as InMegaMesh
		InGroup.ForAllModifiers([World](MeshPartition::UModifierComponent* ModifierComponent)
			{
				if (ensure(ModifierComponent != nullptr))
				{
					UWorld* ModifierWorld = ModifierComponent->GetWorld();
					ensure(ModifierWorld == World);
				}
				return true;
			});
	}
} // namespace UE::MeshPartition::WorldPartitionHelpers