// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionWorldUpdater.h"
#include "EngineUtils.h"
#include "MeshPartitionEditorModule.h" // LogMegaMeshEditor
#include "MeshPartitionDependencyContext.h"
#include "MeshPartitionEditorSubsystem.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Engine/StaticMesh.h"
#include "Editor/EditorEngine.h"
#include "Engine/Texture2DArray.h"
#include "MeshPartitionEditorUtils.h"
#include "MeshPartitionWorldPartitionHelpers.h"
#include "MeshPartition.h"

#define LOCTEXT_NAMESPACE "FMeshPartitionWorldUpdater"

extern UNREALED_API UEditorEngine* GEditor;

namespace UE::MeshPartition
{
	namespace MeshPartitionWorldUpdaterLocals
	{
		// Spawns a PIE placeholder section at world Identity (NOT at the parent MeshPartition's
		// transform) and stores its streaming bounds in world space for World Partition visibility.
		// The build pipeline produces cell meshes in MeshPartition-local space, so spawning at the
		// parent's transform would double the offset once those meshes are rendered.
		ACompiledSection* SpawnPlaceholderSection(UWorld* InWorld, const FActorSpawnParameters& InSpawnParams, const FBox& InStreamingBoundsWorld)
		{
			check(InWorld);

			// Callers always set bNoFail = true, so SpawnActor must return a valid actor.
			// If a future caller drops bNoFail, this check makes the violation loud rather than a deref crash.
			ACompiledSection* Placeholder = InWorld->SpawnActor<ACompiledSection>(ACompiledSection::StaticClass(), FTransform::Identity, InSpawnParams);
			check(Placeholder);

			if (Placeholder->GetRootComponent())
			{
				Placeholder->GetRootComponent()->SetMobility(EComponentMobility::Static);
			}

			Placeholder->SetIsPlaceholder(true);
			Placeholder->SetPlaceholderStreamingBounds(InStreamingBoundsWorld);

			// Post-condition: we spawned at Identity, so the section's world transform must still
			// be Identity. Catches any future change that moves the placeholder before we return.
			check(Placeholder->GetActorTransform().Equals(FTransform::Identity));

			return Placeholder;
		}
	}

	FMeshPartitionWorldUpdater::FMeshPartitionWorldUpdater(UWorld* InWorld, EUpdateMode InUpdateMode)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshPartitionWorldUpdater::FMeshPartitionWorldUpdater);

		// create a unique build key to identify placeholders created
		UpdateMode = InUpdateMode;
		BuildKey = FGuid::NewGuid();

		// we will need the asset registry to check the up-to-date status of compiled sections (we will force completion of any relevant pending scans before checking)
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry* AssetRegistry = &AssetRegistryModule.Get();
		check(AssetRegistry);

		MeshPartition::FModifierDescriptorCache* DescriptorCache = UMeshPartitionEditorSubsystem::GetDescriptorCache();
		check(DescriptorCache);

		// in editor worlds, update the descriptor cache immediately
		if (InWorld->IsEditorWorld())
		{
			DescriptorCache->UpdateCacheForAllMeshPartitionsInWorld(InWorld);
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Verbose, "Constructing FMeshPartitionWorldUpdater on a non-editor world - this relies on previously cached descriptor data");
		}

		// create an FMeshPartitionUpdater for each of the mesh partitions in the editor world
		for (AMeshPartition* MeshPartition : TActorRange<AMeshPartition>(InWorld))
		{
			if (MeshPartition)
			{
				const FGuid& MeshPartitionGUID = MeshPartition->GetActorGuid();

				const UMeshPartitionDefinition* MPDefinition = MeshPartition->GetMeshPartitionDefinition();
				if (MPDefinition == nullptr)
				{
					UE_LOGF(LogMegaMeshEditor, Verbose, "No Mesh Partition Definition provided for %ls (%.6ls), using default settings.", *MeshPartition->GetActorNameOrLabel(), *MeshPartitionGUID.ToString());
					MPDefinition = UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
				}
				check(MPDefinition);

				TSharedPtr<MeshPartition::FCachedDescriptors> MeshPartitionAllCachedDescriptors = DescriptorCache->GetCachedDescriptors(MeshPartition);
				if (!MeshPartitionAllCachedDescriptors.IsValid())
				{
					UE_LOGF(LogMegaMeshEditor, Verbose, "Ran without descriptors cached for MeshPartition '%ls', attempting to update the cache in the current world", *MeshPartition->GetActorNameOrLabel());
					DescriptorCache->UpdateCacheForAllMeshPartitionsInWorld(InWorld);
					MeshPartitionAllCachedDescriptors = DescriptorCache->GetCachedDescriptors(MeshPartition);
					if (!MeshPartitionAllCachedDescriptors.IsValid())
					{
						if (UpdateMode == EUpdateMode::ForPIE)
						{
							// In PIE, an unsaved MeshPartition won't have serialized descriptors yet. Skip it
							// gracefully - the user has already been warned about unsaved modifiers via dialog.
							UE_LOGF(LogMegaMeshEditor, Warning, "Skipping MeshPartition '%ls' (%.6ls) for PIE - no cached descriptors available. The asset may need to be saved first.",
								*MeshPartition->GetActorNameOrLabel(), *MeshPartitionGUID.ToString());
							continue;
						}
						else
						{
							// During compile or cook, missing descriptors is unexpected and should not be silently ignored.
							UE_LOGF(LogMegaMeshEditor, Warning, "MeshPartition '%ls' (%ls) has no cached descriptors during %ls. Ensure the asset is saved before compiling or cooking.",
								*MeshPartition->GetActorNameOrLabel(), *MeshPartitionGUID.ToString(),
								UpdateMode == EUpdateMode::ForCook ? TEXT("cook") : TEXT("compile"));
							continue;
						}
					}
				}
				check(MeshPartitionAllCachedDescriptors->GetMeshPartitionGUID() == MeshPartition->GetActorGuid());

				TArray<FName> BuildVariantNames;
				switch (UpdateMode)
				{
					case EUpdateMode::ForPIE:
						{
							// in PIE, process variants for the preview platform only
							FName PreviewPlatformName;
							ITargetPlatform* PreviewTargetPlatform = UE::MeshPartition::EditorUtils::GetPreviewPlatform(PreviewPlatformName);
							BuildVariantNames = MPDefinition->GetCompiledSectionBuildVariantNamesForCurrentEditorPreview(PreviewPlatformName, PreviewTargetPlatform);
						}
						break;
					case EUpdateMode::ForCompile:
						// when compiling, process all build variants
						BuildVariantNames = MPDefinition->GetAllCompiledSectionBuildVariantNames();
						break;
					case EUpdateMode::ForCook:
						{
							// when cooking, prepare build variants for all target platforms
							TArray<ITargetPlatform*> TargetPlatforms = GetTargetPlatformManager()->GetActiveTargetPlatforms();
							BuildVariantNames = MPDefinition->GetCompiledSectionBuildVariantNamesForPlatforms(TargetPlatforms);
						}
						break;
				}
				check(!BuildVariantNames.IsEmpty());

				// add an entry for this mesh partition
				FMeshPartitionUpdater& MeshPartitionUpdater = MeshPartitionUpdaters.Add(
					MeshPartitionGUID,
					FMeshPartitionUpdater(BuildKey, MeshPartition, *MPDefinition, MeshPartitionAllCachedDescriptors, TConstArrayView<FName>(BuildVariantNames), *AssetRegistry, InWorld));

				// in PIE, we create placeholders for any missing bases.  In cook, we just warn that they are missing.
				if (UpdateMode == EUpdateMode::ForCook)
				{
					MeshPartitionUpdater.ForAllBuildVariants([](const FBuildVariantUpdater& BuildVariantUpdater)
						{
							if (BuildVariantUpdater.GetBaseModifiersNeedingCreation().Num() > 0)
							{
								UE_LOGF(LogMegaMeshEditor, Display, "Mesh Partition '%ls' BuildVariant:%ls is missing compiled sections for %d base modifiers.  Cooked builds will not show these missing regions until the compiled sections are built.",
									*BuildVariantUpdater.GetMeshPartitionPath().ToString(),
									*BuildVariantUpdater.GetBuildVariantName().ToString(),
									BuildVariantUpdater.GetBaseModifiersNeedingCreation().Num());
							}
							return true;
						});
				}
			}
		}
	}

	FMeshPartitionUpdater::FMeshPartitionUpdater(const FGuid& InBuildKey, const AMeshPartition* InMeshPartition, const UMeshPartitionDefinition& InMeshPartitionDefinition, TSharedPtr<MeshPartition::FCachedDescriptors> InMeshPartitionAllCachedDescriptors, TConstArrayView<FName> InTargetBuildVariantNames, IAssetRegistry& InAssetRegistry, const UWorld* InWorld)
		: AllCachedDescriptors(InMeshPartitionAllCachedDescriptors)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::FMeshPartitionUpdater::FMeshPartitionUpdater);

		check(AllCachedDescriptors.IsValid());
		check(InMeshPartition != nullptr);

		MeshPartition = InMeshPartition;

		// create an FBuildVariantUpdater for each of the target build variants in the mesh partition
		for (const FName& TargetBuildVariantName : InTargetBuildVariantNames)
		{
			const MeshPartition::FModifierGrouping* TargetGrouping = AllCachedDescriptors->GetCachedBuildVariant(TargetBuildVariantName);
			if (!TargetGrouping)
			{
				UE_LOGF(LogMegaMeshEditor, Error, "Could not find build variant '%ls' in the cached descriptors, this may result in incorrect behaviors in MeshPartition", *TargetBuildVariantName.ToString());
			}
			else
			{
				check(TargetGrouping->GetBuildVariantName() == TargetBuildVariantName);
				BuildVariantUpdaters.Emplace(TargetBuildVariantName, FBuildVariantUpdater(InBuildKey, InMeshPartition, InMeshPartitionDefinition, *TargetGrouping, AllCachedDescriptors, InAssetRegistry, InWorld));
			}
		}

		// grab the compiled section descriptor array, initialize the corresponding status array with default state, and initialize the lookup table for compiled sections
		CompiledSections = AllCachedDescriptors->GetCompiledSections();
		CompiledSectionStatus.Init(MeshPartition::FCompiledSectionStatus(), CompiledSections.Num());
		for (int32 CompiledSectionIndex = 0; CompiledSectionIndex < CompiledSections.Num(); CompiledSectionIndex++)
		{
			const FCompiledSectionDescriptor& CompiledSectionDesc = CompiledSections[CompiledSectionIndex];
			ActorGuidToCompiledSectionIndex.Add(CompiledSectionDesc.ActorDescGuid, CompiledSectionIndex);
		}

		// let's build the correspondence between the target groups (what we want)
		// and the compiled sections (what we have), and check the up to date status.
		// 
		// For a compiled section to be an up-to-date match for a target group, we require:
		// 1) same mesh partition, (GUID, path, definition, build variant)
		// 2) same set of modifiers (modifier set hash, which is the hash of the modifier paths)
		// 3) same BuildVariantHash - settings on the definition and build variant have not changed
		// 4) package hash and class hashes are up to date - the content and code used to build the compiled section have not changed
		// 
		// we must also check for duplicates, and ensure we have at most one up to date compiled section per group
		// 

		// iterate the compiled sections, looking for any that match a target group on our build variants (and then update the compiled section status appropriately)
		for (int32 CompiledSectionIndex = 0; CompiledSectionIndex < CompiledSections.Num(); CompiledSectionIndex++)
		{
			const FCompiledSectionDescriptor& CompiledSection = CompiledSections[CompiledSectionIndex];
			MeshPartition::FCompiledSectionStatus& Status = CompiledSectionStatus[CompiledSectionIndex];

			check(!Status.bChecked);
			Status.bChecked = true;

			// ignore placeholders -- they shouldn't be considered a properly built compiled section.
			// we don't serialize placeholder status into the build info, but we can deduce it from an unset modifier set hash
			const MeshPartition::FCompiledSectionBuildInfo& BuildInfo = CompiledSection.Info;
			if (!BuildInfo.ModifierSetHash.IsValid())
			{
				Status.bIsPlaceholder = true;
				continue;
			}

			// find the corresponding build variant updater
			FBuildVariantUpdater* BuildVariantUpdater = BuildVariantUpdaters.Find(BuildInfo.BuildVariantName);
			if (BuildVariantUpdater == nullptr)
			{
				Status.bNonTargetVariant = true;
				continue;
			}

			// figure out which group this compiled section maps to, by looking up its modifier set hash
			int32 GroupIndex = BuildVariantUpdater->GetTargetGroupIndexFromModifierSetHash(BuildInfo.ModifierSetHash);
			if (GroupIndex < 0)
			{
				Status.bMismatchedModifiers = true;
				continue;
			}

			Status.bBuildVariantHashMatches = (BuildInfo.BuildVariantHash == BuildVariantUpdater->GetBuildInfo().BuildVariantHash);
			Status.bPackageHashMatches = UE::MeshPartition::EditorUtils::PackageHashIsUpToDate(BuildInfo, InAssetRegistry);
			Status.bClassHashMatches = UE::MeshPartition::EditorUtils::ClassHashIsUpToDate(BuildInfo);

			const MeshPartition::FModifierGroup& ModifierGroup = BuildVariantUpdater->GetTargetModifierGroups()[GroupIndex];

			// double check that the modifier paths match exactly, because even 128 bit hashes might collide
			check(ModifierGroup.BaseDescs().Num() == BuildInfo.BaseModifierPaths.Num());
			for (int32 BaseModifierIndex = 0; BaseModifierIndex < ModifierGroup.BaseDescs().Num(); BaseModifierIndex++)
			{
				check(ModifierGroup.BaseDescs()[BaseModifierIndex].ModifierPath == BuildInfo.BaseModifierPaths[BaseModifierIndex]);
			}

			// TODO: this shouldn't be necessary -- ideally we should ensure ActorDesc is up to date before caching the descriptors...
			// (currently after deleting a compiled section, the ActorDescs stick around for a while...)
			// for now we double check if compiled sections were deleted
			bool bStillExists = InAssetRegistry.GetAssetByObjectPath(CompiledSection.ActorPath).IsValid();
			if (!bStillExists)
			{
				Status.bCannotFindFile = true;
				continue;
			}

			const bool bUpToDate = bStillExists && Status.bBuildVariantHashMatches && Status.bPackageHashMatches && Status.bClassHashMatches;
			if (bUpToDate)
			{
				// Check for duplicates within the same grid cell (or same non-grid section).
				// Grid-split sections are keyed by GridCellCoord; non-grid sections use InvalidGridCellCoord.
				const FIntVector GridCellCoord = BuildInfo.GridCellCoord;
				TMap<FIntVector, int32>& ReusableIndices = BuildVariantUpdater->ReusableCompiledSectionIndicesForGroup[GroupIndex];

				if (int32* ExistingCompiledSectionIndex = ReusableIndices.Find(GridCellCoord))
				{
					// there is a previous match for this grid cell -- determine which one to keep
					MeshPartition::FCompiledSectionStatus& PrevBestStatus = CompiledSectionStatus[*ExistingCompiledSectionIndex];
					check(PrevBestStatus.bChecked && PrevBestStatus.bReuse);

					// use an arbitrary deterministic method to determine which one to keep
					if (CompiledSections[*ExistingCompiledSectionIndex].ActorDescGuid < CompiledSection.ActorDescGuid)
					{
						// use the previous best -- this one is a duplicate
						Status.bIsDuplicate = true;
					}
					else
					{
						// use this one -- set previous to be a duplicate
						PrevBestStatus.bIsDuplicate = true;
						PrevBestStatus.bReuse = false;
						*ExistingCompiledSectionIndex = CompiledSectionIndex;
					}
				}
				else
				{
					// no previous match for this grid cell, use this one
					ReusableIndices.Add(GridCellCoord, CompiledSectionIndex);
					Status.bIsDuplicate = false;
				}

				Status.bReuse = bUpToDate && !Status.bIsDuplicate;
			}
		}

		// Build the BaseModifiersNeedingCreation array
		for (auto& Pair : BuildVariantUpdaters)
		{
			FBuildVariantUpdater& BuildVariantUpdater = Pair.Value;
			for (int32 GroupIndex = 0; GroupIndex < BuildVariantUpdater.GetTargetModifierGroups().Num(); GroupIndex++)
			{
				const TMap<FIntVector, int32>& ReusableIndices = BuildVariantUpdater.ReusableCompiledSectionIndicesForGroup[GroupIndex];

				if (ReusableIndices.IsEmpty())
				{
					// this group does not have any reusable compiled sections, and needs to be created.
					// add all of the target group's base modifiers to the list of those needing to be created
					const MeshPartition::FModifierGroup& ModifierGroup = BuildVariantUpdater.GetTargetModifierGroups()[GroupIndex];
					for (const MeshPartition::FModifierDesc& ModifierDesc : ModifierGroup.BaseDescs())
					{
						BuildVariantUpdater.BaseModifiersNeedingCreation.Add(ModifierDesc.ModifierPath, ModifierDesc);
					}
				}
				else
				{
					// sanity check all reusable sections
					for (const auto& GridIndexPair : ReusableIndices)
					{
						MeshPartition::FCompiledSectionStatus& Status = CompiledSectionStatus[GridIndexPair.Value];
						check(Status.bChecked && Status.bReuse);
					}
				}
			}
		}
	}

	FBuildVariantUpdater::FBuildVariantUpdater(	const FGuid& InBuildKey, const AMeshPartition* InMeshPartition, const UMeshPartitionDefinition& InMeshPartitionDefinition, const MeshPartition::FModifierGrouping& InTargetGrouping, TSharedPtr<MeshPartition::FCachedDescriptors> InAllMeshPartitionDescriptors, IAssetRegistry& InAssetRegistry, const UWorld* InWorld)
		: AllMeshPartitionDescriptors(InAllMeshPartitionDescriptors)
		, TargetGrouping(InTargetGrouping)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::BuildVariantUpdater::MatchCompiledSections);

		using namespace EditorUtils;

		check(&TargetGrouping != nullptr);
		check(&InMeshPartitionDefinition != nullptr);
		check(AllMeshPartitionDescriptors.IsValid());
		check(InMeshPartition != nullptr);

		// construct the BuildInfo
		{
			BuildInfo.BuildKey = InBuildKey;
			BuildInfo.SetMegaMeshDefinition(&InMeshPartitionDefinition);
			BuildInfo.MegaMeshGUID = InAllMeshPartitionDescriptors->GetMeshPartitionGUID();
			BuildInfo.BuildVariantName = InTargetGrouping.GetBuildVariantName();
			{	// hash the build variant — must match the main builder's hash (WorldPartitionMeshPartitionBuilder.cpp)
				const MeshPartition::FCompiledSectionBuildVariant& BuildVariant = InMeshPartitionDefinition.GetCompiledSectionBuildVariantByName(BuildInfo.BuildVariantName);

				FDependencyHash DependencyHash;
				InMeshPartition->GatherDependencies(DependencyHash);
				InMeshPartitionDefinition.GatherDependencies(DependencyHash, BuildInfo.BuildVariantName);

				// Resolve and hash grid configuration to match the main builder
				if (BuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid)
				{
					GridSettings = WorldPartitionHelpers::ResolveGridFromPipeline(BuildVariant, InWorld);
					GridSettings.GatherDependencies(DependencyHash);
				}

				BuildInfo.BuildVariantHash = DependencyHash.GetDependentDataHash();
			}
			// Note: get the path from the descriptor ref, because it will be the original path (not the PIE temporary path)
			BuildInfo.MegaMeshPath = InAllMeshPartitionDescriptors->GetMeshPartitionPath();
		}

		// Note: this includes compiled sections that are not for our specific build variant
		TConstArrayView<FCompiledSectionDescriptor> AllExistingCompiledSections = InAllMeshPartitionDescriptors->GetCompiledSections();

		if (InAssetRegistry.IsGathering())
		{
			UE_LOGF(LogMegaMeshEditor, Log, "AssetRegistry is not complete, forcing synchronous scan of the MeshPartition related assets");

			// force the AssetRegistry to complete scanning on the packages we care about
			TArray<FName> PackageNamesToQuery;
			for (const FCompiledSectionDescriptor& ExistingCompiledSection : AllExistingCompiledSections)
			{
				PackageNamesToQuery.Add(ExistingCompiledSection.ActorPath.GetLongPackageFName());
				PackageNamesToQuery.Append(ExistingCompiledSection.Info.PackageDependencies);
			}

			// convert package names to file paths
			TArray<FString> PackageFilePathsToQuery;
			for (FName PackageName : PackageNamesToQuery)
			{
				FString PackageFilename;
				if (FPackageName::DoesPackageExist(PackageName.ToString(), &PackageFilename))
				{
					PackageFilePathsToQuery.Add(PackageFilename);
				}
			}

			// ask asset registry to complete scanning the packages
			InAssetRegistry.ScanFilesSynchronous(PackageFilePathsToQuery);
		}

		TConstArrayView<MeshPartition::FModifierGroup> TargetModifierGroups = InTargetGrouping.GetModifierGroups();

		// init all groups to have no corresponding compiled sections (we will determine the mapping later in the MeshPartition updater)
		ReusableCompiledSectionIndicesForGroup.SetNum(TargetModifierGroups.Num());

		// build the ModifierSetHashToTargetGroupIndex
		for (int GroupIndex = 0; GroupIndex < TargetModifierGroups.Num(); GroupIndex++)
		{
			const MeshPartition::FModifierGroup& ModifierGroup = TargetModifierGroups[GroupIndex];
			FGuid ModifierSetHash = ModifierGroup.ComputeModifierSetHash();
			ModifierSetHashToTargetGroupIndex.Add(ModifierSetHash, GroupIndex);
		}
	}


	bool FBuildVariantUpdater::HasReusableCompiledSectionsForGroup(int32 InGroupIndex, const FMeshPartitionUpdater& InMeshPartitionUpdater) const
	{
		const TMap<FIntVector, int32>& ReusableIndices = ReusableCompiledSectionIndicesForGroup[InGroupIndex];

		for (const auto& Pair : ReusableIndices)
		{
			if (const FCompiledSectionStatus* Status = InMeshPartitionUpdater.GetCompiledSectionStatus(Pair.Value))
			{
				ensure(Status->bChecked);
				ensure(Status->bReuse);
			}
		}

		return !ReusableIndices.IsEmpty();
	}

	int32 FMeshPartitionWorldUpdater::CreatePlaceholderActors(UWorld* InTargetWorld)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::WorldUpdater::CreatePlaceholderActors);

		check(UpdateMode == EUpdateMode::ForPIE);

		int32 PlaceholdersCreated = 0;
		for (TPair<FGuid, FMeshPartitionUpdater>& Pair : MeshPartitionUpdaters)
		{
			FMeshPartitionUpdater& MeshPartitionUpdater = Pair.Value;
			MeshPartitionUpdater.ForAllBuildVariants([InTargetWorld, &MeshPartitionUpdater, &PlaceholdersCreated](const FBuildVariantUpdater& BuildVariantUpdater)
			{
				TConstArrayView<MeshPartition::FModifierGroup> ModifierGroups = BuildVariantUpdater.GetTargetGrouping().GetModifierGroups();
				for (int32 GroupIndex = 0; GroupIndex < ModifierGroups.Num(); GroupIndex++)
				{
					if (BuildVariantUpdater.HasReusableCompiledSectionsForGroup(GroupIndex, MeshPartitionUpdater))
					{
						continue; // no need to create placeholders if this group already has reusable sections
					}

					// insert placeholder(s) for this modifier group
					const MeshPartition::FModifierGroup& ModifierGroup = ModifierGroups[GroupIndex];
					const MeshPartition::UMeshPartitionDefinition* Definition = BuildVariantUpdater.GetBuildInfo().GetMegaMeshDefinition();
					const MeshPartition::FCompiledSectionBuildVariant& BuildVariant = Definition->GetCompiledSectionBuildVariantByName(BuildVariantUpdater.GetBuildVariantName());

					// Resolve the MeshPartition actor once for this group. WorldToLocal/LocalToWorld
					// reads its live transform, hoisted out of the per-cell loop below.
					const AMeshPartition* MeshPartitionActor = MeshPartitionUpdater.MeshPartition.Get();
					check(MeshPartitionActor);

					// Build the bases list from the modifier paths
					FCompiledSectionBuildInfo BaseBuildInfo = BuildVariantUpdater.GetBuildInfo();
					BaseBuildInfo.BaseModifierPaths.Empty();
					Algo::Transform(ModifierGroup.BaseDescs(), BaseBuildInfo.BaseModifierPaths, [](const MeshPartition::FModifierDesc& InBaseDescriptor)
						{
							// the path is taken from the editor world, so it should not need any massaging (unlike PIE world paths)
							return InBaseDescriptor.ModifierPath;
						});

					// Compute bounds from base modifiers + growth-flagged non-bases.
					// ComputeBaseBounds unions BaseDescs bounds, then extends along axes
					// where non-base modifiers have BaseGrowth set (actual deformations).
					// This avoids inflating bounds from global modifiers with huge extents.
					// Result is in WORLD space (descriptor bounds are world-space).
					const FBox GroupBoundsWorld = ModifierGroup.ComputeBaseBounds();

					// Determine placeholder grid: either N per-cell placeholders or one per group
					TArray<WorldPartitionHelpers::FGridCellEstimate> PlaceholderCells;
					const FGridSettings& GridSettings = BuildVariantUpdater.GetGridSettings();

					// The build pipeline produces a mesh in MeshPartition-LOCAL space (BuilderSettings.Transform
					// is metadata only -- vertices stay local). BuildGridCellMeshes derives cell keys from that
					// local mesh's bounds, so to make placeholder cell keys match what the build will produce,
					// estimate cells in local space too. Stream/render with world-space bounds for WP visibility.
					const FBox GroupBoundsLocal = MeshPartitionActor->WorldToLocal(GroupBoundsWorld);

					if (BuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid && GridSettings.IsGridSplit())
					{
						// Grid-split: create one placeholder per estimated grid cell
						PlaceholderCells = WorldPartitionHelpers::EstimateGridCells(GroupBoundsLocal, GridSettings, MeshPartitionActor->GetActorTransform());
					}
					else
					{
						// Non-grid: single placeholder with full group bounds
						PlaceholderCells.Add({ FCompiledSectionBuildInfo::InvalidGridCellCoord, GroupBoundsLocal });
					}

					// Note: We don't fill out ModifiersSetHash because we don't have the full set of non-base modifiers
					// (The placeholder builder will find them when it loads everything)
					const FString BaseName = TEXT("CompiledSection_") + BuildVariant.Name.ToString() + TEXT("_PIE");

					for (const WorldPartitionHelpers::FGridCellEstimate& Cell : PlaceholderCells)
					{
						FName UniqueName = MakeUniqueObjectName(InTargetWorld, UStaticMesh::StaticClass(), FName(BaseName), EUniqueObjectNameOptions::UniversallyUnique);

						FActorSpawnParameters SpawnParams;
						SpawnParams.Name = UniqueName;
						SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
						SpawnParams.bNoFail = true;
						SpawnParams.ObjectFlags = RF_NoFlags;

						// Streaming bounds must be world-space for WP to load the placeholder when the player
						// enters the cell. Cell.CellBounds is in local space (we passed local bounds to
						// EstimateGridCells); transform back to world here.
						const FBox CellBoundsWorld = MeshPartitionActor->LocalToWorld(Cell.CellBounds);

						MeshPartition::ACompiledSection* Placeholder = MeshPartitionWorldUpdaterLocals::SpawnPlaceholderSection(InTargetWorld, SpawnParams, CellBoundsWorld);

						FCompiledSectionBuildInfo PlaceholderBuildInfo = BaseBuildInfo;
						PlaceholderBuildInfo.GridCellCoord = Cell.GridCellCoord;
						PlaceholderBuildInfo.GridSettings = GridSettings;

						Placeholder->SetBuildInfo(PlaceholderBuildInfo);
						Placeholder->SetActorLabel(BaseName);

						UE_LOGF(LogMegaMeshEditor, Display, "    ADDED Placeholder %.6ls (%p %ls) GridCell:(%d,%d,%d) CellSize:%u Is2D:%d (for %d bases)",
							   *Placeholder->GetActorGuid().ToString(), Placeholder, *Placeholder->GetActorNameOrLabel(),
							   Cell.GridCellCoord.X, Cell.GridCellCoord.Y, Cell.GridCellCoord.Z, GridSettings.CellSize, GridSettings.bIs2D ? 1 : 0, Placeholder->GetBuildInfo().BaseModifierPaths.Num());

						PlaceholdersCreated++;
						MeshPartitionUpdater.AddedPlaceholderCompiledSections++;
					}
				}
				return true; // continue iterating build variants
			});
		}
		PlaceholdersAddedToWorld = InTargetWorld;
		return PlaceholdersCreated;
	}

	int32 FMeshPartitionWorldUpdater::RemovePlaceholderActors()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::WorldUpdater::RemovePlaceholderActors);

		check(UpdateMode == EUpdateMode::ForPIE);

		int32 PlaceholdersRemoved = 0;
		if (UWorld* World = PlaceholdersAddedToWorld.Get())
		{
			for (TActorIterator<MeshPartition::ACompiledSection> It(World); It; ++It)
			{
				MeshPartition::ACompiledSection* CompiledSection = *It;
				if (CompiledSection)
				{
					if (CompiledSection->IsPlaceholder())
					{
						CompiledSection->Destroy();
						PlaceholdersRemoved++;
					}
				}
			}
		}
		PlaceholdersAddedToWorld = nullptr;
		return PlaceholdersRemoved;
	}

	void FMeshPartitionUpdater::ForAllBuildVariants(TFunctionRef<bool(const MeshPartition::FBuildVariantUpdater&)> InFunc) const
	{
		for (const TPair<FName, FBuildVariantUpdater>& Pair : BuildVariantUpdaters)
		{
			const FBuildVariantUpdater& BuildVariantUpdater = Pair.Value;
			if (!InFunc(BuildVariantUpdater))
			{
				return;
			}
		}
	}

	const FCompiledSectionStatus* FMeshPartitionUpdater::GetCompiledSectionStatus(int32 CompiledSectionIndex) const
	{
		if (CompiledSectionStatus.IsValidIndex(CompiledSectionIndex))
		{
			return &CompiledSectionStatus[CompiledSectionIndex];
		}
		return nullptr;
	}

	const FCompiledSectionStatus* FMeshPartitionUpdater::GetCompiledSectionStatus(const ACompiledSection* CompiledSection) const
	{
		int32 CompiledSectionIndex = ActorGuidToCompiledSectionIndex.FindRef(CompiledSection->GetActorGuid(), -1);
		if (CompiledSectionIndex >= 0)
		{
			return &CompiledSectionStatus[CompiledSectionIndex];
		}
		return nullptr;
	}

	const FMeshPartitionUpdater* FMeshPartitionWorldUpdater::GetMeshPartitionUpdaterFor(const ACompiledSection* CompiledSection) const
	{
		check(CompiledSection);
		return MeshPartitionUpdaters.Find(CompiledSection->GetBuildInfo().MegaMeshGUID);
	}

	const FCompiledSectionStatus* FMeshPartitionWorldUpdater::GetCompiledSectionStatus(const ACompiledSection* CompiledSection) const
	{
		if (const FMeshPartitionUpdater* MeshPartitionUpdater = GetMeshPartitionUpdaterFor(CompiledSection))
		{
			return MeshPartitionUpdater->GetCompiledSectionStatus(CompiledSection);
		}
		return nullptr;
	}

	void FMeshPartitionWorldUpdater::ReportStats() const
	{
		for (const TPair<FGuid, FMeshPartitionUpdater>& Pair : MeshPartitionUpdaters)
		{
			const FMeshPartitionUpdater& MeshPartitionUpdater = Pair.Value;

			UE_LOGF(LogMegaMeshEditor, Verbose, "  -- MeshPartition '%ls' : %d out of date, %d non-target build variant, %d reused, %d placeholders created",
				*MeshPartitionUpdater.AllCachedDescriptors->GetMeshPartitionPath().ToString(),
				MeshPartitionUpdater.RemovedOutOfDate,
				MeshPartitionUpdater.RemovedNonTargetBuildVariant,
				MeshPartitionUpdater.ReusedCompiledSections,
				MeshPartitionUpdater.AddedPlaceholderCompiledSections);
		}
	}

	void FMeshPartitionWorldUpdater::GetStatusSummary(FStatusSummary& InOutSummary) const
	{
		for (const TPair<FGuid, FMeshPartitionUpdater>& Pair : MeshPartitionUpdaters)
		{
			const FMeshPartitionUpdater& MeshPartitionUpdater = Pair.Value;
			MeshPartitionUpdater.GetStatusSummary(InOutSummary);
		}
	}

	void FMeshPartitionUpdater::GetStatusSummary(FStatusSummary& InOutSummary) const
	{
		InOutSummary.MeshPartitionCount++;

		// iterate compiled sections
		for (const MeshPartition::FCompiledSectionStatus& Status : CompiledSectionStatus)
		{
			check(Status.bChecked);
			InOutSummary.CompiledSectionCount++;

			// done this way so the counts are exclusive (even though a section could fail for multiple reasons)
			// the order here determines the priority for what is counted as the failure reason
			if (Status.bIsPlaceholder)
			{
				InOutSummary.CompiledSections.Placeholders++;
			}
			else if (Status.bNonTargetVariant)
			{
				InOutSummary.CompiledSections.NonTargetVariant++;
			}
			else if (Status.bMismatchedModifiers)
			{
				InOutSummary.CompiledSections.MismatchedModifiers++;
			}
			else if (!Status.bBuildVariantHashMatches)
			{
				InOutSummary.CompiledSections.BuildVariantHashFails++;
			}
			else if (!Status.bPackageHashMatches)
			{
				InOutSummary.CompiledSections.PackageHashFails++;
			}
			else if (!Status.bClassHashMatches)
			{
				InOutSummary.CompiledSections.ClassHashFails++;
			}
			else if (Status.bCannotFindFile)
			{
				InOutSummary.CompiledSections.CannotFindFile++;
			}
			else if (Status.bIsDuplicate)
			{
				InOutSummary.CompiledSections.Duplicates++;
			}
			else if (Status.bReuse)
			{
				InOutSummary.CompiledSections.Reuse++;
			}
			else
			{
				InOutSummary.CompiledSections.Unknown++;
			}
		}

		// also check build variant groups for any groups that do not have a compiled section
		for (const TPair<FName, FBuildVariantUpdater>& Pair : BuildVariantUpdaters)
		{
			const FBuildVariantUpdater& VariantUpdater = Pair.Value;
			InOutSummary.TargetBuildVariantCount++;

			// look for any target groups with no corresponding compiled section -- these are missing
			for (const TMap<FIntVector, int32>& GridCellMap : VariantUpdater.ReusableCompiledSectionIndicesForGroup)
			{
				InOutSummary.GroupCount++;
				if (GridCellMap.IsEmpty())
				{
					InOutSummary.CompiledSections.Missing++;
				}
			}
		}
	}

	void FMeshPartitionWorldUpdater::ReportSectionStatus(UWorld* InWorld) const
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
		IAssetRegistry* AssetRegistry = &AssetRegistryModule.Get();
		check(AssetRegistry);

		TMap<FGuid, AMeshPartition*> LoadedMeshPartitions;
		TMap<FGuid, ACompiledSection*> LoadedCompiledSections;
		if (InWorld)
		{
			for (AMeshPartition* MeshPartition : TActorRange<AMeshPartition>(InWorld))
			{
				if (MeshPartition)
				{
					const FGuid& MeshPartitionGUID = MeshPartition->GetActorGuid();
					LoadedMeshPartitions.Add(MeshPartitionGUID, MeshPartition);
				}
			}
			for (ACompiledSection* CompiledSection : TActorRange<ACompiledSection>(InWorld))
			{
				if (CompiledSection)
				{
					const FGuid& CompiledSectionGUID = CompiledSection->GetActorGuid();
					LoadedCompiledSections.Add(CompiledSectionGUID, CompiledSection);
				}
			}
		}

		for (const TPair<FGuid, FMeshPartitionUpdater>& Pair : MeshPartitionUpdaters)
		{
			const FMeshPartitionUpdater& MeshPartitionUpdater = Pair.Value;
			TSet<int32> CompiledSectionIndicesReported;

			FGuid MeshPartitionGuid = MeshPartitionUpdater.AllCachedDescriptors->GetMeshPartitionGUID();
			AMeshPartition** LoadedActor = LoadedMeshPartitions.Find(MeshPartitionGuid);

			FString LoadedActorPath;
			FString LoadedActorLabel;
			if (LoadedActor)
			{
				LoadedActorPath = (*LoadedActor)->GetPathName();
				LoadedActorLabel = (*LoadedActor)->GetActorNameOrLabel();
			}

			FStatusSummary Summary;
			MeshPartitionUpdater.GetStatusSummary(Summary);

			auto ReportCompiledSection = [&MeshPartitionUpdater, AssetRegistry, &LoadedCompiledSections](int32 CompiledSectionIndex, int32 SectionBaseCount, int32 TotalBaseCount, FName BuildVariantName)
				{
					FString StatusString;
					FString LoadedString;
					FString DetailsString;
					bool bReuse = false;

					if (CompiledSectionIndex >= 0)
					{
						const FCompiledSectionStatus* Status = MeshPartitionUpdater.GetCompiledSectionStatus(CompiledSectionIndex);
						check(Status);

						StatusString = Status->ToString();

						const FCompiledSectionDescriptor& CompiledSectionDesc = MeshPartitionUpdater.CompiledSections[CompiledSectionIndex];
						DetailsString = CompiledSectionDesc.ActorPath.ToString();

						LoadedString = "      ";
						if (const FCompiledSectionDescriptor* Descriptor = MeshPartitionUpdater.GetCompiledSectionDescriptor(CompiledSectionIndex))
						{
							if (ACompiledSection** CompiledSectionPtr = LoadedCompiledSections.Find(Descriptor->ActorDescGuid))
							{
								// add loaded details
								if (ACompiledSection* CompiledSection = *CompiledSectionPtr)
								{
									LoadedString = "Loaded";

									if (UTexture2DArray* ChannelTexture = Cast<UTexture2DArray>(CompiledSection->GetChannelTexture()))
									{
										DetailsString.Appendf(TEXT(" tex(%dx%dx%d %d/%d mips)"),
											ChannelTexture->GetSizeX(),
											ChannelTexture->GetSizeY(),
											ChannelTexture->GetArraySize(),
											ChannelTexture->GetNumResidentMips(),
											//ChannelTexture->GetNumResidentMips(),
											ChannelTexture->GetNumMips()
										);
									}

									TArray<TObjectPtr<UStaticMesh>> StaticMeshes = CompiledSection->GetStaticMeshes();
									for (TObjectPtr<UStaticMesh> StaticMesh : StaticMeshes)
									{
										if (StaticMesh)
										{
											int32 LODCount = StaticMesh->GetNumLODs();
											DetailsString.Appendf(TEXT(" mesh(%d tris %d verts %d lods)"),
												StaticMesh->GetNumTriangles(0),
												StaticMesh->GetNumVertices(0),
												LODCount);
										}
									}
								}
							}
						}

						if (!Status->bReuse)
						{
							DetailsString = Status->GetDetailsString(CompiledSectionIndex, MeshPartitionUpdater, *AssetRegistry);
						}
					}
					else
					{
						StatusString = TEXT("Missing");
					}

					UE_LOGF(LogMegaMeshEditor, Warning, "  [%ls] [%d/%d bases] %ls (%ls) %ls",
						*BuildVariantName.ToString(), SectionBaseCount, TotalBaseCount, *LoadedString, *StatusString, *DetailsString);
				};

			UE_LOGF(LogMegaMeshEditor, Warning, "%ls %ls (GUID: %ls)", *LoadedActorLabel, *LoadedActorPath, *MeshPartitionGuid.ToString());

			// iterate variants
			for (const TPair<FName, FBuildVariantUpdater>& Pair2 : MeshPartitionUpdater.BuildVariantUpdaters)
			{
				const FBuildVariantUpdater& VariantUpdater = Pair2.Value;
				check(Pair2.Key == VariantUpdater.GetBuildVariantName());
				FName BuildVariantName = VariantUpdater.GetBuildVariantName();

				UE_LOGF(LogMegaMeshEditor, Warning, " --- Variant: %ls", *BuildVariantName.ToString());

				int32 TotalBaseCount = MeshPartitionUpdater.AllCachedDescriptors->GetAllBaseModifiers().Num();

				// first report on the groups that reuse a compiled section (one line per grid cell)
				TConstArrayView<MeshPartition::FModifierGroup> ModifierGroups = VariantUpdater.GetTargetModifierGroups();
				for (int32 GroupIndex = 0; GroupIndex < ModifierGroups.Num(); GroupIndex++)
				{
					const MeshPartition::FModifierGroup& Group = ModifierGroups[GroupIndex];
					const TMap<FIntVector, int32>& GridCellMap = VariantUpdater.GetReusableCompiledSectionIndicesForGroup(GroupIndex);
					if (GridCellMap.IsEmpty())
					{
						// no compiled section exists for this group at all
						ReportCompiledSection(-1, Group.BaseDescs().Num(), TotalBaseCount, BuildVariantName);
					}
					else
					{
						for (const TPair<FIntVector, int32>& CellPair : GridCellMap)
						{
							ReportCompiledSection(CellPair.Value, Group.BaseDescs().Num(), TotalBaseCount, BuildVariantName);
							if (CellPair.Value >= 0)
							{
								CompiledSectionIndicesReported.Add(CellPair.Value);
							}
						}
					}
				}
			}

			UE_LOGF(LogMegaMeshEditor, Warning, " === REMOVE:");

			// Now report on all existing compiled sections that were not yet reported - these are not mapped to any target build variant and need to be removed
			int32 TotalBaseCount = 0; // we can't be sure how many modifiers were originally in this compiled section, without a corresponding grouping

			for (int32 CompiledSectionIndex = 0; CompiledSectionIndex < MeshPartitionUpdater.CompiledSections.Num(); CompiledSectionIndex++)
			{
				if (!CompiledSectionIndicesReported.Contains(CompiledSectionIndex))
				{
					const FCompiledSectionDescriptor& ExistingDesc = MeshPartitionUpdater.CompiledSections[CompiledSectionIndex];
					FName BuildVariantName = ExistingDesc.Info.BuildVariantName;
					ReportCompiledSection(CompiledSectionIndex, ExistingDesc.Info.BaseModifierPaths.Num(), TotalBaseCount, BuildVariantName);
				}
			}

			UE_LOGF(LogMegaMeshEditor, Warning, " ===");

			UE_LOGF(LogMegaMeshEditor, Warning, " - Sections: %d UpToDate, %d OutOfDate, %d GroupsMissing, %d GroupsTotal",
				Summary.SectionsUpToDate(), Summary.SectionsOutOfDate(), Summary.GroupsMissing(), Summary.GroupCount);

			UE_LOGF(LogMegaMeshEditor, Warning, " - CompiledSections: %d ToRemove, %d Total",
				Summary.CompiledSectionsToRemove(), Summary.CompiledSectionCount);

			UE_LOGF(LogMegaMeshEditor, Warning, " ===");
		}
	}

	FString FCompiledSectionStatus::ToString() const
	{
		FString StatusString;
		check(bChecked);

		if (bReuse)
		{
			StatusString = TEXT("UpToDate");
		}
		if (bIsDuplicate)
		{
			StatusString += TEXT("Duplicate");
		}
		if (bCannotFindFile)
		{
			StatusString += TEXT("CannotFindFile");
		}
		if (bIsPlaceholder)
		{
			StatusString += TEXT("Placeholder");
		}
		if (bNonTargetVariant)
		{
			StatusString += TEXT("NonTargetVariant");
		}
		if (bMismatchedModifiers)
		{
			StatusString += TEXT("MismatchedModifiers");
		}

		// Hash fields are only meaningful for sections that reached the hash-check stage
		// (i.e., not early-categorized as placeholder, non-target variant, or mismatched modifiers)
		const bool bReachedHashCheck = !bIsPlaceholder && !bNonTargetVariant && !bMismatchedModifiers;

		if (bReachedHashCheck && !bBuildVariantHashMatches)
		{
			StatusString += TEXT("BuildVariantHashFail");
		}
		if (bReachedHashCheck && !bPackageHashMatches)
		{
			StatusString += TEXT("PackageHashFail");
		}
		if (bReachedHashCheck && !bClassHashMatches)
		{
			StatusString += TEXT("ClassHashFail");
		}
		if (StatusString.IsEmpty())
		{
			StatusString = TEXT("<UNKNOWN>");
		}
		return StatusString;
	}

	FString FCompiledSectionStatus::GetDetailsString(int32 CompiledSectionIndex, const FMeshPartitionUpdater& MeshPartitionUpdater, IAssetRegistry& AssetRegistry) const
	{
		FString Details;
		TConstArrayView<FCompiledSectionDescriptor> CompiledSections = MeshPartitionUpdater.AllCachedDescriptors->GetCompiledSections();
		if (CompiledSections.IsValidIndex(CompiledSectionIndex))
		{
			const FCompiledSectionDescriptor& Section = CompiledSections[CompiledSectionIndex];

			if (!bPackageHashMatches)
			{
				Details += TEXT(" PackageHashChanged(");

				int32 PackageCount = Section.Info.PackageDependencies.Num();
				if (Section.Info.PackageChecksums.Num() == PackageCount)
				{
					TArray<uint32> CurrentChecksums;
					FGuid PackageHash = UE::MeshPartition::EditorUtils::ComputePackageHash(AssetRegistry, Section.Info.PackageDependencies, &CurrentChecksums);
					check(CurrentChecksums.Num() == PackageCount);

					bool First = true;
					for (int32 PackageIndex = 0; PackageIndex < PackageCount; PackageIndex++)
					{
						if (CurrentChecksums[PackageIndex] != Section.Info.PackageChecksums[PackageIndex])
						{
							if (!First)
							{
								Details += TEXT(", ");
							}
							First = false;
							Details += Section.Info.PackageDependencies[PackageIndex].ToString();
						}
					}
				}
				else
				{
					Details += TEXT("No Checksums Recorded");
				}
				Details += TEXT(")");
			}

			if (!bClassHashMatches)
			{
				Details += TEXT(" ClassHashChanged(");
				int32 ClassCount = Section.Info.ClassDependencies.Num();
				if (Section.Info.ClassChecksums.Num() == ClassCount)
				{
					TArray<const UClass*> Classes;
					Algo::Transform(Section.Info.ClassDependencies, Classes, [](FName ClassPathName)
						{
							FSoftClassPath ClassPath(ClassPathName.ToString());
							return ClassPath.ResolveClass();
						});

					TArray<uint32> CurrentChecksums;
					FGuid ClassHash = UE::MeshPartition::EditorUtils::ComputeClassHash(Classes, &CurrentChecksums);
					check(CurrentChecksums.Num() == ClassCount);

					bool First = true;
					for (int32 ClassIndex = 0; ClassIndex < ClassCount; ClassIndex++)
					{
						if (CurrentChecksums[ClassIndex] != Section.Info.ClassChecksums[ClassIndex])
						{
							if (!First)
							{
								Details += TEXT(", ");
							}
							First = false;
							Details += Section.Info.ClassDependencies[ClassIndex].ToString();
						}
					}
				}
				else
				{
					Details += TEXT("No Checksums Recorded");
				}
				Details += TEXT(")");
			}
		}
		else
		{
			Details += TEXT("<NONE>");
		}

		return Details;
	}
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE