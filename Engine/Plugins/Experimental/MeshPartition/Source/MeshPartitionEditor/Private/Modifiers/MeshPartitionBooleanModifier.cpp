// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionBooleanModifier.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "Operations/MeshBoolean.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/MeshNormals.h"
#include "Operations/MeshClusterSimplifier.h"
#include "PrimitiveDrawingUtils.h"

#include "ToolSetupUtil.h" // For standard ModelingComponents materials

namespace UE::MeshPartition
{
namespace MegaMeshBooleanLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName, TSharedRef<const FAsyncMeshInstanceData> InMeshInstance)
			: MeshPartition::IModifierBackgroundOp(InOperationName)
			, MeshInstance(MoveTemp(InMeshInstance))
		{}

		FBox OperatorBounds;
		TSharedRef<const FAsyncMeshInstanceData> MeshInstance;
		FTransform ComponentTransform; // component to world

		MeshPartition::EBooleanOperation BooleanOp;
		MeshPartition::EBooleanInsideTestMethod InsideTargetMeshTestMethod;
		MeshPartition::EBooleanToolMeshEmbedding ToolMeshEmbedding;
		double ToolWindingThreshold, TargetWindingThreshold;
		bool bSimplifyAlongNewEdges;
		bool bWeldSharedEdges;
		TArray<MeshPartition::FBooleanModifierWeightEntry> WeightChannels;
	
		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;

		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("db5b82df-ad41-4c8b-a379-ece5e5648197"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }

	private:
		// Helper to rename/add weight channels as needed to the tool mesh, based on the WeightChannels array
		void AddRequestedWeightChannels(Geometry::FDynamicMesh3& ToolMesh) const;
	};

	void FBackgroundOp::AddRequestedWeightChannels(Geometry::FDynamicMesh3& ToolMesh) const
	{
		// Track known destinations; if multiple channels are requested to map to the same destination, we only use the first
		TSet<FName> FoundDests;

		// Mapping from source to destination channels; the first such mapping we can simply rename, but if there are multiple we must duplicate
		TMap<FName, TArray<FName, TInlineAllocator<2>>> SourceToDests;
		TMap<FName, Geometry::FDynamicMeshWeightAttribute*> ExistingSources;
		// Mapping from destination to source color channel; we always need to add these channels
		TMap<FName, int32> ColorDests;
		Geometry::FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		// Mapping from destination to const values; we always need to add these channels
		TMap<FName, float> ConstDests;
	
		// Find which weight and color attributes exist on the tool mesh
		if (Geometry::FDynamicMeshAttributeSet* Attribs = ToolMesh.Attributes())
		{
			for (int32 Idx = 0; Idx < Attribs->NumWeightLayers(); ++Idx)
			{
				FName WeightName = Attribs->GetWeightLayer(Idx)->GetName();
				if (!WeightName.IsNone())
				{
					ExistingSources.Add(WeightName, Attribs->GetWeightLayer(Idx));
				}
			}

			ColorOverlay = Attribs->PrimaryColors();
		}

		// Validate requested weight channels / decide which we will transfer to the tool mesh
		for (const MeshPartition::FBooleanModifierWeightEntry& Entry : WeightChannels)
		{
			if (FoundDests.Contains(Entry.ChannelName))
			{
				UE_LOGF(LogMegaMeshEditor, Warning, "Boolean modifier ignoring duplicate Weight Channel Destination: \"%ls\"", *Entry.ChannelName.GetName().ToString());
				continue;
			}
			if (Entry.SourceMode == MeshPartition::EBooleanModifierChannelSourceMode::VertexColor)
			{
				if (!ColorOverlay)
				{
					UE_LOGF(LogMegaMeshEditor, Warning, "Boolean modifier ignoring vertex color weight channels: source mesh does not have vertex colors");
					continue;
				}
				if (Entry.VertexColorIndex < 0 || Entry.VertexColorIndex > 3)
				{
					UE_LOGF(LogMegaMeshEditor, Warning, "Boolean modifier ignoring vertex color weight channels with invalid color channel index: %d", Entry.VertexColorIndex);
					continue;
				}
				FoundDests.Add(Entry.ChannelName);
				ColorDests.Add(Entry.ChannelName, Entry.VertexColorIndex);
				continue;
			}
			else if (Entry.SourceMode == MeshPartition::EBooleanModifierChannelSourceMode::Constant)
			{
				FoundDests.Add(Entry.ChannelName);
				ConstDests.Add(Entry.ChannelName, Entry.WriteValue);
				continue;
			}
			if (!ExistingSources.Contains(Entry.SourceWeightChannelName))
			{
				UE_LOGF(LogMegaMeshEditor, Warning, "Boolean modifier could not find Weight Channel Source: \"%ls\"", *Entry.SourceWeightChannelName.ToString());
				continue;
			}
			FoundDests.Add(Entry.ChannelName);
			SourceToDests.FindOrAdd(Entry.SourceWeightChannelName).Add(Entry.ChannelName);
		}

		ToolMesh.EnableAttributes();
		Geometry::FDynamicMeshAttributeSet* Attribs = ToolMesh.Attributes();

		// Clear weight layers that we don't need
		for (int32 WeightLayerIdx = Attribs->NumWeightLayers() - 1; WeightLayerIdx >= 0; --WeightLayerIdx)
		{
			Geometry::FDynamicMeshWeightAttribute* WeightLayer = Attribs->GetWeightLayer(WeightLayerIdx);
			FName LayerName = WeightLayer->GetName();
			if (LayerName.IsNone() || !SourceToDests.Contains(LayerName))
			{
				Attribs->RemoveWeightLayer(WeightLayerIdx);
			}
		}
		// Rename sources -> dests, and duplicate as needed
		for (const TPair<FName, TArray<FName, TInlineAllocator<2>>>&SD : SourceToDests)
		{
			if (!ensure(!SD.Value.IsEmpty()))
			{
				continue;
			}
			ExistingSources[SD.Key]->SetName(SD.Value[0]);
			for (int32 DupIdx = 1; DupIdx < SD.Value.Num(); ++DupIdx)
			{
				int32 NewLayerIdx = Attribs->NumWeightLayers();
				Attribs->SetNumWeightLayers(NewLayerIdx + 1);

				Attribs->GetWeightLayer(NewLayerIdx)->Copy(*ExistingSources[SD.Key]);
				Attribs->GetWeightLayer(NewLayerIdx)->SetName(SD.Value[DupIdx]);
			}
		}
		// Convert requested color channels to weights channels
		if (ColorOverlay)
		{
			for (const TPair<FName, int32>& ColorChannel : ColorDests)
			{
				int32 NewLayerIdx = Attribs->NumWeightLayers();
				Attribs->SetNumWeightLayers(NewLayerIdx + 1);
				Geometry::FDynamicMeshWeightAttribute* WeightLayer = Attribs->GetWeightLayer(NewLayerIdx);
				WeightLayer->SetName(ColorChannel.Key);
				for (int32 ElID : ColorOverlay->ElementIndicesItr())
				{
					int32 VID = ColorOverlay->GetParentVertex(ElID);
					if (VID != INDEX_NONE)
					{
						float ColorChannelValue = ColorOverlay->GetElement(ElID)[ColorChannel.Value];
						WeightLayer->SetValue(VID, &ColorChannelValue);
					}
				}
			}
		}
		for (const TPair<FName, float>& ConstChannel : ConstDests)
		{
			int32 NewLayerIdx = Attribs->NumWeightLayers();
			Attribs->SetNumWeightLayers(NewLayerIdx + 1);
			Geometry::FDynamicMeshWeightAttribute* WeightLayer = Attribs->GetWeightLayer(NewLayerIdx);
			WeightLayer->SetName(ConstChannel.Key);
			float WriteValue = ConstChannel.Value;
			for (int32 VID : ToolMesh.VertexIndicesItr())
			{
				WeightLayer->SetValue(VID, &WriteValue);
			}
		}
	}

	void FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
	{
		FInstanceInfo Desc;
		Desc.InstanceID = 0;
		Desc.Bounds = OperatorBounds;

		// currently querying submeshes, which may result in multiple operators in parallel applying the boolean
		// this requires more work to handle boundaries, such as contiguous modifiers 
		// see UE-255687 

		// #todo: we could expose the ability to write-through the uvs as an option. It seems like a sensible default.
		Desc.ReadViewComponents = EMeshViewComponents::DynamicSubmesh | EMeshViewComponents::VertexUVs;
		Desc.WriteViewComponents = EMeshViewComponents::DynamicSubmesh | EMeshViewComponents::VertexUVs;

		if (Desc.Bounds.Intersect(InBounds))
		{
			for (const MeshPartition::FBooleanModifierWeightEntry& WeightChannel : WeightChannels)
			{
				if (!WeightChannel.ChannelName.IsNone())
				{
					Desc.UsedChannels.Emplace(WeightChannel.ChannelName);

					Desc.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
					Desc.WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				}
			}

			OutInstanceInfos.Add(Desc);
		}
	}

	void FBackgroundOp::ApplyModifications(
		MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UBooleanModifier::ApplyModification);

		// InTransform transforms the megamesh into world space
		using namespace Geometry;

		FDynamicMesh3& EditableMesh = InMeshView.GetSubmeshMutable();

		FDynamicMesh3 CombinedMesh;

		// recompute normals
		FMeshNormals MeshNormals(&EditableMesh);
		MeshNormals.ComputeVertexNormals(false, true); // angle weighted for more robust inside/outside 
		MeshNormals.CopyToVertexNormals(&EditableMesh);

		FMeshBoolean::EBooleanOp BooleanOpMode;
		bool bNeedsEmbeddingLogic = true;
		switch (BooleanOp)
		{
		default:
			ensureMsgf(false, TEXT("Unsupported BooleanOp in Mesh Partition Boolean Modifier"));

		case MeshPartition::EBooleanOperation::Union:
			BooleanOpMode = FMeshBoolean::EBooleanOp::Union;
			break;
		case MeshPartition::EBooleanOperation::Subtract:
			BooleanOpMode = FMeshBoolean::EBooleanOp::Difference;
			break;
		case MeshPartition::EBooleanOperation::Trim:
			BooleanOpMode = FMeshBoolean::EBooleanOp::TrimInside;
			bNeedsEmbeddingLogic = false;
			break;
		}

		// passing in the submesh first seems to preserve boundary vertices, but it needs to be verified if this is guaranteed or
		// whether extra work to remap them is needed
		const FDynamicMesh3* UseToolMesh = &MeshInstance->GetMesh();
		FDynamicMesh3 LocalToolMeshCopy; // only used if we need to e.g. adjust weight channels before running the boolean
		if (UseToolMesh->HasAttributes() && !WeightChannels.IsEmpty())
		{
			LocalToolMeshCopy = *UseToolMesh;
			AddRequestedWeightChannels(LocalToolMeshCopy);
			// Make sure the tool and target mesh weight channel arrays are 1:1
			LocalToolMeshCopy.Attributes()->EnableMatchingWeightLayersByNames(EditableMesh.Attributes(), true);
			UseToolMesh = &LocalToolMeshCopy;
		}
		FMeshBoolean MeshBooleanOp(&EditableMesh, InTransform, UseToolMesh, ComponentTransform, &CombinedMesh, BooleanOpMode);

		MeshBooleanOp.WindingThreshold = ToolWindingThreshold;
		MeshBooleanOp.bWeldSharedEdges = bWeldSharedEdges;
		MeshBooleanOp.bSimplifyAlongNewEdges = bSimplifyAlongNewEdges;
		if (!FMeshBoolean::OperatesOnSingleMesh(BooleanOpMode))
		{
			MeshBooleanOp.TrackPerTriangleSourceMesh.Emplace();
		}
		

		FMeshBoolean::FCustomInsideMeshTest ToolWindingTest(
			[this](const FVector3d& Pos, const FMeshBoolean::FCustomInsideTestContext& Context) -> bool
			{
				return Context.OptionalWindingTree->IsInside(Pos, ToolWindingThreshold);
			}, true /*bRequiresWinding*/);
		double UseTargetWindingThreshold = TargetWindingThreshold;
		if (bNeedsEmbeddingLogic)
		{
			if (ToolMeshEmbedding == MeshPartition::EBooleanToolMeshEmbedding::Intersecting)
			{
				// The missing half-enclosure would have added ~.5 to the winding number.
				// In practice, subtracting slightly less than .5 helps give expected results even when the tool mesh is outside of the terrain.
				UseTargetWindingThreshold -= .49;
			}
			else if (ToolMeshEmbedding == MeshPartition::EBooleanToolMeshEmbedding::Inside)
			{
				// The missing full-enclosure would have added 1 to the winding number.
				UseTargetWindingThreshold -= 1.;
			}
		}
		FMeshBoolean::FCustomInsideMeshTest TargetWindingTest(
			[UseTargetWindingThreshold](const FVector3d& Pos, const FMeshBoolean::FCustomInsideTestContext& Context) -> bool
			{
				return Context.OptionalWindingTree->IsInside(Pos, UseTargetWindingThreshold);
			}, true /*bRequiresWinding*/);
		const bool BooleanSuccess = MeshBooleanOp.ComputeWithCustomInside(
			TargetWindingTest,
			ToolWindingTest);

		// If the tool mesh could have added its own geometry, assign 'no base' to triangles that came from the tool mesh
		if (!FMeshBoolean::OperatesOnSingleMesh(BooleanOpMode) && 
			// We should have a triangle mapping from the boolean operation
			ensure(MeshBooleanOp.TrackPerTriangleSourceMesh->Num() == CombinedMesh.MaxTriangleID()) &&
			// We should have a single MegaMeshBaseID polygroup layer on the target
			ensure(
				CombinedMesh.Attributes()->NumPolygroupLayers() == 1 &&
				CombinedMesh.Attributes()->GetPolygroupLayer(0) != nullptr &&
				CombinedMesh.Attributes()->GetPolygroupLayer(0)->GetName() == FName(TEXT("MegaMeshBaseID"))
			)
		)
		{
			FDynamicMeshPolygroupAttribute* BaseIDLayer = CombinedMesh.Attributes()->GetPolygroupLayer(0);
			for (int32 TID : CombinedMesh.TriangleIndicesItr())
			{
				if ((*MeshBooleanOp.TrackPerTriangleSourceMesh)[TID] == 1)
				{
					BaseIDLayer->SetScalarValue(TID, INDEX_NONE);
				}
			}
		}
	
		MeshTransforms::ApplyTransformInverse(CombinedMesh, InTransform, true);
		EditableMesh = CombinedMesh;

		if (!BooleanSuccess)
		{
			UE_LOGF(LogMegaMeshEditor, Warning, "Boolean Modifier reported errors, and may have introduced mesh artifacts (such as cracks).");
		}
	}
}


UBooleanModifier::UBooleanModifier()
{
	bKeepInternalMeshAttributes = true; // perserve attributes (e.g., normal topology, UVs) of tool mesh
}

void UBooleanModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const FName ParentPreviousDefaultType = TEXT("Mesh");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("Boolean");
		SetType(PreviousModifierDefaultType);
	}
}

void UBooleanModifier::UpdateFromMesh()
{
	UpdateMeshInstance();

	// Trigger megamesh update
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

TArray<FBox> UBooleanModifier::ComputeBounds() const
{
	FBox WorldMeshBounds = ComputeWorldExpandedBounds(FVector::Max3(FVector::ZeroVector, ExpandSectionInclusionBounds, ExpandOperatorBounds));
	if (!WorldMeshBounds.IsValid)
	{
		return {};
	}

	return { WorldMeshBounds };
}

FBox UBooleanModifier::ComputeWorldExpandedBounds(FVector ExpandAmount) const
{
	if (!MeshInstance)
	{
		return FBox();
	}

	const FTransform& MeshToWorld = GetComponentToWorld();
	Geometry::FAxisAlignedBox3d WorldMeshBounds(FBox(MeshInstance->GetBounds()).TransformBy(MeshToWorld));
	WorldMeshBounds.Expand(ExpandAmount);
	return FBox(WorldMeshBounds);
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UBooleanModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	if (!MeshInstance)
	{
		return {};
	}

	TSharedPtr<MegaMeshBooleanLocals::FBackgroundOp> BackgroundOp = AllocateBackgroundOp<MegaMeshBooleanLocals::FBackgroundOp>(GetFName(), MeshInstance.ToSharedRef());
	BackgroundOp->OperatorBounds = ComputeWorldExpandedBounds(FVector::Max(FVector::ZeroVector, ExpandOperatorBounds));
	BackgroundOp->ComponentTransform = GetComponentTransform();
	BackgroundOp->BooleanOp = BooleanOp;
	BackgroundOp->ToolMeshEmbedding = ToolMeshEmbedding;
	BackgroundOp->TargetWindingThreshold = TargetWindingThreshold;
	BackgroundOp->ToolWindingThreshold = ToolWindingThreshold;
	BackgroundOp->InsideTargetMeshTestMethod = InsideTargetMeshTestMethod;
	BackgroundOp->bSimplifyAlongNewEdges = bSimplifyAlongNewEdges;
	BackgroundOp->bWeldSharedEdges = bWeldSharedEdges;
	BackgroundOp->WeightChannels = WeightChannels;
	
	return BackgroundOp;
}

void UBooleanModifier::ProcessMeshInstance(FDynamicMesh3& OutMesh)
{
	if (PreOpSimplifierStrength > 0)
	{
		Geometry::FDynamicMesh3 SimplifiedMesh;

		using EConstraintLevel = Geometry::MeshClusterSimplify::FSimplifyOptions::EConstraintLevel;

		Geometry::MeshClusterSimplify::FSimplifyOptions SimplifyOptions;
		SimplifyOptions.TargetEdgeLength = PreOpSimplifierStrength;
		SimplifyOptions.PreserveEdges.Boundary = bSimplifierConstrainBoundary ? EConstraintLevel::Constrained : EConstraintLevel::Free;

		EConstraintLevel SeamConstraint = bSimplifierConstrainSeams ? EConstraintLevel::Constrained : EConstraintLevel::Free;
		SimplifyOptions.PreserveEdges.UVSeam = SeamConstraint;
		SimplifyOptions.PreserveEdges.NormalSeam = SeamConstraint;
		SimplifyOptions.PreserveEdges.ColorSeam = SeamConstraint;
		SimplifyOptions.PreserveEdges.Material = SeamConstraint;
		SimplifyOptions.PreserveEdges.PolyGroup = SeamConstraint;

		Geometry::MeshClusterSimplify::Simplify(OutMesh, SimplifiedMesh, SimplifyOptions);

		OutMesh = MoveTemp(SimplifiedMesh);
	}
}

void UBooleanModifier::PostUpdateMeshInstance(const FDynamicMesh3& MeshInstanceData)
{
	if (MeshInstanceData.VertexCount() > 0)
	{
		if (!ToolPreviewComponent)
		{
			ToolPreviewComponent = NewObject<UDynamicMeshComponent>(this);
			ToolPreviewComponent->SetIsVisualizationComponent(true);
			ToolPreviewComponent->SetupAttachment(this);
			ToolPreviewComponent->RegisterComponent();
		}
		ToolPreviewComponent->SetMesh(FDynamicMesh3(MeshInstanceData));
		ToolPreviewComponent->SetVisibility(ShouldDrawSolidPreview());
		// Mesh has stripped UVs, but can transfer vertex colors, so default to vertex color sculpt material for visualization
		UMaterialInterface* UseMaterial = ToolSetupUtil::GetVertexColorSculptMaterial(nullptr);
		if (UseMaterial)
		{
			ToolPreviewComponent->SetOverrideRenderMaterial(UseMaterial);
		}
	}
	else
	{
		if (ToolPreviewComponent)
		{
			ToolPreviewComponent->UnregisterComponent();
			ToolPreviewComponent->DestroyComponent();
			ToolPreviewComponent = nullptr;
		}
	}
}

void UBooleanModifier::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		if ((PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBooleanModifier, PreOpSimplifierStrength)
			|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBooleanModifier, bSimplifierConstrainBoundary)
			|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UBooleanModifier, bSimplifierConstrainSeams))
			// We'll only do this once we let go of the slider
			&& PropertyChangedEvent.ChangeType != EPropertyChangeType::Interactive)
		{
			UpdateMeshInstance();
			OnChanged(ComputeBounds(), EChangeType::StateChange);
		}

		if (ToolPreviewComponent)
		{
			ToolPreviewComponent->SetVisibility(ShouldDrawSolidPreview());
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UBooleanModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	if (!ShouldRender())
		return;

	// @todo use some globally configured colors for all the visualizers
	const FColor LocalBoundsColor = FColor::Yellow;
	const FColor GlobalBoundsColor = FColor::Orange;
	constexpr float BoundsThickness = 1;
	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	const FTransform& MeshToWorld = GetComponentTransform();

	if ( (bDrawLocalBounds || bDrawWorldBounds) && MeshInstance)
	{
		const Geometry::FAxisAlignedBox3d& MeshBounds = MeshInstance->GetBounds();
		
		// mesh bounds
		if (bDrawLocalBounds)
		{
			DrawWireBox(PDI, MeshToWorld.ToMatrixWithScale(), FBox(MeshBounds),
				LocalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
		}

		// world space bounds
		if (bDrawWorldBounds)
		{
			FMatrix Id;
			Id.SetIdentity();
			DrawWireBox(PDI, Id, ComputeCombinedBounds(),
				FColor::Green, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);

			// operator bounds, if different
			if (!FVector::Max(FVector::ZeroVector, ExpandOperatorBounds).Equals(FVector::Max3(FVector::ZeroVector, ExpandOperatorBounds, ExpandSectionInclusionBounds)))
			{
				FBox OperatorBounds = ComputeWorldExpandedBounds(FVector::Max(FVector::ZeroVector, ExpandOperatorBounds));
				DrawWireBox(PDI, Id, OperatorBounds,
					FColor::Emerald, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
			}
		}

	}
	
	// draw mesh

	// @todo: calling into PDI->DrawLine will become slow for larger meshes, consider LineSetComponent instead. For now we just
	//  don't draw at a certain point, to avoid hanging the editor.
	const int32 EdgeLimit = 100000;
	if (bDrawWireMesh && MeshInstance
		&& MeshInstance->GetMesh().EdgeCount() <= EdgeLimit)
	{
		const FColor MeshColor(100, 100, 100);
		float LineThickness = 2.f;
		float LineDepthBias = 2.f;
		const Geometry::FDynamicMesh3& Mesh = MeshInstance->GetMesh();
		for (const auto Edge : Mesh.EdgesItr())
		{
			const FVector3d VtxA = MeshToWorld.TransformPosition(Mesh.GetVertex(Edge.Vert.A));
			const FVector3d VtxB = MeshToWorld.TransformPosition(Mesh.GetVertex(Edge.Vert.B));
			PDI->DrawLine(VtxA, VtxB, MeshColor, SDPG_World, LineThickness, LineDepthBias, /*bScreenSpace*/ true);
		}
	}
}

FGuid UBooleanModifier::GetCodeVersionKey() const
{
	return MegaMeshBooleanLocals::FBackgroundOp::GetCodeVersionKey();
}

void UBooleanModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UBooleanModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	if (!MeshInstance)
	{
		return;
	}

	Dependencies += ComputeCombinedBounds();

	const FTransform& MeshToWorld = GetComponentToWorld();
	Dependencies += MeshToWorld;

	Dependencies += BooleanOp;
	Dependencies += ToolMeshEmbedding;
	Dependencies += InsideTargetMeshTestMethod;
	Dependencies += TargetWindingThreshold;
	Dependencies += ToolWindingThreshold;
	Dependencies += bSimplifyAlongNewEdges;
	Dependencies += bWeldSharedEdges;
	Dependencies += ExpandSectionInclusionBounds;
	Dependencies += ExpandOperatorBounds;

	for (const FBooleanModifierWeightEntry& Entry : WeightChannels)
	{
		if (Entry.ChannelName.IsNone())
		{
			continue;
		}
		Dependencies += Entry.ChannelName;
		Dependencies += Entry.SourceMode;
		if (Entry.SourceMode == EBooleanModifierChannelSourceMode::VertexColor)
		{
			Dependencies += Entry.VertexColorIndex;
		}
		else if (Entry.SourceMode == EBooleanModifierChannelSourceMode::VertexWeight)
		{
			Dependencies += Entry.SourceWeightChannelName;
		}
		else // Entry.SourceMode == EBooleanModifierChannelSourceMode::Constant
		{
			Dependencies += Entry.WriteValue;
		}
	}
}

double UBooleanModifier::GetComplexity() const
{
	if (MeshInstance)
	{
		return MeshInstance->GetMesh().VertexCount();
	}
	return 0.;
}

} // namespace UE::MeshPartition
