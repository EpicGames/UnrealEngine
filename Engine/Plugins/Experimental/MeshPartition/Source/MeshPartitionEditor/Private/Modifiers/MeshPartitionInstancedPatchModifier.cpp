// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionInstancedPatchModifier.h"

#include "MeshPartitionMeshView.h"

#include "VisualLogger/VisualLogger.h"

namespace UE::MeshPartition
{
namespace MegaMeshInstancedPatchModifierLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		virtual ~FBackgroundOp() {}

		MeshPartition::UInstancedPatchModifier::FSettings Settings;
		FTransform ComponentTransform;
		TSharedPtr<const TArray<FVector>> Instances;

		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;
		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		
		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("9844911f-e523-4c24-9110-1ce703abd5d5"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};

	FBox GetInstanceWorldspaceBounds(const FVector& InstanceLocation, const FTransform& ComponentTransform, float Radius, float Falloff, float MaxZDistance)
	{
		const FBox LocalBounds = FBox(InstanceLocation - FVector(Radius + Falloff, Radius + Falloff, MaxZDistance / 2.f), InstanceLocation + FVector(Radius + Falloff, Radius + Falloff, MaxZDistance / 2.f));
		return LocalBounds.TransformBy(ComponentTransform);
	}
}

UInstancedPatchModifier::UInstancedPatchModifier()
{
}

void UInstancedPatchModifier::SetDisabledByCode(bool bDisabledByCodeIn)
{
	if (bDisabledByCode == bDisabledByCodeIn)
	{
		return;
	}
	bDisabledByCode = bDisabledByCodeIn;

	// ComputeBounds() will give us empty bounds once we're disabled. Currently, previous bounds
	//  are automatically added by the OnChanged call.
	OnChanged(ComputeBounds(), bDisabledByCodeIn ? EChangeType::TransientStateChange : EChangeType::StateChange);
}

void UInstancedPatchModifier::ResetForReuse()
{
	ClearInstances();
}

bool UInstancedPatchModifier::IsUsed() const
{
	return NumInstances() != 0;
}

TArray<FBox> UInstancedPatchModifier::ComputeBounds() const
{
	TArray<FBox> BoundingBoxes;
	for (int32 Index = 0; Index < Instances.Num(); ++Index)
	{
		BoundingBoxes.Emplace(GetInstanceWorldspaceBounds(Index));
	}
	return BoundingBoxes;
}

void MegaMeshInstancedPatchModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	FInstanceInfo Desc;
	Desc.ReadViewComponents = EMeshViewComponents::VertexPos;
	Desc.WriteViewComponents = EMeshViewComponents::VertexPos;

	if (Settings.bWriteToWeightChannel && !Settings.WeightChannelName.IsNone())
	{
		Desc.WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
		Desc.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
		Desc.UsedChannels.Add(Settings.WeightChannelName);
	}
	
	for (int32 Index = 0; Index < Instances->Num(); ++Index)
	{
		Desc.Bounds = GetInstanceWorldspaceBounds((*Instances)[Index], ComponentTransform, 
			Settings.Radius, Settings.Falloff, Settings.MaxZDistance);
		Desc.InstanceID = Index;

		if (Desc.Bounds.Intersect(InBounds))
		{
			OutInstanceInfos.Add(Desc);
		}
	}
}

void UInstancedPatchModifier::AddInstances(const TArray<FVector>& InNewInstances)
{
	// Will need to create a new copy for background op
	InstancesForBackgroundOp.Reset();

	Instances.Reserve(Instances.Num() + InNewInstances.Num());

	TArray<FBox> NewBounds;
	for (const FVector& InstanceLocation : InNewInstances)
	{
		const int InstanceID = Instances.Emplace(InstanceLocation);
		NewBounds.Emplace(GetInstanceWorldspaceBounds(InstanceID));
	}
	OnChanged(NewBounds, EChangeType::StateChange);
}

void UInstancedPatchModifier::ClearInstances()
{
	InstancesForBackgroundOp.Reset();
	Instances.Empty();
}

FVector UInstancedPatchModifier::GetInstanceWorldspaceLocation(int InInstanceID) const
{
	const bool bValidInstanceID = (Instances.Num() > 0) && (InInstanceID < Instances.Num());

	check(bValidInstanceID);

	return GetComponentTransform().TransformPosition(Instances[InInstanceID]);
}
FBox UInstancedPatchModifier::GetInstanceWorldspaceBounds(int InInstanceID) const
{
	return MegaMeshInstancedPatchModifierLocals::GetInstanceWorldspaceBounds(
		Instances[InInstanceID],
		GetComponentTransform(),
		Radius,
		Falloff,
		MaxZDistance
	);
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UInstancedPatchModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshInstancedPatchModifierLocals;

	TSharedPtr<FBackgroundOp> Op = MakeShared<FBackgroundOp>(GetFName());
	Op->Settings = FSettings(*this);
	Op->ComponentTransform = GetComponentTransform();

	if (!InstancesForBackgroundOp.IsValid())
	{
		InstancesForBackgroundOp = MakeShared<const TArray<FVector>>(Instances);
	}
	Op->Instances = InstancesForBackgroundOp;

	return Op;
}

void MegaMeshInstancedPatchModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedPatchModifier::ApplyModifications);

	if (!ensure(Instances && Instances->IsValidIndex(InInstanceDesc.InstanceID)))
	{
		return;
	}

	const FVector MeshSpaceLocation = InTransform.InverseTransformPosition(
		ComponentTransform.TransformPosition((*Instances)[InInstanceDesc.InstanceID]));

	UInstancedPatchModifier::ApplyDeformation(Settings, InMeshView, MeshSpaceLocation);
}

void UInstancedPatchModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedPatchModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	Dependencies += Instances;
}

FGuid UInstancedPatchModifier::GetCodeVersionKey() const
{
	return MegaMeshInstancedPatchModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

bool UInstancedPatchModifier::IsTemporarilyDisabledInEditor() const
{
	return Super::IsTemporarilyDisabledInEditor() || bDisabledByCode;
}
} // namespace UE::MeshPartition
