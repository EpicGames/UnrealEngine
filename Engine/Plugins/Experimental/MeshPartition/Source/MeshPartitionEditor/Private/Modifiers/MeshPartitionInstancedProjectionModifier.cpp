// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionInstancedProjectionModifier.h"

#include "Curves/CurveFloat.h"
#include "MeshPartitionMeshView.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "Modifiers/MeshPartitionMeshProjectModifier.h"
#include "PrimitiveDrawingUtils.h" // DrawWireBox

namespace UE::MeshPartition
{
namespace MegaMeshInstancedMeshProjectModifierLocals
{
	// Uses projection transform and bounds to get the bounds in world space
	FBox GetInstanceWorldspaceBounds(const MeshPartition::FInstancedProjectionModifierInstance& Instance)
	{
		Geometry::FAxisAlignedBox3d ProjectionWorldBounds(Instance.ProjectionSpaceBounds, Instance.ProjectionToWorld);
		return FBox(ProjectionWorldBounds);
	}

	struct FInstanceOnBackgroundOp
	{
		TSharedPtr<const Geometry::FDynamicMesh3> Mesh = nullptr;
		TSharedPtr<const Geometry::FDynamicMeshAABBTree3> Spatial = nullptr;
		FTransform MeshToWorld = FTransform::Identity;
		FTransform ProjectionToWorld = FTransform::Identity;
		FBox ProjectionSpaceBounds;
	};

	FBox GetInstanceWorldspaceBounds(const FInstanceOnBackgroundOp& Instance)
	{
		Geometry::FAxisAlignedBox3d ProjectionWorldBounds(Instance.ProjectionSpaceBounds, Instance.ProjectionToWorld);
		return FBox(ProjectionWorldBounds);
	}

	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		TArray<FInstanceOnBackgroundOp> Instances;
		MeshPartition::UMeshProjectModifier::FProjectOntoMeshOptionalParams OptionalParams;

		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;
		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("37e0a43e-46ff-623e-365d-d1a7f2826039"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};
}

UInstancedProjectionModifier::UInstancedProjectionModifier()
{
}

void UInstancedProjectionModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const FName ParentPreviousDefaultType = TEXT("Misc");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("Mesh");
		SetType(PreviousModifierDefaultType);
	}
}

void UInstancedProjectionModifier::InitializeModifier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedProjectionModifier::InitializeModifier);

	Super::InitializeModifier();

	AttachCurveListeners();

	for (const MeshPartition::FInstancedProjectionModifierInstance& Instance : Instances)
	{
		CreateCachedData(Instance.Mesh, /*bUpdateExisting*/ false);
	}
}

void UInstancedProjectionModifier::UninitializeModifier()
{
	Super::UninitializeModifier();
	DetachCurveListeners();
}

void UInstancedProjectionModifier::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange.GetTail())
	{
		return;
	}

	FProperty* Property = PropertyAboutToChange.GetTail()->GetValue();
	if (!Property)
	{
		return;
	}

	// Handle the actual curve property, or addition/removal of the containing structs
	if (Property->GetName() == GET_MEMBER_NAME_CHECKED(MeshPartition::FProjectModifierFalloffSettings, FalloffCurve)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedProjectionModifier, WeightChannels)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(FProjectModifierWeightEntry, FalloffOverrides))
	{
		DetachCurveListeners();
	}
}

void UInstancedProjectionModifier::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	ON_SCOPE_EXIT{ Super::PostEditChangeProperty(PropertyChangedEvent); };

	if (!PropertyChangedEvent.PropertyChain.GetTail())
	{
		return;
	}
	FProperty* Property = PropertyChangedEvent.PropertyChain.GetTail()->GetValue();
	if (!Property)
	{
		return;
	}
	UObject* OwnerObject = Property->GetOwnerUObject();

	if (Property->GetName() == GET_MEMBER_NAME_CHECKED(MeshPartition::FProjectModifierFalloffSettings, FalloffCurve)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UInstancedProjectionModifier, WeightChannels)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(FProjectModifierWeightEntry, FalloffOverrides))
	{
		AttachCurveListeners();
	}
}

void UInstancedProjectionModifier::AttachCurveListeners()
{
	auto AttachListeners = [this](MeshPartition::FProjectModifierFalloffSettings& Settings)
	{
		if (Settings.FalloffCurve && !Settings.FalloffCurveListenerHandle.IsValid())
		{
			Settings.FalloffCurveListenerHandle = Settings.FalloffCurve->OnUpdateCurve.AddUObject(this, &UInstancedProjectionModifier::OnCurveChanged);
		}
	};

	AttachListeners(HeightFalloff);

	for (FProjectModifierWeightEntry& WeightChannelEntry : WeightChannels)
	{
		if (WeightChannelEntry.FalloffOverrides.IsSet())
		{
			AttachListeners(*WeightChannelEntry.FalloffOverrides);
		}
	}
}

void UInstancedProjectionModifier::DetachCurveListeners()
{
	auto DetachListeners = [](MeshPartition::FProjectModifierFalloffSettings& Settings)
	{
		if (Settings.FalloffCurve && Settings.FalloffCurveListenerHandle.IsValid())
		{
			Settings.FalloffCurve->OnUpdateCurve.Remove(Settings.FalloffCurveListenerHandle);
			Settings.FalloffCurveListenerHandle.Reset();
		}
	};

	DetachListeners(HeightFalloff);

	for (FProjectModifierWeightEntry& WeightChannelEntry : WeightChannels)
	{
		if (WeightChannelEntry.FalloffOverrides.IsSet())
		{
			DetachListeners(*WeightChannelEntry.FalloffOverrides);
		}
	}
}

void UInstancedProjectionModifier::OnCurveChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType)
{
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

TArray<FBox> UInstancedProjectionModifier::ComputeBounds() const
{
	using namespace MegaMeshInstancedMeshProjectModifierLocals;

	TArray<FBox> BoundingBoxes;
	for (const MeshPartition::FInstancedProjectionModifierInstance& Instance : Instances)
	{
		// TODO: Could consider caching the world space bounds
		BoundingBoxes.Emplace(GetInstanceWorldspaceBounds(Instance));
	}
	return BoundingBoxes;
}

void MegaMeshInstancedMeshProjectModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	FInstanceInfo Desc;
	Desc.ReadViewComponents = EMeshViewComponents::VertexPos;
	Desc.WriteViewComponents = EMeshViewComponents::VertexPos;
	
	for (const MeshPartition::UMeshProjectModifier::FWeightEntryParams& WeightChannelEntry : OptionalParams.WeightEntries)
	{
		if (!WeightChannelEntry.WeightChannelName.IsNone())
		{
			Desc.UsedChannels.Emplace(WeightChannelEntry.WeightChannelName);

			Desc.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
			Desc.WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
		}
	}

	for (int32 i = 0; i < Instances.Num(); ++i)
	{
		Desc.Bounds = GetInstanceWorldspaceBounds(Instances[i]);
		Desc.InstanceID = i;
		if (Desc.Bounds.Intersect(InBounds))
		{
			OutInstanceInfos.Add(Desc);
		}
	}
}

void UInstancedProjectionModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	using namespace MegaMeshInstancedMeshProjectModifierLocals;

	const FColor LocalBoundsColor = FColor::Yellow;
	const FColor GlobalBoundsColor = FColor::Orange;
	constexpr float BoundsThickness = 1;
	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	if (!bDrawAffectedBox && !bDrawGlobalBounds)
	{
		return;
	}
	for (const MeshPartition::FInstancedProjectionModifierInstance& Instance : Instances)
	{
		if (bDrawAffectedBox)
		{
			DrawWireBox(PDI, Instance.ProjectionToWorld.ToMatrixWithScale(), FBox(Instance.ProjectionSpaceBounds),
				LocalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
		}
		if (bDrawGlobalBounds)
		{
			DrawWireBox(PDI, FMatrix::Identity, GetInstanceWorldspaceBounds(Instance),
				GlobalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
		}
	}
}

void UInstancedProjectionModifier::AddInstance(const MeshPartition::FInstancedProjectionModifierInstance& NewInstance, bool bUpdateCached)
{
	using namespace MegaMeshInstancedMeshProjectModifierLocals;

	MeshPartition::FInstancedProjectionModifierInstance& InstanceCopy = Instances.Emplace_GetRef(NewInstance);
	InstanceCopy.Mesh = GetOrCreateMeshCopy(NewInstance.Mesh);
	CreateCachedData(InstanceCopy.Mesh, bUpdateCached);
	
	OnChanged({ GetInstanceWorldspaceBounds(NewInstance) }, EChangeType::StateChange);
}

void UInstancedProjectionModifier::ClearInstances()
{
	Instances.Reset();
	SourceMeshToInstanceMesh.Reset();
	ThreadSafeMeshCopies.Reset();
	MeshSpatials.Reset();
	
	// Currently, previous bounds are automatically added by the OnChanged call, else we would
	//  need to gather them before the reset to pass them here.
	OnChanged({}, EChangeType::StateChange);
}

const MeshPartition::FInstancedProjectionModifierInstance* UInstancedProjectionModifier::GetInstanceAtIndex(int32 Index) const
{
	if (ensure(Instances.IsValidIndex(Index)))
	{
		return &Instances[Index];
	}
	return nullptr;
}

bool UInstancedProjectionModifier::CreateCachedData(const UDynamicMesh* MeshKey, bool bUpdateIfExisting)
{
	if (!MeshKey)
	{
		return false;
	}

	TSharedPtr<const Geometry::FDynamicMesh3>* Found = ThreadSafeMeshCopies.Find(MeshKey);
	if (Found && Found->IsValid()
		// If we need to update, we create new copies so that our background op is safe
		&& !bUpdateIfExisting 
		&& ensure(MeshSpatials.Contains(MeshKey)))
	{
		return false;
	}

	TSharedPtr<Geometry::FDynamicMesh3> MeshCopy = MakeShared<Geometry::FDynamicMesh3>();
	MeshKey->ProcessMesh([MeshCopy](const FDynamicMesh3& Source)
	{
		MeshCopy->Copy(Source);
	});
	ThreadSafeMeshCopies.Add(MeshKey, MeshCopy);

	TSharedPtr<Geometry::FDynamicMeshAABBTree3> MeshSpatial = MakeShared<Geometry::FDynamicMeshAABBTree3>();
	MeshSpatial->SetMesh(MeshCopy.Get(), /*bAutoBuild*/ true);
	MeshSpatials.Add(MeshKey, MeshSpatial);

	return true;
}

UDynamicMesh* UInstancedProjectionModifier::GetOrCreateMeshCopy(const UDynamicMesh* MeshKey)
{
	if (!MeshKey)
	{
		return nullptr;
	}

	TWeakObjectPtr<UDynamicMesh>* Found = SourceMeshToInstanceMesh.Find(MeshKey);
	if (Found && Found->IsValid())
	{
		return Found->Get();
	}
	UDynamicMesh* MeshCopy = NewObject<UDynamicMesh>(this);
	MeshKey->ProcessMesh([MeshCopy](const FDynamicMesh3& Source)
	{
		MeshCopy->SetMesh(Source);
	});
	SourceMeshToInstanceMesh.Add(MeshKey, MeshCopy);

	return MeshCopy;
}

void UInstancedProjectionModifier::SetDisabledByCode(bool bDisabledByCodeIn)
{
	if (bDisabledByCode == bDisabledByCodeIn)
	{
		return;
	}
	bDisabledByCode = bDisabledByCodeIn;

	// ComputeBounds() will give us empty bounds once we're disabled. Currently, previous bounds
	//  are automatically added by the OnChanged call.
	OnChanged(ComputeBounds(), bDisabledByCodeIn ? EChangeType::TransientStateChange : EChangeType::StateChange);
}

void UInstancedProjectionModifier::ResetForReuse()
{
	ClearInstances();
}

bool UInstancedProjectionModifier::IsUsed() const
{
	return NumInstances() != 0;
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UInstancedProjectionModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshInstancedMeshProjectModifierLocals;

	if (bDisabledByCode)
	{
		return nullptr;
	}

	TSharedPtr<FBackgroundOp> Op = MakeShared<FBackgroundOp>(GetFName());
	for (const MeshPartition::FInstancedProjectionModifierInstance& Instance : Instances)
	{
		if (!ensure(ThreadSafeMeshCopies.Contains(Instance.Mesh) && MeshSpatials.Contains(Instance.Mesh)))
		{
			continue;
		}

		FInstanceOnBackgroundOp& InstanceCopy = Op->Instances.Emplace_GetRef();
		InstanceCopy.Mesh = ThreadSafeMeshCopies[Instance.Mesh];
		InstanceCopy.Spatial = MeshSpatials[Instance.Mesh];
		InstanceCopy.MeshToWorld = Instance.MeshToWorld;
		InstanceCopy.ProjectionToWorld = Instance.ProjectionToWorld;
		InstanceCopy.ProjectionSpaceBounds = Instance.ProjectionSpaceBounds;
	}

	Op->OptionalParams.BlendMode = BlendMode;

	Op->OptionalParams.FalloffSettings.Initialize(HeightFalloff);

	for (const FProjectModifierWeightEntry& ChannelEntry : WeightChannels)
	{
		if (ChannelEntry.ChannelName.IsNone())
		{
			continue;
		}

		MeshPartition::UMeshProjectModifier::FWeightEntryParams& OpEntry = Op->OptionalParams.WeightEntries.Emplace_GetRef();
		OpEntry.Initialize(ChannelEntry);
	}

	return Op;
}

void MegaMeshInstancedMeshProjectModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView& MeshView,
	const FTransform3d& MegaMeshTransform, const FInstanceInfo& InstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UInstancedProjectionModifier::ApplyModifications);

	if (!ensure(Instances.IsValidIndex(InstanceDesc.InstanceID)))
	{
		return;
	}

	const FInstanceOnBackgroundOp& Instance = Instances[InstanceDesc.InstanceID];
	if (!ensure(Instance.Mesh && Instance.Spatial))
	{
		return;
	}

	MeshPartition::UMeshProjectModifier::ProjectOntoMesh(MeshView, MegaMeshTransform,
		*Instance.Mesh, Instance.MeshToWorld, *Instance.Spatial,
		Instance.ProjectionToWorld, Instance.ProjectionSpaceBounds,
		OptionalParams);
}

FGuid UInstancedProjectionModifier::GetCodeVersionKey() const
{
	return MegaMeshInstancedMeshProjectModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

bool UInstancedProjectionModifier::IsTemporarilyDisabledInEditor() const
{
	return Super::IsTemporarilyDisabledInEditor() || bDisabledByCode;
}
} // namespace UE::MeshPartition