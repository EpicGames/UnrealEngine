// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionLatticeModifier.h"
#include "DeformationOps/LatticeDeformerOp.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartition.h"
#include "ScopedTransaction.h"

// for starting/monitoring the tool:
#include "EditorModeManager.h"	
#include "ModelingToolsEditorMode.h"
#include "ModelingToolsManagerActions.h"
#include "Toolkits/BaseToolkit.h"
#include "InteractiveToolManager.h"

#define LOCTEXT_NAMESPACE "MegaMeshLatticeModifier"

//
// FLatticeModifierBackgroundOp
//

namespace UE::MeshPartition
{
namespace MegaMeshLatticeModifierLocals
{
	class FLatticeModifierBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{

	public:

		FBox LocalBounds;
		FBox GlobalBounds;
		FTransform LatticeTransform;
		FTransform MegaMeshComponentTransform;
		Geometry::FVector3i LatticeResolution;
		Geometry::ELatticeInterpolation InterpolationType;
		TArray<FVector3d> LatticeControlPoints;
	
		FLatticeModifierBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

	private:

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;

		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid Key(TEXT("a481f3f9-27f2-4d34-afe1-223758b29d7d"));
			return Key;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const
		{
			return false;
		}

	};

	void FLatticeModifierBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
	{
		AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);

		if (!OutInstanceInfos.IsEmpty())
		{
			OutInstanceInfos[0].ReadViewComponents = EMeshViewComponents::DynamicSubmesh;
			OutInstanceInfos[0].WriteViewComponents = EMeshViewComponents::DynamicSubmesh;
		}
	}

	void FLatticeModifierBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InOutMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::ULatticeModifier::ApplyModifications);

		TSharedPtr<FDynamicMesh3> OriginalMesh = MakeShared<FDynamicMesh3>();
		OriginalMesh->Copy(InOutMeshView.GetSubmesh());

		MeshTransforms::ApplyTransform(*OriginalMesh, MegaMeshComponentTransform);

		constexpr float Padding = 0.0f;
		TSharedPtr<Geometry::FFFDLattice> Lattice = MakeShared<Geometry::FFFDLattice>(LatticeResolution, OriginalMesh.Get(), Padding, LocalBounds, LatticeTransform);

		ensure(LatticeControlPoints.Num() == LatticeResolution[0] * LatticeResolution[1] * LatticeResolution[2]);

		constexpr bool bDeformNormals = true;
		Geometry::FLatticeDeformerOp LatticeDeformerOp(OriginalMesh, Lattice, LatticeControlPoints, InterpolationType, bDeformNormals);
		LatticeDeformerOp.CalculateResult(nullptr);

		TUniquePtr<FDynamicMesh3> OpResult = LatticeDeformerOp.ExtractResult();

		MeshTransforms::ApplyTransformInverse(*OpResult, MegaMeshComponentTransform);
		InOutMeshView.GetSubmeshMutable() = MoveTemp(*OpResult);
	}

	// Take a new lattice and run it through the existing lattice deformer to get the new deformed lattice positions
	void TransferLatticePoints(const Geometry::FAxisAlignedBox3d& LocalBounds, 
		const Geometry::FVector3i& OriginalLatticeResolution, 
		const TArray<FVector3d>& OriginalLatticeDeformedPoints, 
		const Geometry::FVector3i& NewLatticeResolution, 
		const Geometry::ELatticeInterpolation InterpolationType,
		TArray<FVector3d>& NewLatticePoints)
	{
		ensure(OriginalLatticeDeformedPoints.Num() == OriginalLatticeResolution.X * OriginalLatticeResolution.Y * OriginalLatticeResolution.Z);
		constexpr float Padding = 0.0f;

		TArray<FVector3d> NewLatticeInitialPositions;
		TSharedPtr<Geometry::FFFDLattice> NewLattice = MakeShared<Geometry::FFFDLattice>(NewLatticeResolution, nullptr, Padding, LocalBounds);
		NewLattice->GenerateInitialLatticePositions(NewLatticeInitialPositions);

		FDynamicMesh3 Mesh;
		for (const FVector3d& LatticePosition : NewLatticeInitialPositions)
		{
			Mesh.AppendVertex(LatticePosition);
		}
		TSharedPtr<Geometry::FFFDLattice> OriginalLattice = MakeShared<Geometry::FFFDLattice>(OriginalLatticeResolution, &Mesh, Padding, LocalBounds);
		OriginalLattice->GetDeformedMeshVertexPositions(OriginalLatticeDeformedPoints, NewLatticePoints,	InterpolationType);

		ensure(NewLatticePoints.Num() == NewLatticeInitialPositions.Num());
		ensure(NewLatticePoints.Num() == NewLatticeResolution.X * NewLatticeResolution.Y * NewLatticeResolution.Z);
	}


	void ResizeLatticeBounds(const FVector3d& OriginalExtents,
		const FVector3d& NewExtents,
		const TArray<FVector3d>& OriginalLatticeDeformedPoints,
		TArray<FVector3d>& NewLatticePoints)
	{
		const FVector3d ScaleOldToNew = NewExtents / OriginalExtents;
		NewLatticePoints.SetNum(OriginalLatticeDeformedPoints.Num());
		for (int32 PointIndex = 0; PointIndex < OriginalLatticeDeformedPoints.Num(); ++PointIndex)
		{
			NewLatticePoints[PointIndex] = ScaleOldToNew * OriginalLatticeDeformedPoints[PointIndex];
		}
	}

} // namespace MegaMeshLatticeModifierLocals


//
// MeshPartition::ULatticeModifier
//

#if WITH_EDITOR

void ULatticeModifier::PostInitProperties()
{
	Super::PostInitProperties();

	if (!HasAnyFlags(RF_ClassDefaultObject))
	{
		// Initialize the first lattice points
		if (LatticeControlPoints.Num() != LatticeResolution[0] * LatticeResolution[1] * LatticeResolution[2])
		{
			InitializeControlPoints();
		}
	}
}

void ULatticeModifier::BeginDestroy()
{
	Super::BeginDestroy();

	OnToolEndedDelegateHandle.Reset();
	OnToolStartedDelegateHandle.Reset();
	OnEditorModeChanged.Reset();
}

void ULatticeModifier::PreEditChange(FProperty* PropertyThatWillChange)
{
	Super::PreEditChange(PropertyThatWillChange);

	const FName PropertyName = PropertyThatWillChange ? PropertyThatWillChange->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FIntVector, X) || PropertyName == GET_MEMBER_NAME_CHECKED(FIntVector, Y) || PropertyName == GET_MEMBER_NAME_CHECKED(FIntVector, Z))
	{
		PreviousLatticeResolution = Geometry::FVector3i(LatticeResolution);
	}
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FVector3d, X) || PropertyName == GET_MEMBER_NAME_CHECKED(FVector3d, Y) || PropertyName == GET_MEMBER_NAME_CHECKED(FVector3d, Z))
	{
		PreviousLatticeExtents = Extents;
	}
}

void ULatticeModifier::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.Property ? PropertyChangedEvent.Property->GetFName() : NAME_None;

	// Check resolution change
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FIntVector, X) || PropertyName == GET_MEMBER_NAME_CHECKED(FIntVector, Y) || PropertyName == GET_MEMBER_NAME_CHECKED(FIntVector, Z))
	{
		if (FEditPropertyChain::TDoubleLinkedListNode* Node = PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
		{
			if (Node->GetValue()->GetName() == GET_MEMBER_NAME_CHECKED(ULatticeModifier, LatticeResolution))
			{
				const Geometry::FVector3i NewLatticeResolution(LatticeResolution);
				TArray<FVector3d> NewLatticePoints;
				MegaMeshLatticeModifierLocals::TransferLatticePoints(Geometry::FAxisAlignedBox3d(ComputeLocalBounds()), 
					PreviousLatticeResolution, 
					LatticeControlPoints, 
					NewLatticeResolution, 
					Geometry::ELatticeInterpolation(InterpolationType),
					NewLatticePoints);

				LatticeControlPoints = NewLatticePoints;
				
				OnChanged(ComputeBounds(), EChangeType::StateChange);
			}
		}
	}

	// Check box size change
	if (PropertyName == GET_MEMBER_NAME_CHECKED(FVector3d, X) || PropertyName == GET_MEMBER_NAME_CHECKED(FVector3d, Y) || PropertyName == GET_MEMBER_NAME_CHECKED(FVector3d, Z))
	{
		if (FEditPropertyChain::TDoubleLinkedListNode* Node = PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
		{
			if (Node->GetValue()->GetName() == GET_MEMBER_NAME_CHECKED(ULatticeModifier, Extents))
			{
				TArray<FVector3d> NewLatticePoints;
				MegaMeshLatticeModifierLocals::ResizeLatticeBounds(PreviousLatticeExtents, Extents, LatticeControlPoints, NewLatticePoints);

				LatticeControlPoints = NewLatticePoints;
				OnChanged(ComputeBounds(), EChangeType::StateChange);
			}
		}
	}


}

#endif

TSharedPtr<const MeshPartition::IModifierBackgroundOp> ULatticeModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	TSharedPtr<MegaMeshLatticeModifierLocals::FLatticeModifierBackgroundOp> Op = MakeShared<MegaMeshLatticeModifierLocals::FLatticeModifierBackgroundOp>(GetFName());
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->LocalBounds = ComputeLocalBounds();
	Op->LatticeTransform = GetComponentToWorld();
	Op->LatticeResolution = Geometry::FVector3i(LatticeResolution);
	Op->InterpolationType = Geometry::ELatticeInterpolation(InterpolationType);
	ReadLatticePoints(Op->LatticeControlPoints);
	if (AMeshPartition* MP = GetAffectedMeshPartition())
	{
		Op->MegaMeshComponentTransform = MP->GetTransform();
	}

	return Op;
}

TArray<FBox> ULatticeModifier::ComputeBounds() const
{
	FBox LocalBounds = ComputeLocalBounds();
	return { LocalBounds.TransformBy(GetComponentToWorld()) };
}

FBox ULatticeModifier::ComputeLocalBounds() const
{
	return FBox(-0.5 * Extents, 0.5*Extents);
}

void ULatticeModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	if (bDrawLocalBounds)
	{
		const FMatrix Matrix = GetComponentToWorld().ToMatrixWithScale();
		const FBox LocalBounds = ComputeLocalBounds();
		DrawWireBox(PDI, Matrix, LocalBounds, FLinearColor::Yellow, SDPG_World, 1.0, 0, true);
	}

	if (bDrawLatticePoints)
	{
		const FMatrix Matrix = GetComponentToWorld().ToMatrixWithScale();
		for (const FVector3d& Pos : LatticeControlPoints)
		{
			const FVector3d DrawPos = Matrix.TransformPosition(Pos);
			PDI->DrawPoint(DrawPos, FLinearColor::Yellow, 3.0, SDPG_World);
		}
	}
}

void ULatticeModifier::PrepareForEdit(FDynamicMesh3& EditMesh) const
{
	MeshTransforms::ApplyTransform(EditMesh, GetComponentToWorld());
}

void ULatticeModifier::StoreLatticePoints(const TArray<FVector3d>& InLatticePoints)
{
	FScopedTransaction Transaction(LOCTEXT("StoreLatticePointsTransaction", "Set Lattice Control Points"));
	Modify();

	LatticeControlPoints = InLatticePoints;

	// Save points in component space
	const FMatrix ToComponentMatrix = GetComponentToWorld().ToMatrixWithScale().Inverse();
	for (FVector3d& SavedPoint : LatticeControlPoints)
	{
		SavedPoint = ToComponentMatrix.TransformPosition(SavedPoint);
	}

	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void ULatticeModifier::ReadLatticePoints(TArray<FVector3d>& OutLatticePoints) const
{
	OutLatticePoints = LatticeControlPoints;

	// Return points in world space
	const FMatrix ToWorldMatrix = GetComponentToWorld().ToMatrixWithScale();
	for (FVector3d& OutPoint : OutLatticePoints)
	{
		OutPoint = ToWorldMatrix.TransformPosition(OutPoint);
	}
}

void ULatticeModifier::StoreInterpolationType(ELatticeInterpolationType InInterpolationType)
{
	InterpolationType = InInterpolationType;

	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

ELatticeInterpolationType ULatticeModifier::ReadInterpolationType() const
{
	return InterpolationType;
}

Geometry::FAxisAlignedBox3d ULatticeModifier::GetInitialBounds() const
{
	const FBox LocalBounds = ComputeLocalBounds();
	return Geometry::FAxisAlignedBox3d(LocalBounds);
}

Geometry::FTransformSRT3d ULatticeModifier::GetTransform() const
{
	return GetComponentToWorld();
}

void ULatticeModifier::ClearLattice()
{
	if (!bToolIsRunning)
	{
		FScopedTransaction Transaction(LOCTEXT("ClearLatticePointsTransaction", "Clear Lattice Control Points"));
		Modify();

		LatticeControlPoints.Reset();
		OnChanged(ComputeBounds(), EChangeType::StateChange);
		InitializeControlPoints();
	}
}

void ULatticeModifier::StartLatticeTool()
{
	if (UEdMode* const EditorMode = GLevelEditorModeTools().GetActiveScriptableMode(UModelingToolsEditorMode::EM_ModelingToolsEditorModeId))
	{
		if (UModelingToolsEditorMode* const ModelingMode = Cast<UModelingToolsEditorMode>(EditorMode))
		{
			if (const TSharedPtr<const FModeToolkit> PinnedToolkit = ModelingMode->GetToolkit().Pin())
			{
				const TSharedRef<FUICommandList>& CommandList = PinnedToolkit->GetToolkitCommands();
				const FModelingToolsManagerCommands& ToolManagerCommands = FModelingToolsManagerCommands::Get();
				const TSharedRef<const FUICommandInfo> CommandInfo = ToolManagerCommands.BeginLatticeDeformerTool.ToSharedRef();
				if (CommandList->CanExecuteAction(CommandInfo))
				{
					CommandList->ExecuteAction(CommandInfo);
				}
			}
		}
	}
}

Geometry::FVector3i ULatticeModifier::GetResolution() const
{
	return Geometry::FVector3i(LatticeResolution);
}

void ULatticeModifier::InteractiveToolStarted()
{
	bToolIsRunning = true;
}

void ULatticeModifier::InteractiveToolShutDown()
{
	bToolIsRunning = false;
}

void ULatticeModifier::InitializeControlPoints()
{
	// If we don't have the right number of lattice control points, create them now
	const FBox LocalBounds = ComputeLocalBounds();
	const Geometry::FTransformSRT3d Transform(GetComponentToWorld());

	// Create an FFDLattice with just enough information to generate the initial control points
	constexpr float Padding = 0.0f;
	const TSharedPtr<const Geometry::FFFDLattice> Lattice = MakeShared<Geometry::FFFDLattice>(GetResolution(), nullptr, Padding, LocalBounds, Transform);

	TArray<FVector3d> InitialLatticeControlPoints;
	Lattice->GenerateInitialLatticePositions(InitialLatticeControlPoints);
	StoreLatticePoints(InitialLatticeControlPoints);
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
