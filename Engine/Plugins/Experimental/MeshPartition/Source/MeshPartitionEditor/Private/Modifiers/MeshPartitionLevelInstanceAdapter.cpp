// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionLevelInstanceAdapter.h"

#include "LevelInstance/LevelInstanceActor.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartition.h"
#include "Engine/World.h"
#include "AssetCompilingManager.h"
#include "Engine/Level.h"


namespace UE::MeshPartition
{
ULevelInstanceAdapter::ULevelInstanceAdapter()
{
	FWorldDelegates::LevelAddedToWorld.AddUObject(this, &ULevelInstanceAdapter::OnLevelAdded);
	PrimaryComponentTick.bCanEverTick = true;
}

ULevelInstanceAdapter::~ULevelInstanceAdapter()
{
	FWorldDelegates::LevelAddedToWorld.RemoveAll(this);
}

TArray<FBox> ULevelInstanceAdapter::ComputeBounds() const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(ULevelInstanceAdapter::ComputeBounds);

	TArray<FBox> AllBounds;
	ILevelInstanceInterface* LevelInstance = GetOwningLevelInstance();
	UWorld* World = GetWorld();

	ForEachModifierInLevelInstance([&AllBounds](MeshPartition::UModifierComponent* Modifier)
		{
			AllBounds.Append(Modifier->ComputeBounds());
		});

	return AllBounds;
}

TArray<MeshPartition::UModifierComponent*> ULevelInstanceAdapter::GetInteractiveProxies()
{
	return OwnedModifiers.Array();
}

void ULevelInstanceAdapter::OnChanged(TConstArrayView<const FBox> InBoundingBoxes, EChangeType InChangeType)
{
	// Forward OnChanged to the transient instances of modifiers inside this level instance so their cache keys are updated.
	// If this is not done, the cache keys will remain the same and it will be as if the level instance was never modified.
	ForEachModifierInLevelInstance([InChangeType](MeshPartition::UModifierComponent* Modifier)
		{
			Modifier->OnChanged(Modifier->ComputeBounds(), InChangeType);
		});

	Super::OnChanged(InBoundingBoxes, InChangeType);
}

ILevelInstanceInterface* ULevelInstanceAdapter::GetOwningLevelInstance() const
{
	return GetOwner<ILevelInstanceInterface>();
}

void ULevelInstanceAdapter::ForEachModifierInLevelInstance(TFunctionRef<void(MeshPartition::UModifierComponent*)> InOp) const
{
	ILevelInstanceInterface* LevelInstance = GetOwningLevelInstance();
	UWorld* World = GetWorld();
	if ((LevelInstance != nullptr) && (World != nullptr))
	{
		const ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();

		if (ensure(LevelInstanceSubsystem != nullptr && !LevelInstanceSubsystem->IsEditingLevelInstance(LevelInstance)))
		{
			LevelInstanceSubsystem->ForEachActorInLevelInstance(LevelInstance, [InOp](AActor* SubActor)
			{
				TInlineComponentArray<MeshPartition::UModifierComponent*> Modifiers(SubActor);
				for (MeshPartition::UModifierComponent* Modifier : Modifiers)
				{
					InOp(Modifier);
				}
				return true;
			});
		}
	}
}

void ULevelInstanceAdapter::OnLevelAdded(ULevel* InLevel, UWorld* InWorld)
{
	UWorld* World = GetWorld();
	if (World == nullptr || World != InWorld)
	{
		return;
	}

	ILevelInstanceInterface* LevelInstance = GetOwningLevelInstance();
	
	if (LevelInstance == nullptr)
	{
		return;
	}

	if (LevelInstance->IsEditing())
	{
		OwnedModifiers.Reset();
		return;
	}

	const ULevelInstanceSubsystem* LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>();
	check(LevelInstanceSubsystem);
	if (ILevelInstanceInterface* LevelInstanceInterface = LevelInstanceSubsystem->GetOwningLevelInstance(InLevel))
	{
		check(LevelInstance);

		if (LevelInstanceInterface == LevelInstance)
		{
			LevelInstance->GetLoadedLevel()->OnLoadedActorAddedToLevelEvent.AddUObject(this, &ULevelInstanceAdapter::OnLoadedActorAddedToLevel);
			FixupModifiersInLevelInstance();
		}
	}
}

void ULevelInstanceAdapter::OnLoadedActorAddedToLevel(AActor& InActor)
{
	TInlineComponentArray<MeshPartition::UModifierComponent*> Modifiers(&InActor);
	for (MeshPartition::UModifierComponent* Modifier : Modifiers)
	{
		FixupModifierInstance(Modifier);
	}

	if (UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent())
	{
		EditorComponent->OnModifierAssigned();
	}
}

void ULevelInstanceAdapter::FixupModifiersInLevelInstance()
{
	ILevelInstanceInterface* LevelInstance = GetOwningLevelInstance();
	if (LevelInstance == nullptr)
	{
		return;
	}

	OwnedModifiers.Reset();

	const ULevelInstanceSubsystem* LevelInstanceSubsystem = GetWorld()->GetSubsystem<ULevelInstanceSubsystem>();
	check(LevelInstanceSubsystem);
	check(LevelInstanceSubsystem->IsLoaded(LevelInstance));

	ForEachModifierInLevelInstance([this](MeshPartition::UModifierComponent* Modifier)
		{
			FixupModifierInstance(Modifier);
		});

	if (UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent())
	{
		EditorComponent->OnModifierAssigned();
	}
}

void ULevelInstanceAdapter::FixupModifierInstance(MeshPartition::UModifierComponent* InModifier)
{
	InModifier->SetAffectedMeshPartition(GetAffectedMeshPartition());
	OwnedModifiers.Emplace(InModifier);
}

void ULevelInstanceAdapter::OnRegister()
{
	Super::OnRegister();

	ILevelInstanceInterface* LevelInstance = GetOwningLevelInstance();
	if (LevelInstance == nullptr)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "MegaMeshLevelInstanceAdapter is not attached to a LevelInstance actor (%ls)!", GetOwner() ? *GetOwner()->GetActorNameOrLabel() : TEXT("null"));
		return;
	}
}

void ULevelInstanceAdapter::OnComponentDestroyed(bool bDestroyingHierarchy)
{
	if (ILevelInstanceInterface* LevelInstance = GetOwningLevelInstance())
	{
		if (ULevel* LoadedLevel = LevelInstance->GetLoadedLevel())
		{
			LoadedLevel->OnLoadedActorAddedToLevelEvent.RemoveAll(this);
		}
	}

	// Clear AffectedMeshPartition on every modifier we hooked into the parent MeshPartition,
	// then force a synchronous UpdateModifierList so those modifiers are removed from
	// the editor component's CurrentModifiers list before they (and the LI temp world that contains them)
	// become unreachable.
	for (MeshPartition::UModifierComponent* Modifier : OwnedModifiers)
	{
		if (IsValid(Modifier))
		{
			Modifier->SetAffectedMeshPartition(nullptr);
		}
	}
	OwnedModifiers.Reset();

	if (UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent())
	{
		EditorComponent->OnModifierAssigned();
		EditorComponent->UpdateModifierList();
	}

	Super::OnComponentDestroyed(bDestroyingHierarchy);
}

void ULevelInstanceAdapter::PreEditChange(FProperty* InPropertyAboutToChange)
{
	if (InPropertyAboutToChange == nullptr)
	{
		return;
	}
	
	const FName PropertyName = InPropertyAboutToChange->GetFName();
	UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent();
	
	// Before assigning a new AffectedMegaMesh, fixup all modifiers in the level instance so they are now pointing to a null MegaMesh
	// There is a quirk with how new affected mega mesh properties are assigned where in PreEditChange the affectedmegamesh pointer is set to null,
	// an update is triggered on the old affected mega mesh to unregister this component, then a new update happens in PostEditChange to register to the new one.
	
	// the following block mimics what is done in MeshPartition::UModifierComponent
	if (IsAffectedMeshPartitionPropertyName(PropertyName) && (EditorComponent != nullptr))
	{
		AMeshPartition* MegaMesh = GetAffectedMeshPartition();

		SetAffectedMeshPartition(nullptr);
		FixupModifiersInLevelInstance();
		SetAffectedMeshPartition(MegaMesh);
	}

	// derived class implementation handles triggering the OnModifierAssigned function so it's important this happens after the above fixup:
	Super::PreEditChange(InPropertyAboutToChange);
}

void ULevelInstanceAdapter::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	const FName MemberPropertyName = InPropertyChangedEvent.MemberProperty ? InPropertyChangedEvent.MemberProperty->GetFName() : NAME_None;
	UMeshPartitionEditorComponent* EditorComponent = GetMeshPartitionEditorComponent();
	
	if (IsAffectedMeshPartitionPropertyName(MemberPropertyName) && (EditorComponent != nullptr))
	{
		FixupModifiersInLevelInstance();
	}

	// derived class implementation handles triggering the OnModifierAssigned function so it's important this happens after the above fixup:
	Super::PostEditChangeProperty(InPropertyChangedEvent);
}
} // namespace UE::MeshPartition