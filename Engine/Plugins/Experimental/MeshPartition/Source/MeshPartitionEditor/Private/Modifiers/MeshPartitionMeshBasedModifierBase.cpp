// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionMeshBasedModifierBase.h"
#include "MeshPartitionModifierUtils.h"

#include "Components/DynamicMeshComponent.h"
#include "IndexedMeshToDynamicMesh.h"
#include "MeshPartitionMeshView.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "MeshDescriptionToDynamicMesh.h"
#include "TextureResource.h"
#include "ToolTargets/StaticMeshToolTarget.h"
#include "PrimitiveDrawingUtils.h"


namespace UE::MeshPartition
{
UMeshBasedModifierBase::UMeshBasedModifierBase()
{
}

void UMeshBasedModifierBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const FName ParentPreviousDefaultType = TEXT("Misc");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("Mesh");
		SetType(PreviousModifierDefaultType);
	}
}

void UMeshBasedModifierBase::InitializeModifier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshBasedModifierBase::InitializeModifier);

	Super::InitializeModifier();

	UpdateMeshInstance();
}

void UMeshBasedModifierBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.Property)
	{
		if (PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, MeshSourceMode)
			|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, StaticMesh)
			|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, MeshComponent)
			|| PropertyChangedEvent.Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, DesiredLOD))
		{
			UpdateMeshInstance();
			OnChanged(ComputeBounds(), EChangeType::StateChange);
		}
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMeshBasedModifierBase::PostEditUndo()
{
	Super::PostEditUndo();

	// TODO: We want to avoid having to do this for simple translations, but do need to do it for
	//  changes that affect the mesh.
	UpdateMeshInstance();
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

bool UMeshBasedModifierBase::CopyDynamicMeshComponent(const UDynamicMeshComponent* MeshComponent, 
	bool bWantAttributes, FDynamicMesh3& OutMesh)
{
	if (!IsValid(MeshComponent))
	{
		return false;
	}

	MeshComponent->ProcessMesh([&OutMesh, bWantAttributes](const FDynamicMesh3& Mesh)
	{
		OutMesh.Copy(Mesh,
			bWantAttributes, bWantAttributes,
			bWantAttributes, bWantAttributes);
	});

	return true;
}

bool UMeshBasedModifierBase::CopyStaticMesh(const UStaticMesh* StaticMesh, EMeshLODIdentifier DesiredLOD, 
	bool bWantAttributes, FDynamicMesh3& OutMesh)
{
	if (!StaticMesh)
	{
		return false;
	}

	// Get an actual lod to use from the desired one (based off of UStaticMeshReadOnlyToolTarget::GetValidEditingLOD)
	int32 LODIndexToUse = (int32)EMeshLODIdentifier::LOD0;
	if (DesiredLOD == EMeshLODIdentifier::MaxQuality || DesiredLOD == EMeshLODIdentifier::HiResSource)
	{
		LODIndexToUse = StaticMesh->IsHiResMeshDescriptionValid() ? (int32)EMeshLODIdentifier::HiResSource : (int32)EMeshLODIdentifier::LOD0;
	}
	else if (DesiredLOD == EMeshLODIdentifier::Default)
	{
		LODIndexToUse = (int32)EMeshLODIdentifier::LOD0;
	}
	else
	{
		LODIndexToUse = (int32)DesiredLOD;
		int32 MaxExistingLOD = StaticMesh->GetNumSourceModels() - 1;
		if (LODIndexToUse > MaxExistingLOD)
		{
			LODIndexToUse = MaxExistingLOD;
		}
	}

	if (!ensure(LODIndexToUse == (int32)EMeshLODIdentifier::HiResSource || StaticMesh->IsSourceModelValid(LODIndexToUse)))
	{
		return false;
	}

	if (LODIndexToUse != (int32)EMeshLODIdentifier::HiResSource
		&& !StaticMesh->GetSourceModel(LODIndexToUse).IsSourceModelInitialized())
	{
		// This means that this was an auto-generated LOD. Attempt to get a mesh out of the render data.

		if (!StaticMesh->HasValidRenderData())
		{
			return false;
		}
		const FStaticMeshRenderData* RenderData = StaticMesh->GetRenderData();
		if (!RenderData || !RenderData->LODResources.IsValidIndex(LODIndexToUse))
		{
			return false;
		}

		const FStaticMeshLODResources& LODResource = RenderData->LODResources[LODIndexToUse];
		if (!UE::Conversion::RenderBuffersToDynamicMesh(LODResource.VertexBuffers,
			LODResource.IndexBuffer, LODResource.Sections, OutMesh, 
			/*bAttemptToWeldSeams =*/ true))
		{
			return false;
		}
	}
	else // if we have source data
	{
		UStaticMeshReadOnlyToolTarget::FMeshDescriptionCache UnusedCache;
		UStaticMeshReadOnlyToolTarget::FGetMeshDescriptionOptions Options
		{
			.bAllowRecomputeNormals = bWantAttributes,
			.bAllowRecomputeTangents = bWantAttributes
		};
		const FMeshDescription* MeshDescription = UStaticMeshReadOnlyToolTarget::GetMeshDescriptionWithScaleApplied(
			StaticMesh, LODIndexToUse, UnusedCache);

		if (!MeshDescription)
		{
			return false;
		}

		FMeshDescriptionToDynamicMesh Converter;
		Converter.bDisableAttributes = !bWantAttributes;
		Converter.Convert(MeshDescription, OutMesh);
	}

	return true;
}

void UMeshBasedModifierBase::BP_SetStaticMesh(UStaticMesh* Mesh)
{
	Modify();
	StaticMesh = Mesh;
	MeshSourceMode = EModifierMeshSourceMode::StaticMesh;

	FPropertyChangedEvent ChangedEvent(UMeshBasedModifierBase::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, StaticMesh)));
	PostEditChangeProperty(ChangedEvent);
}

void UMeshBasedModifierBase::OnDynamicMeshChanged(UDynamicMeshComponent*)
{
	UpdateMeshInstance();
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

EMeshLODIdentifier UMeshBasedModifierBase::GetDesiredLOD() const
{
	switch (DesiredLOD)
	{
	case -1:
		return EMeshLODIdentifier::HiResSource;
	case 0:
		return EMeshLODIdentifier::LOD0;
	case 1:
		return EMeshLODIdentifier::LOD1;
	case 2:
		return EMeshLODIdentifier::LOD2;
	case 3:
		return EMeshLODIdentifier::LOD3;
	case 4:
		return EMeshLODIdentifier::LOD4;
	case 5:
		return EMeshLODIdentifier::LOD5;
	case 6:
		return EMeshLODIdentifier::LOD6;
	case 7:
		return EMeshLODIdentifier::LOD7;
	default:
		ensure(false);
	}
	return EMeshLODIdentifier::Default;
}

// TODO: Uncertain whether this can be updated from GetModifierInstanceDescs or ApplyModifications, or
//  whether this would have to be game thread only...
void UMeshBasedModifierBase::UpdateMeshInstance()
{
	using namespace Geometry;
	
	if (MeshInstance)
	{
		MeshInstance.Reset();
	}

	// Don't recreate the instance data if it exists, so that subclasses can use their own subclass if they want
	FDynamicMesh3 MeshData;

	if (UDynamicMeshComponent* OldDynamicMeshComponent = UpdatingDynamicMeshComponent.Get())
	{
		OldDynamicMeshComponent->OnMeshChanged.RemoveAll(this);
	}

	if (MeshSourceMode == MeshPartition::EModifierMeshSourceMode::DynamicMeshComponent)
	{
		UDynamicMeshComponent* ResolvedComponent = nullptr;
		if (MeshComponent.PathToComponent.Len() != 0 || MeshComponent.ComponentProperty != NAME_None || !MeshComponent.OverrideComponent.IsExplicitlyNull())
		{
			ResolvedComponent = Cast<UDynamicMeshComponent>(MeshComponent.GetComponent(GetOwner()));
		}
		if (CopyDynamicMeshComponent(ResolvedComponent, bKeepInternalMeshAttributes, MeshData) && ResolvedComponent)
		{
			ProcessMeshInstance(MeshData);
			UpdatingDynamicMeshComponent = ResolvedComponent;
			ResolvedComponent->OnMeshChanged.AddUObject(this, &UMeshBasedModifierBase::OnDynamicMeshChanged, ResolvedComponent);
		}
		
	}
	else
	{
		if (CopyStaticMesh(StaticMesh.Get(), GetDesiredLOD(), bKeepInternalMeshAttributes, MeshData))
		{
			ProcessMeshInstance(MeshData);
		}
	}

	MeshInstance = MakeShared<FAsyncMeshInstanceData>(MoveTemp(MeshData));
	PostUpdateMeshInstance(MeshInstance->GetMesh());
}

bool UMeshBasedModifierBase::SetMeshComponent(UDynamicMeshComponent* DynamicMeshComponent)
{
	if (!DynamicMeshComponent)
	{
		return false;
	}
	MeshSourceMode = MeshPartition::EModifierMeshSourceMode::DynamicMeshComponent;

	FComponentReference ComponentReference;
	AActor* ComponentOwner = DynamicMeshComponent->GetOwner();
	ComponentReference.OtherActor = GetOwner() == ComponentOwner ? nullptr : ComponentOwner;
	ComponentReference.PathToComponent = DynamicMeshComponent->GetPathName(ComponentOwner);
	ComponentReference.OverrideComponent = DynamicMeshComponent;
	MeshComponent = ComponentReference;
	UpdateMeshInstance();
	OnChanged(ComputeBounds(), EChangeType::StateChange);
	return true;
}

void UMeshBasedModifierBase::SetMeshSourceMode(const MeshPartition::EModifierMeshSourceMode InModifierMeshSourceMode)
{
	if (InModifierMeshSourceMode != MeshSourceMode)
	{
		Modify();
		MeshSourceMode = InModifierMeshSourceMode;
		FPropertyChangedEvent ChangedEvent(UMeshBasedModifierBase::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, MeshSourceMode)));
		PostEditChangeProperty(ChangedEvent);
	}
}

MeshPartition::EModifierMeshSourceMode UMeshBasedModifierBase::GetMeshSourceMode() const
{
	return MeshSourceMode;
}

void UMeshBasedModifierBase::SetStaticMesh(UStaticMesh* InStaticMesh)
{
	if (!InStaticMesh)
	{
		Modify();
		UE_LOGF(LogMegaMeshEditor, Warning, "SetStaticMesh: StaticMesh is null, clearing existing mesh");
		StaticMesh = nullptr;

		FPropertyChangedEvent ChangedEvent(UMeshBasedModifierBase::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, StaticMesh)));
		PostEditChangeProperty(ChangedEvent);
		return;
	}

	if (MeshSourceMode == EModifierMeshSourceMode::StaticMesh)
	{
		Modify();
		StaticMesh = InStaticMesh;
		FPropertyChangedEvent ChangedEvent(UMeshBasedModifierBase::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UMeshBasedModifierBase, StaticMesh)));
		PostEditChangeProperty(ChangedEvent);
	}
	else
	{
		UE_LOGF(LogMegaMeshEditor, Error, "SetStaticMesh: MeshSourceMode needs to be set to StaticMesh");
	}
}

UStaticMesh* UMeshBasedModifierBase::GetStaticMesh() const
{
	return StaticMesh;
}

void UMeshBasedModifierBase::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshBasedModifierBase::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	if (!MeshInstance)
	{
		return;
	}

	// include hash of mesh contents
	Dependencies += MeshInstance->GetHash();

	switch (MeshSourceMode)
	{
		case MeshPartition::EModifierMeshSourceMode::DynamicMeshComponent:
			{
				// dynamic mesh can come from another package, add the dependency
				if (MeshComponent.PathToComponent.Len() != 0 || MeshComponent.ComponentProperty != NAME_None || !MeshComponent.OverrideComponent.IsExplicitlyNull())
				{
					UDynamicMeshComponent* ResolvedComponent = Cast<UDynamicMeshComponent>(MeshComponent.GetComponent(GetOwner()));
					Dependencies.AddPackageDependency(ResolvedComponent);
				}
			}
			break;
		case MeshPartition::EModifierMeshSourceMode::StaticMesh:
			// static mesh reference can come from another package, add the dependency
			Dependencies.AddPackageDependency(StaticMesh.Get());
			break;
		default:
			check(false);
			break;
	}
}
} // namespace UE::MeshPartition