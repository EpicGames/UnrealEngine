// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionMeshProjectModifier.h"

#include "Curves/CurveFloat.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "PrimitiveDrawingUtils.h" // DrawWireBox

namespace UE::MeshPartition
{
namespace MegaMeshMeshProjectModifierLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		FBox GlobalBounds;
		TSharedPtr<const FAsyncMeshInstanceData> MeshInstance;
		FTransform MeshToWorld;
		FTransform ProjectionTransform;
		Geometry::FAxisAlignedBox3d ProjectionSpaceBounds;

		MeshPartition::UMeshProjectModifier::FProjectOntoMeshOptionalParams OptionalParams;

		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;
		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("63cf0199-4a7d-5e41-fdf7-eb965e6a9b04"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};
}

UMeshProjectModifier::UMeshProjectModifier()
{
	bKeepInternalMeshAttributes = true;
}

void UMeshProjectModifier::UpdateFromMesh()
{
	UpdateMeshInstance();

	// Trigger megamesh update
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

TArray<FBox> UMeshProjectModifier::ComputeBounds() const
{
	if (!MeshInstance)
	{
		return {};
	}

	FTransform ProjectionTransform;
	Geometry::FAxisAlignedBox3d ProjectionSpaceBounds;
	GetProjectionTransformAndBounds(ProjectionTransform, ProjectionSpaceBounds);

	Geometry::FAxisAlignedBox3d GlobalBounds(ProjectionSpaceBounds, ProjectionTransform);

	return { FBox(GlobalBounds) };
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UMeshProjectModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshMeshProjectModifierLocals;

	if (!MeshInstance)
	{
		return nullptr;
	}

	TSharedPtr<FBackgroundOp> Op = AllocateBackgroundOp<FBackgroundOp>(GetFName());
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->MeshInstance = MeshInstance;
	Op->MeshToWorld = GetComponentToWorld();
	GetProjectionTransformAndBounds(Op->ProjectionTransform, Op->ProjectionSpaceBounds);
	
	Op->OptionalParams.BlendMode = BlendMode;
	Op->OptionalParams.BlendSoftnessInProjectionSpace = BlendSoftness;

	Op->OptionalParams.FalloffSettings.Initialize(HeightFalloff);

	for (const FProjectModifierWeightEntry& ChannelEntry : WeightChannels)
	{
		if (ChannelEntry.ChannelName.IsNone())
		{
			continue;
		}

		UMeshProjectModifier::FWeightEntryParams& OpEntry = Op->OptionalParams.WeightEntries.Emplace_GetRef();
		OpEntry.Initialize(ChannelEntry);
	}

	return Op;
}

void MegaMeshMeshProjectModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);

	// Add the used weight layer names
	if (!OutInstanceInfos.IsEmpty())
	{
		for (const UMeshProjectModifier::FWeightEntryParams& WeightChannelEntry : OptionalParams.WeightEntries)
		{
			if (!WeightChannelEntry.WeightChannelName.IsNone())
			{
				OutInstanceInfos[0].UsedChannels.Emplace(WeightChannelEntry.WeightChannelName);

				OutInstanceInfos[0].ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				OutInstanceInfos[0].WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
			}
		}
	}
}

void MegaMeshMeshProjectModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView,
	const FTransform3d& MegameshTransform, const FInstanceInfo& InInstanceDesc) const
{
	using namespace Geometry;

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UMeshProjectModifier::ApplyModifications);

	if (!MeshInstance)
	{
		return;
	}

	UMeshProjectModifier::ProjectOntoMesh(InMeshView, MegameshTransform,
		MeshInstance->GetMesh(), MeshToWorld, MeshInstance->GetAABBTree(),
		ProjectionTransform, ProjectionSpaceBounds, OptionalParams);
}

void UMeshProjectModifier::ProjectOntoMesh(MeshPartition::FMeshView& MeshView,
	const FTransform3d& MegameshTransform,
	const FDynamicMesh3& ProjectionMesh, const FTransform& ProjectionMeshToWorld,
	const Geometry::FDynamicMeshAABBTree3& ProjectionMeshSpatial,
	const FTransform& ProjectionTransform, const Geometry::FAxisAlignedBox3d& ProjectionSpaceBounds,
	const FProjectOntoMeshOptionalParams& OptionalParams)
{
	FVector3d WorldProjectionDirection = -ProjectionTransform.GetUnitAxis(EAxis::Z);
	// The above doesn't account for potential negative Z scaling, in which case we want to raycast the other direction.
	if (ProjectionTransform.GetScale3D().Z < 0)
	{
		WorldProjectionDirection *= -1;
	}
	
	FVector3d MeshLocalProjectionDirection = ProjectionMeshToWorld.InverseTransformVector(WorldProjectionDirection);
	MeshLocalProjectionDirection.Normalize();
	
	// Prep values needed for calculating falloff
	Utils::FRectangleFalloffData HeightFalloffData(
		FVector2d(ProjectionSpaceBounds.Extents()), 
		OptionalParams.FalloffSettings.CornerRadius, 
		OptionalParams.FalloffSettings.Distance,
		OptionalParams.FalloffSettings.Mode, 
		OptionalParams.FalloffSettings.FalloffCurve);
	TArray<TOptional<Utils::FRectangleFalloffData>> WeightFalloffData; // 1:1 with WeightEntries
	WeightFalloffData.SetNum(OptionalParams.WeightEntries.Num());
	for (int EntryIndex = 0; EntryIndex < OptionalParams.WeightEntries.Num(); ++EntryIndex)
	{
		const FWeightEntryParams& Entry = OptionalParams.WeightEntries[EntryIndex];
		if (Entry.FalloffOverrides.IsSet())
		{
			WeightFalloffData[EntryIndex] = Utils::FRectangleFalloffData(
				FVector2d(ProjectionSpaceBounds.Extents()),
				Entry.FalloffOverrides->CornerRadius,
				Entry.FalloffOverrides->Distance,
				Entry.FalloffOverrides->Mode,
				Entry.FalloffOverrides->FalloffCurve);
		}
	}

	// Prep ability to sample weights
	const Geometry::FDynamicMeshAttributeSet* Attributes = ProjectionMesh.Attributes();
	const Geometry::FDynamicMeshColorOverlay* ColorOverlay = Attributes && Attributes->HasPrimaryColors() ?
		Attributes->PrimaryColors() : nullptr;
	TArray<TFunction<float(int32 Tid, const FVector3f& BaryCoords)>> WeightSamplers; // 1:1 with WeightEntries
	WeightSamplers.SetNum(OptionalParams.WeightEntries.Num());

	if (Attributes)
	{
		for (int EntryIndex = 0; EntryIndex < OptionalParams.WeightEntries.Num(); ++EntryIndex)
		{
			const FWeightEntryParams& Entry = OptionalParams.WeightEntries[EntryIndex];
			if (Entry.SourceMode == MeshPartition::EProjectModifierChannelSourceMode::VertexColor)
			{
				if (!ColorOverlay || !ensure(Entry.VertexColorIndex >= 0 && Entry.VertexColorIndex < 4))
				{
					continue;
				}
				WeightSamplers[EntryIndex] = [ColorOverlay, VertexColorIndex = Entry.VertexColorIndex](int32 Tid, const FVector3f& BaryCoords)
				{
					FVector4f Color;
					ColorOverlay->GetTriBaryInterpolate(Tid, &BaryCoords.X, &Color.X);
					return Color[VertexColorIndex];
				};
			}
			else // if SourceMode is VertexWeight
			{
				if (Entry.SourceWeightChannelName.IsNone())
				{
					continue;
				}

				// Find the weight channel
				for (int LayerIndex = 0; LayerIndex < Attributes->NumWeightLayers(); ++LayerIndex)
				{
					if (Attributes->GetWeightLayer(LayerIndex)->GetName() == Entry.SourceWeightChannelName)
					{
						WeightSamplers[EntryIndex] = [&ProjectionMesh, WeightOverlay = Attributes->GetWeightLayer(LayerIndex)](int32 Tid, const FVector3f& BaryCoords)
						{
							Geometry::FIndex3i Tri = ProjectionMesh.GetTriangle(Tid);
							FVector3f TriWeights;
							WeightOverlay->GetValue(Tri[0], &TriWeights[0]);
							WeightOverlay->GetValue(Tri[1], &TriWeights[1]);
							WeightOverlay->GetValue(Tri[2], &TriWeights[2]);
							return BaryCoords.Dot(TriWeights);
						};
						break;
					}
				}//end looking for weight channel
			}
		}//end initializing sample functions
	}//end if have attributes
	

	for (int VertexIndex = 0; VertexIndex < MeshView.VertexCount(); ++VertexIndex)
	{
		FVector WorldOriginalVertPosition = MegameshTransform.TransformPosition(MeshView.GetVertexPos(VertexIndex));
		FVector ProjectionOriginalVertPosition = ProjectionTransform.InverseTransformPosition(WorldOriginalVertPosition);

		if (!ProjectionSpaceBounds.Contains(ProjectionOriginalVertPosition))
		{
			continue;
		}

		FVector3d ProjectionSpaceRayOrigin = ProjectionOriginalVertPosition;
		ProjectionSpaceRayOrigin.Z = ProjectionSpaceBounds.Max.Z;

		FVector3d MeshSpaceRayOrigin = ProjectionMeshToWorld.InverseTransformPosition(
			ProjectionTransform.TransformPosition(ProjectionSpaceRayOrigin));

		FRay3d MeshLocalRay(
			MeshSpaceRayOrigin,
			MeshLocalProjectionDirection);

		double LocalHitT = TNumericLimits<double>::Max();
		int32 HitTid = IndexConstants::InvalidID;
		FVector3d HitBaryCoords;
		if (!ProjectionMeshSpatial.FindNearestHitTriangle(MeshLocalRay, LocalHitT, HitTid, HitBaryCoords))
		{
			continue;
		}
		FVector3f HitBaryCoordsFloat(HitBaryCoords);

		FVector3d MeshHitLocation = MeshLocalRay.PointAt(LocalHitT);
		FVector3d WorldHitLocation = ProjectionMeshToWorld.TransformPosition(MeshHitLocation);
		FVector3d ProjectionSpaceHitLocation = ProjectionTransform.InverseTransformPosition(WorldHitLocation);

		// We already know it's within the max z because that's where we raycast from
		bool bStillWithinProjectionBounds = ProjectionSpaceHitLocation.Z >= ProjectionSpaceBounds.Min.Z;
		if (!bStillWithinProjectionBounds)
		{
			continue;
		}

		// Used for weight bApplyHeightMinMaxBlend. More positive value means that the value should be more masked out.
		double MinMaxHeightDelta = 0;
		// Potentially used for weights as well
		double HeightFalloffAlpha = 0;

		// Do the height writing. This is wrapped in a lambda just so that it's easy to skip without skipping the weight
		//  application further below.
		[&OptionalParams, &MinMaxHeightDelta, &ProjectionSpaceHitLocation, &ProjectionOriginalVertPosition, 
			&HeightFalloffAlpha, &HeightFalloffData, &ProjectionSpaceBounds, &ProjectionSpaceRayOrigin, 
			&ProjectionTransform, &MeshView, &MegameshTransform, &WorldHitLocation, VertexIndex]()
		{
			FVector3d ProjectionSpaceDestination = ProjectionSpaceHitLocation;
			bool bAdjustedDestination = false;
			
			if (OptionalParams.BlendMode != MeshPartition::EProjectModifierBlendMode::Set)
			{
				if (OptionalParams.BlendSoftnessInProjectionSpace <= 0)
				{
					bool bWouldRaise = ProjectionSpaceHitLocation.Z > ProjectionOriginalVertPosition.Z;
					if (bWouldRaise != (OptionalParams.BlendMode == MeshPartition::EProjectModifierBlendMode::Raise))
					{
						// skip setting height for this vert
						return;
					}
				}
				else
				{
					// Apply BlendSoftness
					if (OptionalParams.BlendMode == MeshPartition::EProjectModifierBlendMode::Raise)
					{
						MinMaxHeightDelta = ProjectionOriginalVertPosition.Z - ProjectionSpaceHitLocation.Z;
						ProjectionSpaceDestination.Z = FMathd::SmoothMax(
							ProjectionOriginalVertPosition.Z, 
							ProjectionSpaceHitLocation.Z, 
							OptionalParams.BlendSoftnessInProjectionSpace);
					}
					else // MeshPartition::EProjectModifierBlendMode::Lower
					{
						MinMaxHeightDelta = ProjectionSpaceHitLocation.Z - ProjectionOriginalVertPosition.Z;
						ProjectionSpaceDestination.Z = FMathd::SmoothMin(
							ProjectionOriginalVertPosition.Z, 
							ProjectionSpaceHitLocation.Z, 
							OptionalParams.BlendSoftnessInProjectionSpace);
					}

					ProjectionSpaceDestination.Z = FMath::Clamp(ProjectionSpaceDestination.Z, ProjectionSpaceBounds.Min.Z, ProjectionSpaceBounds.Max.Z);
					bAdjustedDestination = true;
				}
			}

			HeightFalloffAlpha = Utils::GetRectangleFalloffAlpha(FVector2d(ProjectionSpaceRayOrigin), HeightFalloffData);
			if (HeightFalloffAlpha <= UE_DOUBLE_SMALL_NUMBER)
			{
				// No need to write if alpha is zero
				return;
			}
		
			if (HeightFalloffAlpha < 1)
			{
				ProjectionSpaceDestination.Z = FMath::Lerp(ProjectionOriginalVertPosition.Z, ProjectionSpaceDestination.Z, HeightFalloffAlpha);
				bAdjustedDestination = true;
			}

			// Write our value back
			if (bAdjustedDestination)
			{
				FVector3d WorldDestination = ProjectionTransform.TransformPosition(ProjectionSpaceDestination);
				MeshView.SetVertexPos(VertexIndex, MegameshTransform.InverseTransformPosition(WorldDestination));
			}
			else
			{
				// Since it wasn't adjusted, can use the known world coordinate of the hit 
				MeshView.SetVertexPos(VertexIndex, MegameshTransform.InverseTransformPosition(WorldHitLocation));
			}
		}();

		// Do the weight writing
		for (int WeightEntryIndex = 0; WeightEntryIndex < OptionalParams.WeightEntries.Num(); ++WeightEntryIndex)
		{
			const FWeightEntryParams& Entry = OptionalParams.WeightEntries[WeightEntryIndex];
			if (!WeightSamplers[WeightEntryIndex])
			{
				// Didn't find this attribute
				continue;
			}

			double WeightAlpha = HeightFalloffAlpha;
			if (WeightFalloffData[WeightEntryIndex].IsSet())
			{
				// Using custom falloff for weights, not the height one
				WeightAlpha = Utils::GetRectangleFalloffAlpha(FVector2d(ProjectionSpaceRayOrigin), *WeightFalloffData[WeightEntryIndex]);
			}
			if (Entry.HeightMinMaxBlendDistance.IsSet() && MinMaxHeightDelta > -*Entry.HeightMinMaxBlendDistance)
			{
				WeightAlpha *= (Entry.HeightMinMaxBlendDistance == 0.0) ? 0.0 // MinMaxHeightDelta known to be positive if HeightMinMaxBlendDistance was 0, so mask out
					// Already clamped to be lower than 1 by the if statement above
					: FMath::Max(0.5 - MinMaxHeightDelta / (2.0 * (*Entry.HeightMinMaxBlendDistance)), 0.0);
			}

			if (WeightAlpha >= UE_DOUBLE_SMALL_NUMBER)
			{
				const double CurrentValue = MeshView.GetVertexAttributeWeight(Entry.WeightChannelName, VertexIndex);
				float DesiredValue = WeightSamplers[WeightEntryIndex](HitTid, HitBaryCoordsFloat);

				if (Entry.BlendMode != MeshPartition::EProjectModifierBlendMode::Set)
				{
					bool bWouldRaise = DesiredValue > CurrentValue;
					if (bWouldRaise != (Entry.BlendMode == MeshPartition::EProjectModifierBlendMode::Raise))
					{
						continue;
					}
				}

				MeshView.SetVertexAttributeWeight(Entry.WeightChannelName, VertexIndex, FMath::Lerp(CurrentValue, DesiredValue, WeightAlpha));
			}
		}
	}
}

void UMeshProjectModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	const FColor RectangleColor = FColor::Red;
	const FColor LocalBoundsColor = FColor::Yellow;
	const FColor GlobalBoundsColor = FColor::Orange;
	constexpr float RectangleThickness = 3;
	constexpr float BoundsThickness = 1;
	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	if (!MeshInstance)
	{
		return;
	}

	if (bDrawLocalBounds || bDrawProjectionRectangle)
	{
		FTransform ProjectionTransform;
		Geometry::FAxisAlignedBox3d ProjectionSpaceBounds;
		GetProjectionTransformAndBounds(ProjectionTransform, ProjectionSpaceBounds);

		if (bDrawLocalBounds)
		{
			DrawWireBox(PDI, ProjectionTransform.ToMatrixWithScale(), FBox(ProjectionSpaceBounds),
				LocalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
		}
		if (bDrawProjectionRectangle)
		{
			DrawRectangle(PDI, ProjectionTransform.GetTranslation(), ProjectionTransform.GetUnitAxis(EAxis::X), ProjectionTransform.GetUnitAxis(EAxis::Y),
				RectangleColor, ProjectionSpaceBounds.Dimension(0), ProjectionSpaceBounds.Dimension(1), SDPG_Foreground,
				RectangleThickness, DepthBias, bScreenSpace);
		}
	}
}

void UMeshProjectModifier::GetProjectionTransformAndBounds(FTransform& ProjectionTransformOut, 
	Geometry::FAxisAlignedBox3d& ProjectionSpaceBoundsOut) const
{
	if (!ensure(MeshInstance))
	{
		return;
	}

	const FTransform& MeshToWorld = GetComponentToWorld();
	ProjectionTransformOut = FTransform(bUseRelativeProjectionDirection 
		? GetComponentToWorld().GetRotation() * RelativeProjectionDirection.Quaternion() 
		: AbsoluteProjectionDirection.Quaternion(),
		MeshToWorld.GetLocation());

	Geometry::FAxisAlignedBox2d Mesh2DBoundsInProjection;
	Geometry::FAxisAlignedBox3d MeshBounds = MeshInstance->GetBounds();
	for (int i = 0; i < 8; ++i)
	{
		FVector3d CornerWorldPosition = MeshToWorld.TransformPosition(MeshBounds.GetCorner(i));
		FVector3d CornerProjectionPosition = ProjectionTransformOut.InverseTransformPosition(CornerWorldPosition);
		Mesh2DBoundsInProjection.Contain(FVector2d(CornerProjectionPosition.X, CornerProjectionPosition.Y));
	}
	ProjectionSpaceBoundsOut = Geometry::FAxisAlignedBox3d(
		FVector3d(Mesh2DBoundsInProjection.Min, -VerticalExtentDown),
		FVector3d(Mesh2DBoundsInProjection.Max, VerticalExtentUp));
}

void UMeshProjectModifier::InitializeModifier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshProjectModifier::InitializeModifier);

	Super::InitializeModifier();
	AttachCurveListeners();
}

void UMeshProjectModifier::UninitializeModifier()
{
	Super::UninitializeModifier();
	DetachCurveListeners();
}

void UMeshProjectModifier::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange.GetTail())
	{
		return;
	}

	FProperty* Property = PropertyAboutToChange.GetTail()->GetValue();
	if (!Property)
	{
		return;
	}

	// Handle the actual curve property, or addition/removal of the containing structs
	if (Property->GetName() == GET_MEMBER_NAME_CHECKED(MeshPartition::FProjectModifierFalloffSettings, FalloffCurve)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshProjectModifier, WeightChannels)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(FProjectModifierWeightEntry, FalloffOverrides))
	{
		DetachCurveListeners();
	}
}

void UMeshProjectModifier::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	ON_SCOPE_EXIT{ Super::PostEditChangeProperty(PropertyChangedEvent); };

	if (!PropertyChangedEvent.PropertyChain.GetTail())
	{
		return;
	}
	FProperty* Property = PropertyChangedEvent.PropertyChain.GetTail()->GetValue();
	if (!Property)
	{
		return;
	}
	UObject* OwnerObject = Property->GetOwnerUObject();

	if (Property->GetName() == GET_MEMBER_NAME_CHECKED(MeshPartition::FProjectModifierFalloffSettings, FalloffCurve)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UMeshProjectModifier, WeightChannels)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(FProjectModifierWeightEntry, FalloffOverrides))
	{
		AttachCurveListeners();
	}

	if (Property->GetName() == GET_MEMBER_NAME_CHECKED(UMeshProjectModifier, bUseRelativeProjectionDirection))
	{
		// When switching how we set the projection direction, make sure that the new direction agrees with the previous
		if (bUseRelativeProjectionDirection)
		{
			RelativeProjectionDirection = GetComponentToWorld().InverseTransformRotation(AbsoluteProjectionDirection.Quaternion()).Rotator();
		}
		else
		{
			AbsoluteProjectionDirection = GetComponentToWorld().TransformRotation(RelativeProjectionDirection.Quaternion()).Rotator();
		}
	}
}

void UMeshProjectModifier::OnCurveChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType)
{
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void UMeshProjectModifier::AttachCurveListeners()
{
	auto AttachListeners = [this](MeshPartition::FProjectModifierFalloffSettings& Settings)
	{
		if (Settings.FalloffCurve && !Settings.FalloffCurveListenerHandle.IsValid())
		{
			Settings.FalloffCurveListenerHandle = Settings.FalloffCurve->OnUpdateCurve.AddUObject(this, &UMeshProjectModifier::OnCurveChanged);
		}
	};

	AttachListeners(HeightFalloff);

	for (FProjectModifierWeightEntry& WeightChannelEntry : WeightChannels)
	{
		if (WeightChannelEntry.FalloffOverrides.IsSet())
		{
			AttachListeners(*WeightChannelEntry.FalloffOverrides);
		}
	}
}

void UMeshProjectModifier::DetachCurveListeners()
{
	auto DetachListeners = [](MeshPartition::FProjectModifierFalloffSettings& Settings)
	{
		if (Settings.FalloffCurve && Settings.FalloffCurveListenerHandle.IsValid())
		{
			Settings.FalloffCurve->OnUpdateCurve.Remove(Settings.FalloffCurveListenerHandle);
			Settings.FalloffCurveListenerHandle.Reset();
		}
	};

	DetachListeners(HeightFalloff);

	for (FProjectModifierWeightEntry& WeightChannelEntry : WeightChannels)
	{
		if (WeightChannelEntry.FalloffOverrides.IsSet())
		{
			DetachListeners(*WeightChannelEntry.FalloffOverrides);
		}
	}
}


void UMeshProjectModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshProjectModifier::GatherDependencies);

	// gather mesh dependencies from the base class
	Super::GatherDependencies(Dependencies);

	if (!MeshInstance)
	{
		return;
	}

	Dependencies += ComputeCombinedBounds();

	Dependencies += bUseRelativeProjectionDirection;
	if (bUseRelativeProjectionDirection)
	{
		Dependencies += RelativeProjectionDirection;
	}
	else
	{
		Dependencies += AbsoluteProjectionDirection;
	}
	Dependencies += VerticalExtentDown;
	Dependencies += VerticalExtentUp;
	Dependencies += BlendMode;
	if (BlendMode != MeshPartition::EProjectModifierBlendMode::Set)
	{
		Dependencies += BlendSoftness;
	}
	
	Dependencies += HeightFalloff.Mode;
	Dependencies += HeightFalloff.Distance;
	Dependencies += HeightFalloff.CornerRadius;
	
	switch (HeightFalloff.Mode)
	{
		case MeshPartition::EProjectModifierFalloffMode::Linear:
			break;
		case MeshPartition::EProjectModifierFalloffMode::Smooth:
			break;
		case MeshPartition::EProjectModifierFalloffMode::CustomCurve:
			Dependencies += HeightFalloff.FalloffCurve;
			break;
	}

	for (const FProjectModifierWeightEntry& Entry : WeightChannels)
	{
		if (Entry.ChannelName.IsNone())
		{
			continue;
		}
		Dependencies += Entry.ChannelName;
		Dependencies += Entry.SourceMode;
		if (Entry.SourceMode == MeshPartition::EProjectModifierChannelSourceMode::VertexColor)
		{
			Dependencies += Entry.VertexColorIndex;
		}
		else // MeshPartition::EProjectModifierChannelSourceMode::VertexWeight
		{
			Dependencies += Entry.SourceWeightChannelName;
		}
		Dependencies += Entry.BlendMode;
		if (Entry.FalloffOverrides.IsSet())
		{
			Dependencies += Entry.FalloffOverrides->Mode;
			Dependencies += Entry.FalloffOverrides->Distance;
			Dependencies += Entry.FalloffOverrides->CornerRadius;

			switch (Entry.FalloffOverrides->Mode)
			{
			case MeshPartition::EProjectModifierFalloffMode::Linear:
				break;
			case MeshPartition::EProjectModifierFalloffMode::Smooth:
				break;
			case MeshPartition::EProjectModifierFalloffMode::CustomCurve:
				Dependencies += Entry.FalloffOverrides->FalloffCurve;
				break;
			}
		}
		if (Entry.bApplyHeightMinMaxBlend)
		{
			Dependencies += Entry.HeightMinMaxBlendDistance;
		}
	}
}

FGuid UMeshProjectModifier::GetCodeVersionKey() const
{
	return MegaMeshMeshProjectModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

void UMeshProjectModifier::FFalloffParams::Initialize(const MeshPartition::FProjectModifierFalloffSettings& Settings)
{
	using namespace MegaMeshMeshProjectModifierLocals;

	Distance = Settings.Distance;
	CornerRadius = Settings.CornerRadius;

	switch (Settings.Mode)
	{
	case MeshPartition::EProjectModifierFalloffMode::Linear:
		Mode = Utils::ERectangleFalloffMode::Linear;
		break;
	case MeshPartition::EProjectModifierFalloffMode::Smooth:
		Mode = Utils::ERectangleFalloffMode::Smooth;
		break;
	case MeshPartition::EProjectModifierFalloffMode::CustomCurve:
		Mode = Utils::ERectangleFalloffMode::CustomCurve;
		break;
	default:
		ensure(false);
	}
	
	if (Mode == Utils::ERectangleFalloffMode::CustomCurve
		&& Settings.FalloffCurve)
	{
		FalloffCurve = MakeShared<FRichCurve>(Settings.FalloffCurve->FloatCurve);
	}
}

void UMeshProjectModifier::FWeightEntryParams::Initialize(const FProjectModifierWeightEntry& Entry)
{
	WeightChannelName = Entry.ChannelName;
	SourceMode = Entry.SourceMode;
	SourceWeightChannelName = Entry.SourceWeightChannelName;
	VertexColorIndex = Entry.VertexColorIndex;
	BlendMode = Entry.BlendMode;
	if (Entry.FalloffOverrides.IsSet())
	{
		FalloffOverrides.Emplace();
		FalloffOverrides->Initialize(*Entry.FalloffOverrides);
	}
	if (Entry.bApplyHeightMinMaxBlend)
	{
		HeightMinMaxBlendDistance = Entry.HeightMinMaxBlendDistance;
	}
}
} // namespace UE::MeshPartition