// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionMeshProvider.h"

#include "Components/DynamicMeshComponent.h"
#include "CoreGlobals.h" // GUndo
#include "Engine/Texture.h"
#include "Engine/Texture2D.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorComponent.h"
#include "Misc/ITransaction.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MegaMeshMeshProvider"

namespace UE::MeshPartition
{
namespace MegaMeshMeshProviderLocals
{
	class FMeshProviderOp : public IBaseMeshProviderOp
	{
	public:
		FMeshProviderOp(const FName& InOperationName) : IBaseMeshProviderOp(InOperationName) {}

		FTransform MeshTransform;
		TSharedPtr<const FDynamicMesh3> Mesh;

		virtual TSharedPtr<const FDynamicMesh3> GetMesh() const override
		{
			return Mesh;
		}
		virtual FTransform GetMeshTransform() const override
		{
			return MeshTransform;
		}
	};
}

UMeshProviderModifier::UMeshProviderModifier()
{
	MeshComponent = CreateDefaultSubobject<UDynamicMeshComponent>(TEXT("MeshComponent"));
	MeshComponent->bIsEditorOnly = true;
	// We don't allow mesh element selection on our internal component because it is usually hidden, except when we 
	//  start a tool on it. Allowing it would also cause our right click modifier finding to fail because we get a 
	//  geometry element menu instead.
	MeshComponent->SetAllowsGeometrySelection(false);

	MeshComponent->AttachToComponent(this, FAttachmentTransformRules::KeepRelativeTransform);
}

void UMeshProviderModifier::OnRegister()
{
	Super::OnRegister();

	MeshComponent->OnMeshChanged.RemoveAll(this);
	MeshComponent->OnMeshChanged.AddUObject(this, &UMeshProviderModifier::OnDynamicMeshChanged);

	MeshComponent->RegisterComponent();
}

void UMeshProviderModifier::PostLoad()
{
	Super::PostLoad();
	
	// Note: We only need to set RF_Transactional here to fix up maps
	// where these modifiers were spawned w/out RF_Transactional set
	// This should not happen for new mesh partitions, so it may be safe to delete this
	// after existing content has been loaded and re-saved once
	AtomicallySetFlags(EObjectFlags::RF_Transactional);
	MeshComponent->AtomicallySetFlags(EObjectFlags::RF_Transactional);
	// (end of section that we should not need in future)
}

void UMeshProviderModifier::InitializeModifier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshProviderModifier::InitializeModifier);

	Super::InitializeModifier();

	const UMeshPartitionDefinition* Definition = GetMegaMeshDefinition();

	if (Definition != nullptr)
	{
		OnMegaMeshDefinitionChanged(Definition);
	}
}

void UMeshProviderModifier::UninitializeModifier()
{
	Super::UninitializeModifier();
}

void UMeshProviderModifier::SetPreviewSection(MeshPartition::APreviewSection* InPreviewSection)
{
	bool bBroadcast = (GetPreviewSection() != InPreviewSection);
	Super::SetPreviewSection(InPreviewSection);
	if (bBroadcast)
	{
		OnPreviewSectionReassignmentDelegate.Broadcast(this, InPreviewSection);
	}
}

TArray<FBox> UMeshProviderModifier::ComputeBounds() const
{
	if (MeshComponent == nullptr || MeshComponent->GetMesh() == nullptr)
	{
		return TArray<FBox>();
	}

	return TArray<FBox>({ FBox(Geometry::FAxisAlignedBox3d(MeshComponent->GetMesh()->GetBounds(), GetComponentTransform())) });
}

const FDynamicMesh3* UMeshProviderModifier::GetMesh() const
{
	if (MeshComponent == nullptr)
	{
		return nullptr;
	}

	return MeshComponent->GetMesh();
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UMeshProviderModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshMeshProviderLocals;

	if (!MeshCopyForBackgroundOps)
	{
		TSharedPtr<FDynamicMesh3> MutableMeshCopy = MakeShared<FDynamicMesh3>();
		MeshComponent->ProcessMesh([MutableMeshCopy](const FDynamicMesh3& Mesh)
		{
			MutableMeshCopy->Copy(Mesh);
		});
		MeshCopyForBackgroundOps = MutableMeshCopy;
	}

	TSharedPtr<FMeshProviderOp> Op = MakeShared<FMeshProviderOp>(GetFName());
	// Currently the mesh transform is given relative to the parent actor. This could change, but then the
	//  functions that build the base need to change accordingly.
	Op->MeshTransform = GetRelativeTransform();
	Op->Mesh = MeshCopyForBackgroundOps;

	return Op;
}

void UMeshProviderModifier::SetMesh(FDynamicMesh3&& InDynamicMesh, bool bEmitChange)
{
	Modify(true);
	MeshCopyForBackgroundOps.Reset();
	
	// If we don't need to emit a change, the action is very simple
	if (!bEmitChange)
	{
		MeshComponent->SetMesh(MoveTemp(InDynamicMesh));
		return;
	}

	// If we got here, we need to emit an undo/redo change. Construct the change object by
	//  getting shared pointers to previous and new meshes.
	UDynamicMesh* DynamicMesh = MeshComponent->GetDynamicMesh();
	TUniquePtr<FDynamicMesh3> CurrentMesh = DynamicMesh->ExtractMesh();
	TSharedPtr<FDynamicMesh3> CurrentMeshShared(CurrentMesh.Release());
	TSharedPtr<FDynamicMesh3> NewMeshShared = MakeShared<FDynamicMesh3>(MoveTemp(InDynamicMesh));
	TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(CurrentMeshShared, NewMeshShared);

	// Apply the edit
	DynamicMesh->EditMesh([&NewMeshShared](FDynamicMesh3& EditMesh) { EditMesh = *NewMeshShared; });

	// Emit the change
	FScopedTransaction Transaction(LOCTEXT("SetMeshTransaction", "Update Mesh"));
	if (ensure(GUndo))
	{
		GUndo->StoreUndo(DynamicMesh, MoveTemp(ReplaceChange));
	}
}

double UMeshProviderModifier::GetComplexity() const
{
	const FDynamicMesh3* Mesh = MeshComponent ? MeshComponent->GetMesh() : nullptr;

	if (Mesh == nullptr)
	{
		return 0.f;
	}

	return Mesh->VertexCount();
}

void UMeshProviderModifier::SetIsTemporarilyHiddenInEditor(const bool bInIsHidden)
{
	Super::SetIsTemporarilyHiddenInEditor(bInIsHidden);

	if (MeshComponent != nullptr)
	{
		MeshComponent->SetIsTemporarilyHiddenInEditor(bInIsHidden);
	}
}

void UMeshProviderModifier::SetMaterial(int32 InElementIndex, UMaterialInterface* InMaterial)
{
	MeshComponent->SetMaterial(InElementIndex, InMaterial);
}

void UMeshProviderModifier::OnDynamicMeshChanged()
{
	//todo(luc.eygasier): investigate why OnMeshChanged event is always fired twice.
	if (LastFrameChanged != GFrameCounter)
	{
		constexpr bool bAlwaysMarkDirty = true;
		Modify(bAlwaysMarkDirty);

		// Will need to create a new copy for background ops.
		MeshCopyForBackgroundOps.Reset();

		OnChanged(ComputeBounds(), EChangeType::StateChange);
		LastFrameChanged = GFrameCounter;

		UpdateBounds();
		MarkRenderTransformDirty();
	}
}

void UMeshProviderModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	// TODO: Ideally we would highlight the boundary of the base mesh, or something like that. However it
	//  would likely require a slightly different approach, where we initialize a lineset when we select the
	//  component, and clear it when we end visualization.
	// For now we'll just draw the bounds.

	const FColor BoundsColor = FColor::Yellow;
	constexpr float BoundsThickness = 1;
	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	DrawWireBox(PDI, GetOwnerBounds(false), BoundsColor, SDPG_Foreground, BoundsThickness, DepthBias, bScreenSpace);
}

void UMeshProviderModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UMeshProviderModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	// From the mesh component we only care about the dynamic mesh
	Dependencies += MeshComponent->GetDynamicMesh();
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
