// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionModifierVisualizer.h"
#include "MeshPartitionModifierComponent.h"
#include "PrimitiveDrawingUtils.h" // FPrimitiveDrawInterface

void UE::MeshPartition::FMegaMeshModifierVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (const MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(Component))
	{
		Modifier->DrawVisualization(View, PDI);
	}
}
