// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"

namespace UE::MeshPartition
{
	/**
	 * Provides an interface for components to visualize themselves when selected in the editor.
	*/
	class FMegaMeshModifierVisualizer : public FComponentVisualizer
	{
	private:
		virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	};
} // namespace UE::MeshPartition