// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionNoiseModifier.h"

#include "MathUtil.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "PrimitiveDrawingUtils.h"
#include "Math/UnrealMathUtility.h"

namespace UE::MeshPartition
{
namespace MegaMeshNoiseModifierLocals
{

template <typename T>
inline T FallOff1D(const T Value, const T FallOffDist)
{
	ensure(Value <= T(1));
	ensure(Value >= T(0));

	if (Value < FallOffDist)
	{
		return FMath::SmoothStep(0., FallOffDist, Value);
	}
	else if (Value > T(1) - FallOffDist)
	{
		return FMath::SmoothStep(0., FallOffDist, (T(1) - Value));
	}

	return T(1.);
}

class FNoiseBackgroundOp : public MeshPartition::IModifierBackgroundOp
{
public:
	FNoiseBackgroundOp(const FName& InOperationName)
		: MeshPartition::IModifierBackgroundOp(InOperationName)
	{ }
	
	virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;

	virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
		const FInstanceInfo& InInstanceDesc) const override;

	Geometry::FAxisAlignedBox3d GetLocalCoverage() const
	{
		return Geometry::FAxisAlignedBox3d(-UnscaledCoverage / 2., UnscaledCoverage / 2.);
	}

	// Generate a new random guid before submitting any code changes to the op
	static FGuid GetCodeVersionKey()
	{
		static FGuid VersionKey(TEXT("b4c4e20d-244b-44a4-8936-90582af0e65c"));
		return VersionKey;
	}

	// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
	// and poisoning the cache/generating lots of unused intermediate data.
	virtual bool DisableDDCWrite() const override { return false; }

	FBox GlobalBounds;
	FTransform ComponentTransform;

	FVector3d UnscaledCoverage;
	FVector2D NoiseTranslate;
	FVector2D NoiseFrequency;
	double NoiseRotation;
	MeshPartition::ENoiseModifierType DisplacementType;
	MeshPartition::ENoiseParameterization ParametrizationType;
	MeshPartition::EFBMMode FBMMode;
	double Intensity;
	double Falloff;
	int FBMOctaves;
	double FBMLacunarity;
	double FBMGain;
	double FBMSmoothness;
	double FBMGamma;
	bool bWriteToWeightChannel;
	FName WeightChannelName;
};

void FNoiseBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	if (GlobalBounds.Intersect(InBounds))
	{
		FInstanceInfo InstanceInfo;

		// we want to both read and write vertex positions
		InstanceInfo.ReadViewComponents = EMeshViewComponents::VertexPos;
		InstanceInfo.WriteViewComponents = EMeshViewComponents::VertexPos;
		InstanceInfo.Bounds = GlobalBounds;
		InstanceInfo.InstanceID = 0;
		
		if (bWriteToWeightChannel && !WeightChannelName.IsNone())
		{
			InstanceInfo.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
			InstanceInfo.WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
			InstanceInfo.UsedChannels.Emplace(WeightChannelName);
		}

		OutInstanceInfos.Add(MoveTemp(InstanceInfo));
	}
}

void FNoiseBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UNoiseModifier::ApplyModifications);

	// local to world
	const Geometry::FAxisAlignedBox3d LocalBounds = GetLocalCoverage();

	// convert to enum needed by UE::Geometry
	Geometry::EFBMMode FBMNoiseMode;
	switch (FBMMode) {
	case MeshPartition::EFBMMode::Standard:
		FBMNoiseMode = Geometry::EFBMMode::Standard;
		break;
	case MeshPartition::EFBMMode::Turbulent:
		FBMNoiseMode = Geometry::EFBMMode::Turbulent;
		break;
	case MeshPartition::EFBMMode::Ridge:
		FBMNoiseMode = Geometry::EFBMMode::Ridge;
		break;
	default:
		FBMNoiseMode = Geometry::EFBMMode::Standard;
		check(false);
	};

	for (int VertexIndex = 0; VertexIndex < InMeshView.VertexCount(); ++VertexIndex)
	{
		// vertex coordinates in local space of mesh
		FVector3d MeshVertex = InMeshView.GetVertexPos(VertexIndex);

		// world space position
		FVector3d WorldOriginalVertPosition = InTransform.TransformPosition(MeshVertex);

		FVector3d PatchLocalVertPosition = ComponentTransform.InverseTransformPosition(WorldOriginalVertPosition);

		if (!LocalBounds.Contains(PatchLocalVertPosition))
		{
			// Vertex is outside of patch
			continue;
		}

		const FVector2d PatchUV = FVector2d(PatchLocalVertPosition.X / UnscaledCoverage.X,
			PatchLocalVertPosition.Y / UnscaledCoverage.Y);

		// magnitude of offset to apply in component-local Z direction
		double TotalOffset{ 0. };

		const double NoiseRotationRad = FMath::DegreesToRadians(NoiseRotation);
		Geometry::FMatrix2d Rotate{ FMath::Cos(NoiseRotationRad), -FMath::Sin(NoiseRotationRad), FMath::Sin(NoiseRotationRad), FMath::Cos(NoiseRotationRad) };

		// untransformed 
		FVector2d NoiseST = ParametrizationType == MeshPartition::ENoiseParameterization::World ?
			FVector2d(WorldOriginalVertPosition.X, WorldOriginalVertPosition.Y) :
			PatchUV;

		// transform
		NoiseST = NoiseTranslate + Rotate * NoiseST * NoiseFrequency;

		if (DisplacementType == MeshPartition::ENoiseModifierType::SineWave)
		{
			TotalOffset += FMath::Sin(NoiseST.X) * FMath::Sin(NoiseST.Y);
		}
		else if (DisplacementType == MeshPartition::ENoiseModifierType::FBmNoise)
		{
			TotalOffset = Geometry::FractalBrownianMotionNoise(FBMNoiseMode, FBMOctaves, NoiseST, FBMLacunarity, FBMGain, FBMSmoothness, FBMGamma);
		}

		// Relative distance towards the boundary of the rectangular modifier region.
		const double FallOffDist = 0.5 * FMath::Clamp(Falloff, 0., 1.);

		// Apply falloff  
		TotalOffset *= MegaMeshNoiseModifierLocals::FallOff1D(FMath::Clamp(PatchUV.X + 0.5, 0., 1.), FallOffDist) * MegaMeshNoiseModifierLocals::FallOff1D(FMath::Clamp(PatchUV.Y + 0.5, 0., 1.), FallOffDist);

		if (bWriteToWeightChannel && !WeightChannelName.IsNone())
		{
			// \todo assuming that weights outside [0,1] are allowed and correspondingly handled in the material
			InMeshView.SetVertexAttributeWeight(WeightChannelName, VertexIndex, TotalOffset);
		}

		PatchLocalVertPosition.Z += Intensity * TotalOffset;

		const FVector3d NewVertWorldPosition = ComponentTransform.TransformPosition(PatchLocalVertPosition);
		InMeshView.SetVertexPos(VertexIndex, InTransform.InverseTransformPosition(NewVertWorldPosition));
	}
}

} // UE::MeshPartition::Private

UNoiseModifier::UNoiseModifier()
{
}

void UNoiseModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const FName ParentPreviousDefaultType = TEXT("Misc");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("Noise");
		SetType(PreviousModifierDefaultType);
	}
}

TArray<FBox> UNoiseModifier::ComputeBounds() const
{
	Geometry::FAxisAlignedBox3d Coverage = GetLocalCoverage();
	return { FBox(Geometry::FAxisAlignedBox3d(Coverage, GetComponentTransform())) };
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UNoiseModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	TSharedPtr<MegaMeshNoiseModifierLocals::FNoiseBackgroundOp> Op = MakeShared<MegaMeshNoiseModifierLocals::FNoiseBackgroundOp>(GetFName());
	
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->ComponentTransform = GetComponentTransform();
	Op->UnscaledCoverage = UnscaledCoverage;
	Op->NoiseTranslate = NoiseTranslate;
	Op->NoiseFrequency = NoiseFrequency;
	Op->NoiseRotation = NoiseRotation;
	Op->DisplacementType = DisplacementType;
	Op->ParametrizationType = ParametrizationType;
	Op->FBMMode = FBMMode;
	Op->Intensity = Intensity;
	Op->Falloff = Falloff;
	Op->FBMOctaves = FBMOctaves;
	Op->FBMLacunarity = FBMLacunarity;
	Op->FBMGain = FBMGain;
	Op->FBMSmoothness = FBMSmoothness;
	Op->FBMGamma = FBMGamma;
	Op->bWriteToWeightChannel = bWriteToWeightChannel;
	Op->WeightChannelName = WeightChannelName;
	
	return Op;
}

void UNoiseModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	if (!ShouldRender())
		return;

	const FColor RectangleColor = FColor::Red;
	const FColor LocalBoundsColor = FColor::Yellow;
	const FColor GlobalBoundsColor = FColor::Orange;
	const float RectangleThickness = 3;
	const float BoundsThickness = 1;
	const float DepthBias = 1;
	const bool bScreenSpace = true;

	FTransform PatchToWorld = GetComponentTransform();
	if (bDrawPatchRectangle)
	{
		DrawRectangle(PDI, PatchToWorld.GetTranslation(), PatchToWorld.GetUnitAxis(EAxis::X), PatchToWorld.GetUnitAxis(EAxis::Y),
			RectangleColor, UnscaledCoverage.X * PatchToWorld.GetScale3D().X, UnscaledCoverage.Y * PatchToWorld.GetScale3D().Y, SDPG_Foreground,
			RectangleThickness, DepthBias, bScreenSpace);
	}

	if (bDrawAffectedBox)
	{
		Geometry::FAxisAlignedBox3d LocalCoverage = GetLocalCoverage();

		FBox LocalBounds(LocalCoverage.Min, LocalCoverage.Max);
        
		DrawWireBox(PDI, PatchToWorld.ToMatrixWithScale(), LocalBounds,
			LocalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
	}
}

FGuid UNoiseModifier::GetCodeVersionKey() const
{
	return MegaMeshNoiseModifierLocals::FNoiseBackgroundOp::GetCodeVersionKey();
}
} // namespace UE::MeshPartition