// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionPatchModifier.h"

#include "CanvasItem.h"
#include "CanvasTypes.h"
#include "Engine/TextureRenderTarget2D.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "MaterialShared.h"
#include "MeshPartitionMeshView.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "RHIUtilities.h"
#include "SceneInterface.h"
#include "TextureResource.h"
#include "UObject/Package.h"

namespace UE::MeshPartition
{
namespace MegaMeshPatchModifierLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		FBox GlobalBounds;
		MeshPartition::UPatchModifier::FSettings Settings;
		FVector WorldLocation = FVector::ZeroVector;

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override
		{
			AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);

			if (Settings.bWriteToWeightChannel && !Settings.WeightChannelName.IsNone() && !OutInstanceInfos.IsEmpty())
			{
				OutInstanceInfos[0].ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				OutInstanceInfos[0].WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				OutInstanceInfos[0].UsedChannels.Emplace(Settings.WeightChannelName);
			}
		}
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UPatchModifier::ApplyModifications);

			const FVector MeshSpaceLocation = InTransform.InverseTransformPosition(WorldLocation);

			MeshPartition::UPatchModifier::ApplyDeformation(Settings, InMeshView, MeshSpaceLocation);
		}
		
		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("0e0a6eef-8039-41a9-8888-775358ee29e7"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};
}

UPatchModifier::UPatchModifier()
{
}

void UPatchModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const FName ParentPreviousDefaultType = TEXT("Misc");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("Patch");
		SetType(PreviousModifierDefaultType);
	}
}

TArray<FBox> UPatchModifier::ComputeBounds() const
{
	const FVector PatchLocation = GetComponentLocation();
	return { FBox(PatchLocation - FVector(Radius + Falloff, Radius + Falloff, MaxZDistance / 2.f), PatchLocation + FVector(Radius + Falloff, Radius + Falloff, MaxZDistance / 2.f)) };
}

void UPatchModifier::SetRadius(float InRadius)
{
	Radius = InRadius;
}

void UPatchModifier::SetFalloff(float InFalloff)
{
	Falloff = InFalloff;
}

void UPatchModifier::SetMaxZDistance(float InMaxZDistance)
{
	MaxZDistance = InMaxZDistance;
}

void UPatchModifier::SetWriteToWeightChannel(bool bInWriteToWeightChannel)
{
	bWriteToWeightChannel = bInWriteToWeightChannel;
}

void UPatchModifier::SetWeightChannelName(FName InWeightChannelName)
{
	WeightChannelName = InWeightChannelName;
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UPatchModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshPatchModifierLocals;

	TSharedPtr<FBackgroundOp> Op = MakeShared<FBackgroundOp>(GetFName());
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->Settings = FSettings(*this);
	Op->WorldLocation = GetComponentLocation();

	return Op;
}

FGuid UPatchModifier::GetCodeVersionKey() const
{
	return MegaMeshPatchModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

void UPatchModifier::ApplyDeformation(const FSettings& Settings, MeshPartition::FMeshView& InMeshView, const FVector& Location)
{
	const bool bShouldWriteToWeightChannel = Settings.bWriteToWeightChannel && (!Settings.WeightChannelName.IsNone());

	for (int VertexIndex = 0; VertexIndex < InMeshView.VertexCount(); ++ VertexIndex)
	{
		FVector3d MeshVertex = InMeshView.GetVertexPos(VertexIndex);
		const double Distance = FVector2D::Distance(FVector2D(MeshVertex), FVector2D(Location));

		if (Distance > (Settings.Radius + Settings.Falloff))
		{
			continue;
		}
		
		if (Distance < Settings.Radius)
		{
			const float NewVertexHeight = Location.Z;

			MeshVertex.Z = NewVertexHeight;
			InMeshView.SetVertexPos(VertexIndex, MeshVertex);

			if (bShouldWriteToWeightChannel)
			{
				InMeshView.SetVertexAttributeWeight(Settings.WeightChannelName, VertexIndex, 1.f);
			}
		}
		else // inside falloff region:
		{
			const double Alpha = FMath::Clamp((Distance - Settings.Radius) / Settings.Falloff, 0., 1.0);

			const float NewVertexHeight = FMath::Lerp(Location.Z, MeshVertex.Z, Alpha);

			MeshVertex.Z = NewVertexHeight;
			InMeshView.SetVertexPos(VertexIndex, MeshVertex);

			if (bShouldWriteToWeightChannel)
			{
				float CurrentValue = InMeshView.GetVertexAttributeWeight(Settings.WeightChannelName, VertexIndex);
				InMeshView.SetVertexAttributeWeight(Settings.WeightChannelName, VertexIndex, FMath::Max(CurrentValue, 1.0 - Alpha));
			}
		}
	}
}

void UPatchModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UPatchModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	Dependencies += Radius;
	Dependencies += Falloff;
	Dependencies += MaxZDistance;

	const bool bShouldWriteToWeightChannel = bWriteToWeightChannel && (!WeightChannelName.IsNone());
	Dependencies += bShouldWriteToWeightChannel;

	if (bShouldWriteToWeightChannel)
	{
		Dependencies += WeightChannelName;
	}
}
} // namespace UE::MeshPartition
