// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"

#include "CoreGlobals.h" // GUndo
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "Misc/ITransaction.h"
#include "PrimitiveDrawingUtils.h" // DrawWireBox
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "MegaMeshProjectSculptLayersModifier"

DEFINE_LOG_CATEGORY(LogSculptLayersModifier);

namespace UE::MeshPartition
{
namespace MegaMeshProjectSculptLayersModifierLocals
{
	const Geometry::FDynamicMeshWeightAttribute* FindMeshWeightAttributeByName(const FDynamicMesh3& Mesh, const FName& ChannelName)
	{
		using namespace Geometry;

		if (const FDynamicMeshAttributeSet* Attribs = Mesh.Attributes())
		{
			for (int32 Idx = 0; Idx < Attribs->NumWeightLayers(); ++Idx)
			{
				const FDynamicMeshWeightAttribute* Layer = Attribs->GetWeightLayer(Idx);
				if (Layer->GetName() == ChannelName)
				{
					return Layer;
				}
			}
		}
		return nullptr;
	}
	// non-const version
	Geometry::FDynamicMeshWeightAttribute* FindMeshWeightAttributeByName(FDynamicMesh3& Mesh, const FName& ChannelName)
	{
		using namespace Geometry;
		if (FDynamicMeshAttributeSet* Attribs = Mesh.Attributes())
		{
			for (int32 Idx = 0; Idx < Attribs->NumWeightLayers(); ++Idx)
			{
				FDynamicMeshWeightAttribute* Layer = Attribs->GetWeightLayer(Idx);
				if (Layer->GetName() == ChannelName)
				{
					return Layer;
				}
			}
		}
		return nullptr;
	}

	FVector3d GetMeshLocalProjectionDirection(const FTransform& ProjectionMeshToWorld, const FTransform& ProjectionTransform)
	{
		FVector3d WorldProjectionDirection = -ProjectionTransform.GetUnitAxis(EAxis::Z) * FMathd::SignNonZero(ProjectionTransform.GetScale3D().Z);

		FVector3d MeshLocalProjectionDirection = ProjectionMeshToWorld.InverseTransformVector(WorldProjectionDirection);
		MeshLocalProjectionDirection.Normalize();
		return MeshLocalProjectionDirection;
	}

	void ProjectVertexOntoMesh(const FVector3d& WorldOriginalVertPosition,
		const FVector3d& MeshLocalProjectionDirection,
		const FDynamicMesh3& ProjectionMesh, const FTransform& ProjectionMeshToWorld,
		const Geometry::FDynamicMeshAABBTree3& ProjectionMeshSpatial,
		const FTransform& ProjectionTransform, const Geometry::FAxisAlignedBox3d& ProjectionSpaceBounds,
		TFunctionRef<void(int32 MeshHitTID, const FVector3d& MeshHitBaryCoords, const FVector3d& MeshLocalHitPos)> ModifyFromProjection)
	{
		FVector ProjectionOriginalVertPosition = ProjectionTransform.InverseTransformPosition(WorldOriginalVertPosition);

		if (!ProjectionSpaceBounds.Contains(ProjectionOriginalVertPosition))
		{
			return;
		}

		FVector3d ProjectionSpaceRayOrigin = ProjectionOriginalVertPosition;
		ProjectionSpaceRayOrigin.Z = ProjectionSpaceBounds.Max.Z;

		FVector3d MeshSpaceRayOrigin = ProjectionMeshToWorld.InverseTransformPosition(
			ProjectionTransform.TransformPosition(ProjectionSpaceRayOrigin));

		FRay3d MeshLocalRay(
			MeshSpaceRayOrigin,
			MeshLocalProjectionDirection);

		double LocalHitT = TNumericLimits<double>::Max();
		int32 HitTID = IndexConstants::InvalidID;
		FVector3d BaryCoords;
		if (!ProjectionMeshSpatial.FindNearestHitTriangle(Geometry::FWatertightRay3d(MeshLocalRay), LocalHitT, HitTID, BaryCoords))
		{
			return;
		}

		FVector3d MeshHitLocation = MeshLocalRay.PointAt(LocalHitT);
		FVector3d WorldHitLocation = ProjectionMeshToWorld.TransformPosition(MeshHitLocation);
		FVector3d ProjectionSpaceHitLocation = ProjectionTransform.InverseTransformPosition(WorldHitLocation);

		bool bStillWithinProjectionBounds = ProjectionSpaceHitLocation.Z >= ProjectionSpaceBounds.Min.Z;
		if (bStillWithinProjectionBounds)
		{
			ModifyFromProjection(HitTID, BaryCoords, MeshHitLocation);
		}
	}

	void VertexOntoMeshClosestPoint(const FVector3d& WorldSpaceVertexPosition,
		const FDynamicMesh3& RefMesh, const FTransform& ReferenceMeshToWorld,
		double MeshSpaceSearchDist, bool bUseBoundaryFalloff, double BoundaryFalloffDistanceFactor,
		const Geometry::FDynamicMeshAABBTree3& RefMeshSpatial,
		TFunctionRef<void(double ScaleFactor, int32 MeshHitTID, const FVector3d& MeshHitBaryCoords, const FVector3d& MeshLocalHitPos)> ModifyFromProjection)
	{
		using namespace Geometry;

		FVector3d MeshSpaceVertexPosition = ReferenceMeshToWorld.InverseTransformPosition(WorldSpaceVertexPosition);

		IMeshSpatial::FQueryOptions Options(MeshSpaceSearchDist);
		double NearDistSq;
		int32 NearTID = RefMeshSpatial.FindNearestTriangle(MeshSpaceVertexPosition, NearDistSq, Options);
		if (NearTID == INDEX_NONE)
		{
			return;
		}

		FTriangle3d Tri;
		RefMesh.GetTriVertices(NearTID, Tri.V[0], Tri.V[1], Tri.V[2]);
		FDistPoint3Triangle3d TriDist(MeshSpaceVertexPosition, Tri);
		TriDist.ComputeResult();

		FVector3d ClosestPt = TriDist.ClosestTrianglePoint;

		double ScaleFactor = 1.;
		if (bUseBoundaryFalloff)
		{
			bool bIsOnBoundary = false;
			for (int32 VertSubIdx = 0; VertSubIdx < 3; ++VertSubIdx)
			{
				double VertBaryWt = TriDist.TriangleBaryCoords[VertSubIdx];
				if (VertBaryWt > 1 - UE_DOUBLE_KINDA_SMALL_NUMBER)
				{
					if (RefMesh.IsBoundaryVertex(RefMesh.GetTriangle(NearTID)[VertSubIdx]))
					{
						bIsOnBoundary = true;
					}
					break;
				}
				else if (VertBaryWt < UE_DOUBLE_KINDA_SMALL_NUMBER)
				{
					int32 OppEdgeSubIdx = VertSubIdx + 1;
					if (OppEdgeSubIdx > 2)
					{
						OppEdgeSubIdx = 0;
					}
					if (RefMesh.IsBoundaryEdge(RefMesh.GetTriEdge(NearTID, OppEdgeSubIdx)))
					{
						bIsOnBoundary = true;
						break;
					}
				}
			}

			if (bIsOnBoundary)
			{
				const double FalloffDist = MeshSpaceSearchDist * BoundaryFalloffDistanceFactor;
				if (FalloffDist > 0.)
				{
					FVector3d TriNormal = RefMesh.GetTriNormal(NearTID);
					FVector3d Separation = MeshSpaceVertexPosition - TriDist.ClosestTrianglePoint;
					FVector3d InTriPlaneSeparation = Separation - (Separation.Dot(TriNormal)) * TriNormal;
					// When using smoothed falloff, extrapolate off-bounds closest points to match the falloff surface
					// (otherwise vertices in the falloff range would be 'pinched' toward the boundary)
					ClosestPt += InTriPlaneSeparation;

					double InTriPlaneDist = InTriPlaneSeparation.Length();
					double Falloff = FMath::Min(1., InTriPlaneDist / FalloffDist);
					ScaleFactor = 1. - Falloff;
				}
				else
				{
					ScaleFactor = 0.;
				}
			}
		}
		ModifyFromProjection(ScaleFactor, NearTID, TriDist.TriangleBaryCoords, ClosestPt);
	}

	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName, TSharedRef<const FAsyncMeshInstanceData> InMeshInstance)
			: MeshPartition::IModifierBackgroundOp(InOperationName)
			, MeshInstance(MoveTemp(InMeshInstance))
		{}

		FBox GlobalBounds;
		TArray<double> LayerWeights;
		FTransform MeshToWorld;
		TSharedRef<const FAsyncMeshInstanceData> MeshInstance;
		ESculptLayerProjectMethod ProjectMethod;
		bool bSculptAbsolutePositions;
		float VerticalExtentUp;
		float VerticalExtentDown;

		TArray<FSculptLayerModifierWeightAttributeEntry> AttributeWeightChannels;

		// FixedDirection settings
		FTransform ProjectionTransform;

		// ClosestPoint settings
		float MaxClosestPointDistance;
		bool bUseBoundaryFalloff;
		float BoundaryFalloffDistanceFactor;

		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;
		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("64a3d146-334e-4327-bd40-726af9f0e850"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};

	inline static bool IsTriangleSculpted(const Geometry::FIndex3i& Tri, const FDynamicMesh3& Mesh, const Geometry::FDynamicMeshSculptLayers* SculptLayers)
	{
		for (int32 LayerIdx = 1; LayerIdx < SculptLayers->NumLayers(); ++LayerIdx)
		{
			for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
			{
				int32 VID = Tri[SubIdx];
				FVector3d Offset;
				SculptLayers->GetLayer(LayerIdx)->GetValue(VID, Offset);
				if (!Offset.IsNearlyZero())
				{
					return true;
				}
			}
		}
		return false;
	}

	Geometry::FAxisAlignedBox3d GetProjectionBounds(const Geometry::FAxisAlignedBox3d& InputBounds, const FTransform& InProjectionTransform,
	                                                const FTransform& MeshToWorld, const float VerticalExtentUp, const float VerticalExtentDown)
	{
		Geometry::FAxisAlignedBox2d Mesh2DBoundsInProjection;
		for (int i = 0; i < 8; ++i)
		{
			FVector3d CornerWorldPosition = MeshToWorld.TransformPosition(InputBounds.GetCorner(i));
			FVector3d CornerProjectionPosition = InProjectionTransform.InverseTransformPosition(CornerWorldPosition);
			Mesh2DBoundsInProjection.Contain(FVector2d(CornerProjectionPosition.X, CornerProjectionPosition.Y));
		}

		return Geometry::FAxisAlignedBox3d(
			FVector3d(Mesh2DBoundsInProjection.Min, -VerticalExtentDown),
			FVector3d(Mesh2DBoundsInProjection.Max, VerticalExtentUp));
	}
}

UProjectMeshLayersModifier::UProjectMeshLayersModifier()
{
	MeshObject = CreateDefaultSubobject<UDynamicMesh>(TEXT("SculptMesh"));
	MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &MeshPartition::UProjectMeshLayersModifier::OnMeshObjectChanged);
}

void UProjectMeshLayersModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		// Sculpt modifier default priority layer was missed when converting others. If saved before the other
		//  conversions, the default value we would get from our parent is Misc, and if saved after, it would be 
		//  None. In either case, reinstate the old sculpt modifier default (Sculpt).
		const FName PreviousModifierDefaultType = TEXT("Sculpt");
		if (GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone)
		{
			if (GetType() == TEXT("Misc"))
			{
				SetType(PreviousModifierDefaultType);
			}
		}
		else if (GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::SculptPriorityLayerSetToNone
			&& GetType().IsNone())
		{
			SetType(PreviousModifierDefaultType);
		}
	}
}

void UProjectMeshLayersModifier::PostLoad()
{
	Super::PostLoad();

	// The intention here is that MeshObject is never nullptr, however we cannot guarantee this
	// Avoid immediate crashes by creating a new UDynamicMesh here in such cases
	if (ensure(MeshObject != nullptr) == false)
	{
		MeshObject = NewObject<UDynamicMesh>(this, TEXT("SculptMesh"));
		MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UProjectMeshLayersModifier::OnMeshObjectChanged);
	}
}

void UProjectMeshLayersModifier::PostEditImport()
{
	Super::PostEditImport();

	// MeshObject should never be null here, but we re-validate that it isn't (similar to PostLoad method, above)
	if (ensure(MeshObject != nullptr) == false)
	{
		MeshObject = NewObject<UDynamicMesh>(this, TEXT("DynamicMesh"));
		MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UProjectMeshLayersModifier::OnMeshObjectChanged);
	}

	ApplyMeshUpdate();
}

void UProjectMeshLayersModifier::BuildCachedData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UProjectMeshLayersModifier::BuildCachedData);

	if (!ensure(MeshObject))
	{
		MeshInstance.Reset();
		return;
	}


	MeshInstance = MakeShared<FAsyncMeshInstanceData>(MeshObject->GetMeshRef());
}

void UProjectMeshLayersModifier::ComputeMaxLayerOffsets()
{
	MaxLayerOffset.Reset();

	int32 NumLayers = GetProjectionMeshNumLayers();
	if (NumLayers < 2)
	{
		return;
	}
	MeshObject->ProcessMesh([this, NumLayers](const Geometry::FDynamicMesh3& Mesh)
	{
		const Geometry::FDynamicMeshSculptLayers* SculptLayers = Mesh.Attributes()->GetSculptLayers();
		MaxLayerOffset.SetNumZeroed(NumLayers - 1);
		// transform additional layers as vectors, and store the max offsets per layer
		for (int32 LayerIdx = 1; LayerIdx < NumLayers; ++LayerIdx)
		{
			if (const Geometry::FDynamicMeshSculptLayerAttribute* SculptLayer = SculptLayers->GetLayer(LayerIdx))
			{
				for (int32 VID : Mesh.VertexIndicesItr())
				{
					FVector3d Offset;
					SculptLayer->GetValue(VID, Offset);
					MaxLayerOffset[LayerIdx - 1] = FVector3d::Max(MaxLayerOffset[LayerIdx - 1], Offset.GetAbs());
				}
			}
		}
	});
}

void UProjectMeshLayersModifier::ApplyMeshUpdate()
{
	BuildCachedData();
	ComputeMaxLayerOffsets();
	OnChanged(ComputeBounds(), EChangeType::StateChange);
	OnMeshLayersPanelRequestRebuild.Broadcast();
}

void UProjectMeshLayersModifier::OnMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo)
{
	ApplyMeshUpdate();
}

void UProjectMeshLayersModifier::ApplyEditWithMesh(const Geometry::FDynamicMesh3& UpdatedMesh)
{
	SetSculptLayerMesh(UpdatedMesh, true);
}

void UProjectMeshLayersModifier::SetSculptLayerMesh(const Geometry::FDynamicMesh3& UpdatedMesh, bool bTransact)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UProjectMeshLayersModifier::ApplyEditWithMesh);
	
	if (!ensure(MeshObject))
	{
		return; // MeshObject should always exist
	}

	if (!UpdatedMesh.HasAttributes() || 
		(UpdatedMesh.Attributes()->NumSculptLayers() < 2 && UpdatedMesh.Attributes()->NumWeightLayers() == 0))
	{
		MeshObject->Reset();
		return;
	}

	TSharedPtr<Geometry::FDynamicMesh3> PreviousMeshForTransaction;
	TSharedPtr<Geometry::FDynamicMesh3> NewMeshForTransaction;
	TOptional<FScopedTransaction> Transaction;

	if (bTransact)
	{
		PreviousMeshForTransaction = MakeShared<Geometry::FDynamicMesh3>();
		NewMeshForTransaction = MakeShared<Geometry::FDynamicMesh3>();
		Transaction.Emplace(LOCTEXT("MegaMeshProjectMeshLayerModifier_CommitMeshChange", "Update Mesh Layer Modifier"));
		Modify();
	}

	MeshObject->EditMesh([this, &UpdatedMesh, &PreviousMeshForTransaction, &NewMeshForTransaction, bTransact](Geometry::FDynamicMesh3& Mesh)
	{
		const Geometry::FDynamicMeshSculptLayers* InputSculptLayers = UpdatedMesh.Attributes()->GetSculptLayers();

		if (bTransact)
		{
			PreviousMeshForTransaction->CompactCopy(Mesh);
		}

		Mesh = UpdatedMesh;

		// Remove any attribute layers on the mesh that don't have an entry in AttributeWeightChannels 
		for (int32 AttribLayerIdx = 0; AttribLayerIdx < Mesh.Attributes()->NumWeightLayers(); ++AttribLayerIdx)
		{
			Geometry::FDynamicMeshWeightAttribute* WeightAttrib = Mesh.Attributes()->GetWeightLayer(AttribLayerIdx);
			const FSculptLayerModifierWeightAttributeEntry* Entry = AttributeWeightChannels.FindByPredicate(
				[Name = WeightAttrib->GetName()](const FSculptLayerModifierWeightAttributeEntry& Entry) { return Entry.ChannelName == Name; });
			if (!Entry)
			{
				Mesh.Attributes()->RemoveWeightLayer(AttribLayerIdx);
				--AttribLayerIdx;
			}
		}

		// Consider discarding unsculpted triangles if there are no weight channels
		// We currently preserve all triangles if we are also projecting weights
		if (bDiscardUnsculpted && AttributeWeightChannels.IsEmpty())
		{
			Geometry::FDynamicMeshSculptLayers* ResultSculptLayers = Mesh.Attributes()->GetSculptLayers();
			// make sure the sculpt layer offsets are up-to-date
			ResultSculptLayers->UpdateLayersFromMesh();
			// delete triangles with no offsets on any vertex
			for (int32 TID = 0; TID < Mesh.MaxTriangleID(); ++TID)
			{
				if (!Mesh.IsTriangle(TID))
				{
					continue;
				}

				if (!MegaMeshProjectSculptLayersModifierLocals::IsTriangleSculpted(Mesh.GetTriangle(TID), Mesh, ResultSculptLayers))
				{
					Mesh.RemoveTriangle(TID);
				}
			}
			Mesh.CompactInPlace();
		}

		// Save the active layer so we'll use it for future edits as well
		ActiveLayer = InputSculptLayers->GetActiveLayer();

		// Save layer weights and set the projection target back to the base mesh
		int32 NumLayers = Mesh.Attributes()->NumSculptLayers();
		LayerWeights.Reset();
		if (NumLayers > 0)
		{
			LayerWeights.SetNum(NumLayers - 1);
			TConstArrayView<double> InputSculptLayerWeights = InputSculptLayers->GetLayerWeights();
			for (int32 LayerIdx = 0; LayerIdx < LayerWeights.Num(); ++LayerIdx)
			{
				LayerWeights[LayerIdx] = InputSculptLayerWeights[LayerIdx + 1];
			}

			Geometry::FDynamicMeshSculptLayers* ResultSculptLayers = Mesh.Attributes()->GetSculptLayers();

			// Set the projection mesh to its base layer positions, so we raycast vs the mesh with no sculpting applied
			TArray<double> SetToBaseLayerWeights;
			SetToBaseLayerWeights.SetNumZeroed(InputSculptLayerWeights.Num());
			SetToBaseLayerWeights[0] = 1.0;
			ResultSculptLayers->UpdateLayerWeights(SetToBaseLayerWeights);
		}

		if (bTransact)
		{
			NewMeshForTransaction->CompactCopy(Mesh);
		}
	});
	
	if (bTransact)
	{
		TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(PreviousMeshForTransaction, NewMeshForTransaction);
		check(GUndo != nullptr);
		MeshObject->Modify();
		GUndo->StoreUndo(MeshObject, MoveTemp(ReplaceChange));
	}

	// Note: MeshSpatial will be updated by the OnMeshObjectChanged callback
}

FName UProjectMeshLayersModifier::GetLayerName(const int32 LayerIndex) const
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(LayerIndex))
	{
		UE_LOGF(LogSculptLayersModifier, Warning, "Sculpt Layer not found. Invalid index: %d.", LayerIndex);
		return NAME_None;
	}
	
	FName LayerName;
	MeshObject->ProcessMesh([this, &LayerIndex, &LayerName](const Geometry::FDynamicMesh3& Mesh)
	{
		const Geometry::FDynamicMeshSculptLayers* SculptLayers = Mesh.Attributes()->GetSculptLayers();
		// ensure there are always sculpt layers to read the names from
		LayerName = SculptLayers->NumLayers() == 0 ? LayerName = NAME_None :
			LayerName = SculptLayers->GetLayer(LayerIndex + NumLockedBaseSculptLayers())->GetName();
	});
	return LayerName;
}

void UProjectMeshLayersModifier::SetLayerWeight(const int32 InLayerIndex, const double InWeight, const EPropertyChangeType::Type ChangeType)
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(InLayerIndex))
	{
		UE_LOGF(LogSculptLayersModifier, Warning, "Sculpt Layer not found. Invalid index: %d.", InLayerIndex);
		return;
	}

	if (FMath::IsNearlyEqual(LayerWeights[InLayerIndex], InWeight))
	{
		return;
	}
	
	Modify();

	LayerWeights[InLayerIndex] = InWeight;
	
	FProperty* WtProp = FindFProperty<FProperty>(UProjectMeshLayersModifier::StaticClass(),
		GET_MEMBER_NAME_CHECKED(UProjectMeshLayersModifier, LayerWeights));
	FPropertyChangedEvent PropertyChangedEvent(WtProp, ChangeType);
	PostEditChangeProperty(PropertyChangedEvent);
}

void UProjectMeshLayersModifier::SetLayerName(const int32 InLayerIndex, const FName InName)
{
	// ensure the provided index is valid
	if (!LayerWeights.IsValidIndex(InLayerIndex))
	{
		UE_LOGF(LogSculptLayersModifier, Warning, "Sculpt Layer not found. Invalid index: %d.", InLayerIndex);
		return;
	}
	
	Modify();

	MeshObject->EditMesh([this, &InLayerIndex, &InName](UE::Geometry::FDynamicMesh3& Mesh)
	{
		UE::Geometry::FDynamicMeshSculptLayers* SculptLayers = Mesh.Attributes()->GetSculptLayers();
		SculptLayers->GetLayer(InLayerIndex + NumLockedBaseSculptLayers())->SetName(InName);
	});
}

void UProjectMeshLayersModifier::ClearMesh()
{
	TSharedPtr<Geometry::FDynamicMesh3> PreviousMeshForTransaction = MakeShared<Geometry::FDynamicMesh3>();

	MeshObject->ProcessMesh([this, &PreviousMeshForTransaction](const Geometry::FDynamicMesh3& Mesh)
	{
		PreviousMeshForTransaction->CompactCopy(Mesh);
	});
	
	// Create a clear mesh transaction
	FScopedTransaction Transaction(LOCTEXT("MegaMeshProjectMeshLayerModifier_CommitMeshClear", "Clear Mesh Layer Modifier"));
	TSharedPtr<Geometry::FDynamicMesh3> NewMeshForTransaction = MakeShared<Geometry::FDynamicMesh3>();
	TUniquePtr<FMeshReplacementChange> ReplaceChange = MakeUnique<FMeshReplacementChange>(PreviousMeshForTransaction, NewMeshForTransaction);
	check(GUndo != nullptr);
	MeshObject->Modify();
	GUndo->StoreUndo(MeshObject, MoveTemp(ReplaceChange));

	Modify();
	LayerWeights.Reset();

	// Reset the mesh
	MeshObject->Reset();
}

void UProjectMeshLayersModifier::PrepareForEdit(FDynamicMesh3& EditMesh) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UProjectMeshLayersModifier::PrepareForEdit);

	using namespace MegaMeshProjectSculptLayersModifierLocals;

	if (!(ensure(MeshObject) && ensure(MeshInstance)))
	{
		return;
	}

	int32 StoredNumLayers = GetProjectionMeshNumLayers();

	EditMesh.CompactInPlace();

	EditMesh.EnableAttributes();
	
	// Enable weight layers matching the modifier's AttributeWeightChannels
	{
		// We keep all channels, including ones that we don't write to, so that tools can use them if needed
		//  (we will remove them on commit if they don't match AttributeWeightChannels). However, we want to
		//  clear any built data for channels that are marked as "Add", as we want those channels to only hold
		//  the data we would add. We do this by removing and re-adding them before we transfer existing data.
		for (int32 AttribLayerIdx = 0; AttribLayerIdx < EditMesh.Attributes()->NumWeightLayers(); ++AttribLayerIdx)
		{
			Geometry::FDynamicMeshWeightAttribute* WeightAttrib = EditMesh.Attributes()->GetWeightLayer(AttribLayerIdx);
			const FSculptLayerModifierWeightAttributeEntry* Entry = AttributeWeightChannels.FindByPredicate(
				[Name = WeightAttrib->GetName()](const FSculptLayerModifierWeightAttributeEntry& Entry) { return Entry.ChannelName == Name; });
			if (Entry && Entry->Method == ESculptLayerSetWeightChannelMethod::Add)
			{
				EditMesh.Attributes()->RemoveWeightLayer(AttribLayerIdx);
				--AttribLayerIdx;
			}
		}

		// Enable channels we need to exist on the mesh
		for (const FSculptLayerModifierWeightAttributeEntry& Entry : AttributeWeightChannels)
		{
			if (!FindMeshWeightAttributeByName(EditMesh, Entry.ChannelName))
			{
				int32 NewLayerIdx = EditMesh.Attributes()->NumWeightLayers();
				EditMesh.Attributes()->SetNumWeightLayers(NewLayerIdx + 1);
				EditMesh.Attributes()->GetWeightLayer(NewLayerIdx)->SetName(Entry.ChannelName);
			}
		}
	}
	
	EditMesh.Attributes()->EnableSculptLayers(FMath::Max(2, StoredNumLayers));
	Geometry::FDynamicMeshSculptLayers* EditSculptLayers = EditMesh.Attributes()->GetSculptLayers();
	int32 UseActiveLayer = FMath::Clamp(ActiveLayer, 1, EditSculptLayers->NumLayers() - 1);

	EditSculptLayers->SetActiveLayer(UseActiveLayer);
	TArray<double> UseSculptLayers{ 1.0 };
	UseSculptLayers.Append(GetActiveLayerWeights());
	EditSculptLayers->UpdateLayerWeights(UseSculptLayers);

	// If we have stored layers, transfer them to the edit mesh
	if (StoredNumLayers > 1)
	{
		MeshObject->ProcessMesh([this, &EditMesh, &EditSculptLayers, StoredNumLayers](const Geometry::FDynamicMesh3& Mesh)
		{
			const Geometry::FDynamicMeshSculptLayers* SourceSculptLayers = Mesh.Attributes()->GetSculptLayers();
			using FWeightChannelSourceDestPair = TPair<const Geometry::FDynamicMeshWeightAttribute*, Geometry::FDynamicMeshWeightAttribute*>;
			for (int32 LayerIdx = 0; LayerIdx < EditSculptLayers->NumLayers() && LayerIdx < SourceSculptLayers->NumLayers(); ++LayerIdx)
			{
				EditSculptLayers->GetLayer(LayerIdx)->SetName(SourceSculptLayers->GetLayer(LayerIdx)->GetName());
			}
			TArray<FWeightChannelSourceDestPair> WeightSourceDests;
			for (const FSculptLayerModifierWeightAttributeEntry& Entry : AttributeWeightChannels)
			{
				FWeightChannelSourceDestPair WeightSrcDst
				{
					FindMeshWeightAttributeByName(Mesh, Entry.ChannelName),
					FindMeshWeightAttributeByName(EditMesh, Entry.ChannelName)
				};
				if (WeightSrcDst.Key && WeightSrcDst.Value)
				{
					WeightSourceDests.Add(WeightSrcDst);
				}
			}
			auto CopyAttributes = [&Mesh, EditSculptLayers, SourceSculptLayers, StoredNumLayers, &WeightSourceDests]
				(int32 TargetVID, int32 MeshHitTID, const FVector3d& MeshHitBaryCoords) -> void
				{
					Geometry::FIndex3i Tri = Mesh.GetTriangle(MeshHitTID);
					for (int32 LayerIdx = 1; LayerIdx < StoredNumLayers; ++LayerIdx)
					{
						const Geometry::FDynamicMeshSculptLayerAttribute* Layer = SourceSculptLayers->GetLayer(LayerIdx);
						FVector3d InterpOffset(0);
						for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
						{
							FVector3d Offset;
							Layer->GetValue(Tri[SubIdx], Offset);
							InterpOffset += MeshHitBaryCoords[SubIdx] * Offset;
						}
						Geometry::FDynamicMeshSculptLayerAttribute* ToLayer = EditSculptLayers->GetLayer(LayerIdx);
						ToLayer->SetValue(TargetVID, InterpOffset);
					}
					for (FWeightChannelSourceDestPair WtSrcDest : WeightSourceDests)
					{
						float InterpWt = 0;
						for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
						{
							float Wt;
							WtSrcDest.Key->GetValue(Tri[SubIdx], &Wt);
							InterpWt += MeshHitBaryCoords[SubIdx] * Wt;
						}
						WtSrcDest.Value->SetValue(TargetVID, &InterpWt);
					}
				};

			auto CopyBasePosition = [this, &EditMesh, EditSculptLayers]
				(int32 TargetVID, const FVector3d& BasePos) -> void
				{
					EditMesh.SetVertex(TargetVID, BasePos);
					Geometry::FDynamicMeshSculptLayerAttribute* Layer = EditSculptLayers->GetLayer(0);
					Layer->SetValue(TargetVID, BasePos);
				};

			const FTransform& MeshToWorld = GetComponentTransform();

			const Geometry::FDynamicMeshAABBTree3& AABBTree = MeshInstance->GetAABBTree();

			if (ProjectMethod == ESculptLayerProjectMethod::ClosestPoints)
			{
				double MeshSpaceSearchDist = MaxClosestPointDistance / FMath::Max(FMathd::ZeroTolerance, FMath::Abs(MeshToWorld.GetMinimumAxisScale()));
				ParallelFor(EditMesh.MaxVertexID(),
					[&Mesh, &MeshToWorld, &EditMesh, &CopyAttributes, &CopyBasePosition, MeshSpaceSearchDist, &AABBTree, this](int32 VID) -> void
					{
						if (!EditMesh.IsVertex(VID))
						{
							return;
						}
						FVector3d WorldMeshPos = MeshToWorld.TransformPosition(EditMesh.GetVertex(VID));
						VertexOntoMeshClosestPoint(WorldMeshPos,
							Mesh, MeshToWorld,
							MeshSpaceSearchDist, bUseBoundaryFalloff, BoundaryFalloffDistanceFactor,
							AABBTree,
							[VID, &CopyAttributes, &CopyBasePosition, &EditMesh, this]
							(double ScaleFactor, int32 MeshHitTID, const FVector3d& MeshHitBaryCoords, const FVector3d& MeshLocalHitPos) -> void
							{
								CopyAttributes(VID, MeshHitTID, MeshHitBaryCoords * ScaleFactor);
								if (bSculptAbsolutePositions)
								{
									CopyBasePosition(VID, FMath::Lerp(EditMesh.GetVertex(VID), MeshLocalHitPos, ScaleFactor));
								}
							}
						);
					});
			}
			else // ProjectMethod = ESculptLayerProjectMethod::FixedDirection
			{
				FTransform ProjectionTransform = GetProjectionTransform();
				Geometry::FAxisAlignedBox3d ProjectionSpaceBounds = GetProjectionBounds(AABBTree.GetBoundingBox(), ProjectionTransform);
				FVector LocalProjectionDirection = GetMeshLocalProjectionDirection(MeshToWorld, ProjectionTransform);
				ParallelFor(EditMesh.MaxVertexID(),
				            [&Mesh, &MeshToWorld, &EditMesh, &CopyAttributes, &CopyBasePosition, &ProjectionTransform, &ProjectionSpaceBounds, &
					            LocalProjectionDirection, &AABBTree, this]
					(int32 VID) -> void
					{
						if (!EditMesh.IsVertex(VID))
						{
							return;
						}
						FVector3d WorldMeshPos = MeshToWorld.TransformPosition(EditMesh.GetVertex(VID));
						ProjectVertexOntoMesh(WorldMeshPos, LocalProjectionDirection,
							Mesh, MeshToWorld, AABBTree, ProjectionTransform, ProjectionSpaceBounds,
							[VID, &CopyAttributes, &CopyBasePosition, &EditMesh, this]
							(int32 MeshHitTID, const FVector3d& MeshHitBaryCoords, const FVector3d& MeshLocalHitPos) -> void
							{
								CopyAttributes(VID, MeshHitTID, MeshHitBaryCoords);
								if (bSculptAbsolutePositions)
								{
									CopyBasePosition(VID, MeshLocalHitPos);
								}
							}
						);
					});
			}

			EditSculptLayers->RebuildMesh();
		}
		);

	}

	// Make sure normals on the edit mesh reflect the sculpt mesh layer offsets
	Geometry::FMeshNormals::QuickRecomputeOverlayNormals(EditMesh);
}

TArray<Geometry::FOrientedBox3d> UProjectMeshLayersModifier::GetBoundsForEdit() const
{
	FTransform ProjectionTransform = GetProjectionTransform();
	Geometry::FAxisAlignedBox3d ProjectionSpaceBounds = GetProjectionEditBounds();
	Geometry::FOrientedBox3d GlobalBounds(Geometry::FFrame3d(ProjectionTransform), ProjectionSpaceBounds.Extents() * ProjectionTransform.GetScale3D());
	return { GlobalBounds };
}

TArray<FBox> UProjectMeshLayersModifier::ComputeBounds() const
{
	if (!MeshInstance)
	{
		return {};
	}

	const Geometry::FAxisAlignedBox3d MeshSpatialBounds = MeshInstance->GetBounds();

	// catch empty bounds early, otherwise the very large numbers may cause overflows and NaNs in the projection calculations below
	if (MeshSpatialBounds.IsEmpty())
	{
		return {};
	}

	Geometry::FAxisAlignedBox3d GlobalBounds;
	if (ProjectMethod == ESculptLayerProjectMethod::FixedDirection)
	{
		// Can affect vertices within the projection of the bounds along the chosen direction
		FTransform ProjectionTransform = GetProjectionTransform();
		const Geometry::FAxisAlignedBox3d ProjectionSpaceBounds = GetProjectionBounds(MeshSpatialBounds, ProjectionTransform);
		GlobalBounds = Geometry::FAxisAlignedBox3d(ProjectionSpaceBounds, ProjectionTransform);
	}
	else // ProjectMethod == ESculptLayerProjectMethod::ClosestPoints
	{
		// Can affect vertices within the max closest point distance of the bounds
		GlobalBounds = Geometry::FAxisAlignedBox3d(MeshSpatialBounds, GetComponentTransform());
		GlobalBounds.Expand(MaxClosestPointDistance);
	}

	// Conservatively expand the bounds based on how much the sculpt layers will move the mesh vertices
	double ExpandBy = 0;
	TConstArrayView<double> ActiveWeights = GetActiveLayerWeights();
	const int32 NumLayers = FMath::Min(MaxLayerOffset.Num(), ActiveWeights.Num());
	for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
	{
		ExpandBy += MaxLayerOffset[LayerIdx].Length() * FMath::Abs(ActiveWeights[LayerIdx]) + .01;
	}
	GlobalBounds.Expand(ExpandBy);

	return { FBox(GlobalBounds) };
}

TConstArrayView<double> UProjectMeshLayersModifier::GetActiveLayerWeights() const
{
	return TConstArrayView<double>(LayerWeights);
}

int32 UProjectMeshLayersModifier::GetProjectionMeshNumLayers() const
{
	int32 NumLayers = 0;
	if (ensure(MeshObject))
	{
		MeshObject->ProcessMesh([&NumLayers](const Geometry::FDynamicMesh3& Mesh)
		{
			if (Mesh.HasAttributes())
			{
				NumLayers = Mesh.Attributes()->NumSculptLayers();
			}
		});
	}
	return NumLayers;
}


void UProjectMeshLayersModifier::OnRegister()
{
	Super::OnRegister();

	// Because cache data is not a UPROPERTY() it may need to be rebuilt
	if (!MeshInstance)
	{
		BuildCachedData();
	}
}

void MegaMeshProjectSculptLayersModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	if (AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos))
	{
		// If Background Op has attribute weight channels, enable their read/write also
		if (!AttributeWeightChannels.IsEmpty())
		{
			FInstanceInfo& Info = OutInstanceInfos.Last();
			Info.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
			Info.WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
			for (const FSculptLayerModifierWeightAttributeEntry& Entry : AttributeWeightChannels)
			{
				if (!Entry.bInternalChannel)
				{
					Info.UsedChannels.Emplace(Entry.ChannelName);
				}
			}
		}
	}
}

TArray<FSculptLayerModifierWeightAttributeEntry> UProjectMeshLayersModifier::GetActiveWeightChannels() const
{
	using namespace MegaMeshProjectSculptLayersModifierLocals;

	if (MeshInstance.IsValid())
	{
		const FDynamicMesh3& Mesh = MeshObject->GetMeshRef();

		if (Mesh.HasAttributes() && Mesh.Attributes()->NumWeightLayers() > 0)
		{
			// Filter down to channels that the Op can actually project -- i.e., channels with valid name, that exist on the cached mesh
			return AttributeWeightChannels.FilterByPredicate(
				[&Mesh](const FSculptLayerModifierWeightAttributeEntry& Entry) -> bool
				{
					return Entry.Method != ESculptLayerSetWeightChannelMethod::Disabled &&
						!Entry.ChannelName.IsNone() &&
						FindMeshWeightAttributeByName(Mesh, Entry.ChannelName) != nullptr;
				});
		}
	}

	return {};
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UProjectMeshLayersModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshProjectSculptLayersModifierLocals;

	if (!ensure(MeshInstance))
	{
		return nullptr;
	}

	TConstArrayView<double> ActiveLayerWeights = GetActiveLayerWeights();
	TArray<FSculptLayerModifierWeightAttributeEntry> WeightChannels = GetActiveWeightChannels();

	const int32 NumLayers = FMath::Min(GetProjectionMeshNumLayers(), ActiveLayerWeights.Num() + 1);
	
	if (NumLayers < 2 && WeightChannels.IsEmpty())
	{
		return nullptr;
	}

	TSharedPtr<FBackgroundOp> Op = AllocateBackgroundOp<FBackgroundOp>(GetFName(), MeshInstance.ToSharedRef());
	Op->LayerWeights = ActiveLayerWeights;

	Op->AttributeWeightChannels = MoveTemp(WeightChannels);

	// if there aren't at least two sculpt layers on the mesh, and there aren't active attribute weight channels, there's no sculpting to do
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->bUseBoundaryFalloff = bUseBoundaryFalloff;
	Op->BoundaryFalloffDistanceFactor = BoundaryFalloffDistanceFactor;
	Op->ProjectMethod = ProjectMethod;
	Op->bSculptAbsolutePositions = bSculptAbsolutePositions;
	Op->MaxClosestPointDistance = MaxClosestPointDistance;
	Op->MeshToWorld = GetComponentTransform();
	Op->VerticalExtentUp = VerticalExtentUp;
	Op->VerticalExtentDown = VerticalExtentDown;
	Op->ProjectionTransform = GetProjectionTransform();

	return Op;
}

void UProjectMeshLayersModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UProjectMeshLayersModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	if (!MeshInstance)
	{
		return;
	}

	const int32 NumLayers = FMath::Min(GetProjectionMeshNumLayers(), LayerWeights.Num() + 1);
	TArray<FSculptLayerModifierWeightAttributeEntry> ActiveWeightChannels = GetActiveWeightChannels();

	// if there aren't at least two sculpt layers on the mesh, and there aren't active attribute weight channels, there's no sculpting to do
	if (NumLayers < 2 && ActiveWeightChannels.IsEmpty())
	{
		return;
	}

	Dependencies += MeshInstance->GetHash();
	Dependencies += ComputeCombinedBounds();
	Dependencies += ProjectMethod;
	if (ProjectMethod == ESculptLayerProjectMethod::ClosestPoints)
	{
		Dependencies += bUseBoundaryFalloff;
		if (bUseBoundaryFalloff)
		{
			Dependencies += BoundaryFalloffDistanceFactor;
		}
		Dependencies += MaxClosestPointDistance;
	}
	else // ProjectMethod == ESculptLayerProjectMethod::FixedDirection
	{
		Dependencies += VerticalExtentDown;
		Dependencies += VerticalExtentUp;
	}
	Dependencies += bSculptAbsolutePositions;

	Dependencies += LayerWeights;

	for (const FSculptLayerModifierWeightAttributeEntry& Entry : ActiveWeightChannels)
	{
		if (Entry.ChannelName.IsNone())
		{
			continue;
		}
		Dependencies += Entry.ChannelName;
		Dependencies += Entry.Method;
		Dependencies += Entry.BlendWeight;
		Dependencies += Entry.bScaleSculptLayers;
		Dependencies += Entry.bClampRange;
		Dependencies += Entry.bInternalChannel;
	}
}

void MegaMeshProjectSculptLayersModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView & InMeshView,
	const FTransform3d& MegameshTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UProjectMeshLayersModifier::ApplyModifications);

	using namespace Geometry;

	const FDynamicMesh3& Mesh = MeshInstance->GetMesh();

	struct FWeightChannelInfo
	{
		const FDynamicMeshWeightAttribute* MeshAttribute = nullptr;
		float Scale = 1.0f;
		bool bScaleSculptLayers = false, bClampRange = false, bInternalChannel = true;
	};
	// Find all the channels we need to transfer from the mesh, and keep separate arrays per transfer method
	TArray<FWeightChannelInfo> MeshSetChannels, MeshAddChannels;
	for (const FSculptLayerModifierWeightAttributeEntry& Entry : AttributeWeightChannels)
	{
		if (const FDynamicMeshWeightAttribute* FoundChannel = FindMeshWeightAttributeByName(Mesh, Entry.ChannelName))
		{
			if (Entry.Method == ESculptLayerSetWeightChannelMethod::Set)
			{
				MeshSetChannels.Add(FWeightChannelInfo{FoundChannel, Entry.BlendWeight, Entry.bScaleSculptLayers, Entry.bClampRange, Entry.bInternalChannel });
			}
			else if (Entry.Method == ESculptLayerSetWeightChannelMethod::Add)
			{
				MeshAddChannels.Add(FWeightChannelInfo{FoundChannel, Entry.BlendWeight, Entry.bScaleSculptLayers, Entry.bClampRange, Entry.bInternalChannel });
			}
		}
	}

	const FDynamicMeshSculptLayers* SculptLayers = Mesh.Attributes()->GetSculptLayers();
	const int32 NumLayers = FMath::Min(LayerWeights.Num(), SculptLayers->NumLayers() - 1);

	auto GetBaryOffset = [SculptLayers, NumLayers, this](const FIndex3i& Tri, const FVector3d& BaryCoords) -> FVector3d
		{
			FVector3d OffsetsSum(0);
			for (int32 LayerWtIdx = 0; LayerWtIdx < NumLayers; ++LayerWtIdx)
			{
				const FDynamicMeshSculptLayerAttribute* Layer = SculptLayers->GetLayer(LayerWtIdx + 1);
				double LayerWt = LayerWeights[LayerWtIdx];
				for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
				{
					FVector3d Offset;
					Layer->GetValue(Tri[SubIdx], Offset);
					OffsetsSum += LayerWt * BaryCoords[SubIdx] * Offset;
				}
			}
			return OffsetsSum;
		};

	// Helper lambda to do all attribute weight channel calculations for a given vertex.
	// Returns the scale factor that the channels apply to sculpt layers.
	auto TransferAttribWeights = [&InMeshView, &MeshSetChannels, &MeshAddChannels]
		(int32 TargetVertexIndex, const FIndex3i& SourceTri, const FVector3d& BaryCoords, double ScaleFactor = 1.0) -> double
		{
			auto GetBaryWt = [](const FDynamicMeshWeightAttribute* WtChannel, const FIndex3i& SourceTri, const FVector3d& BaryCoords) -> float
				{
					float InterpWt = 0;
					for (int32 SubIdx = 0; SubIdx < 3; ++SubIdx)
					{
						float Wt;
						WtChannel->GetValue(SourceTri[SubIdx], &Wt);
						InterpWt += Wt * BaryCoords[SubIdx];
					}
					return InterpWt;
				};

			double ScaleSculpt = 1.0;

			auto ApplyWeight = [&ScaleSculpt, &InMeshView, TargetVertexIndex]
			(const FWeightChannelInfo& Info, float SetWt) -> void
			{
				if (Info.bClampRange)
				{
					SetWt = FMath::Clamp(SetWt, 0.f, 1.f);
				}
				if (!Info.bInternalChannel)
				{
					InMeshView.SetVertexAttributeWeight(Info.MeshAttribute->GetName(), TargetVertexIndex, SetWt);
				}
				if (Info.bScaleSculptLayers)
				{
					ScaleSculpt *= SetWt;
				}
			};
			for (const FWeightChannelInfo& Info : MeshSetChannels)
			{
				float SetWt = GetBaryWt(Info.MeshAttribute, SourceTri, BaryCoords);
				float PrevWt = Info.bInternalChannel ? 0.f : InMeshView.GetVertexAttributeWeight(Info.MeshAttribute->GetName(), TargetVertexIndex);
				// In 'set' mode, scaling down the weight's contribution is implemented as a lerp to previous value
				ApplyWeight(Info, FMath::Lerp(PrevWt, SetWt, ScaleFactor* Info.Scale));
			}
			for (const FWeightChannelInfo& Info : MeshAddChannels) 
			{
				float AddWt = GetBaryWt(Info.MeshAttribute, SourceTri, BaryCoords) * Info.Scale * ScaleFactor;
				float PrevWt = Info.bInternalChannel ? 0.f : InMeshView.GetVertexAttributeWeight(Info.MeshAttribute->GetName(), TargetVertexIndex);
				ApplyWeight(Info, AddWt + PrevWt);
			}
			return ScaleSculpt;
		};

	auto ToAbsolutePositionIfNeeded = [this, MegameshTransform]
	(FVector3d& InOutBasePos, const FVector3d& MeshLocalHitPos, double WithScale) -> void
		{
			if (bSculptAbsolutePositions)
			{
				FVector3d TransformedHitPos = MegameshTransform.InverseTransformPosition(MeshToWorld.TransformPosition(MeshLocalHitPos));
				InOutBasePos = FMath::Lerp(InOutBasePos, TransformedHitPos, FMath::Clamp(WithScale, 0, 1));
			}
		};

	const FDynamicMeshAABBTree3& AABBTree = MeshInstance->GetAABBTree();

	if (ProjectMethod == ESculptLayerProjectMethod::ClosestPoints)
	{
		const double MeshSpaceSearchDist = MaxClosestPointDistance / FMath::Max(FMathd::ZeroTolerance, FMath::Abs(MeshToWorld.GetMinimumAxisScale()));
		ParallelFor(InMeshView.VertexCount(), 
			[this, &Mesh, &AABBTree, &InMeshView, &MegameshTransform, MeshSpaceSearchDist, &GetBaryOffset, &TransferAttribWeights, &ToAbsolutePositionIfNeeded]
			(int32 VertexIndex) -> void
			{
				FVector3d MeshVertPosition = InMeshView.GetVertexPos(VertexIndex);
				FVector3d WorldOriginalVertPosition = MegameshTransform.TransformPosition(MeshVertPosition);

				VertexOntoMeshClosestPoint(WorldOriginalVertPosition,
					Mesh, MeshToWorld,
					MeshSpaceSearchDist, bUseBoundaryFalloff, BoundaryFalloffDistanceFactor, AABBTree,
					[&GetBaryOffset, this, &Mesh, VertexIndex, &MeshVertPosition, &InMeshView, &MegameshTransform, &TransferAttribWeights, &ToAbsolutePositionIfNeeded]
					(double ScaleFactor, int32 MeshHitTID, const FVector3d& MeshHitBaryCoords, const FVector3d& MeshLocalHitPos)
					{
						FIndex3i Tri = Mesh.GetTriangle(MeshHitTID);
						double ScaleSculptLayers = TransferAttribWeights(VertexIndex, Tri, MeshHitBaryCoords, ScaleFactor);
						FVector3d OffsetsSum = MeshToWorld.TransformVector(GetBaryOffset(Tri, MeshHitBaryCoords)) * ScaleFactor * ScaleSculptLayers;
						FVector3d BasePos = MeshVertPosition;
						ToAbsolutePositionIfNeeded(BasePos, MeshLocalHitPos, ScaleFactor * ScaleSculptLayers);
						InMeshView.SetVertexPos(VertexIndex, BasePos + MegameshTransform.InverseTransformVector(OffsetsSum));
					});
			});
	}
	else // ESculptLayerProjectMethod::FixedDirection
	{
		const FAxisAlignedBox3d ProjectionSpaceBounds = MegaMeshProjectSculptLayersModifierLocals::GetProjectionBounds(
			AABBTree.GetBoundingBox(), ProjectionTransform, MeshToWorld, VerticalExtentUp, VerticalExtentDown);
		const FVector LocalProjectionDirection = GetMeshLocalProjectionDirection(MeshToWorld, ProjectionTransform);

		ParallelFor(InMeshView.VertexCount(), 
			[this, &Mesh, &AABBTree, ProjectionSpaceBounds, &InMeshView, &MegameshTransform, &LocalProjectionDirection, &GetBaryOffset, &TransferAttribWeights, &ToAbsolutePositionIfNeeded]
			(int32 VertexIndex) -> void
			{
				FVector3d MeshVertPosition = InMeshView.GetVertexPos(VertexIndex);
				FVector3d WorldOriginalVertPosition = MegameshTransform.TransformPosition(MeshVertPosition);
				ProjectVertexOntoMesh(WorldOriginalVertPosition, LocalProjectionDirection,
					Mesh, MeshToWorld, AABBTree, ProjectionTransform, ProjectionSpaceBounds,
					[&GetBaryOffset, this, &Mesh, VertexIndex, &MeshVertPosition, &InMeshView, &MegameshTransform, &TransferAttribWeights, &ToAbsolutePositionIfNeeded]
					(int32 MeshHitTID, const FVector3d& MeshHitBaryCoords, const FVector3d& MeshLocalHitPos)
					{
						FIndex3i Tri = Mesh.GetTriangle(MeshHitTID);
						double ScaleSculptLayers = TransferAttribWeights(VertexIndex, Tri, MeshHitBaryCoords);
						FVector3d OffsetsSum = MeshToWorld.TransformVector(GetBaryOffset(Tri, MeshHitBaryCoords)) * ScaleSculptLayers;
						FVector3d BasePos = MeshVertPosition;
						ToAbsolutePositionIfNeeded(BasePos, MeshLocalHitPos, ScaleSculptLayers);
						InMeshView.SetVertexPos(VertexIndex, BasePos + MegameshTransform.InverseTransformVector(OffsetsSum));
					});
			});
	}
}

void UProjectMeshLayersModifier::InitializeModifier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UProjectMeshLayersModifier::InitializeModifier);

	Super::InitializeModifier();

	BuildCachedData();
	ComputeMaxLayerOffsets();
}


void UProjectMeshLayersModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	const FColor EditBoundsColor = FColor::Green;
	const FColor ProjectMeshBoundsColor = FColor::Yellow;
	constexpr float RectangleThickness = 3;
	constexpr float BoundsThickness = 1;
	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	if (bDrawAffectedBox)
	{
		FTransform ProjectionTransform = GetProjectionTransform();
		Geometry::FAxisAlignedBox3d ProjectionSpaceBounds = GetProjectionEditBounds();

		DrawWireBox(PDI, ProjectionTransform.ToMatrixWithScale(), FBox(ProjectionSpaceBounds),
			EditBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
	}

	if (MeshInstance.IsValid())
	{
		if (bDrawSculptSourceBounds)
		{
			const Geometry::FAxisAlignedBox3d MeshBounds = MeshInstance->GetBounds();
			if (!MeshBounds.IsEmpty())
			{
				FTransform ModifierTransform = GetComponentTransform();
				DrawWireBox(PDI, ModifierTransform.ToMatrixWithScale(), FBox(MeshBounds),
					ProjectMeshBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
			}
		}

		if (bDrawDebugMesh)
		{
			const FDynamicMesh3& Mesh = MeshObject->GetMeshRef();
			FTransform ModifierTransform = GetComponentTransform();
			for (int32 Eid : Mesh.EdgeIndicesItr())
			{
				auto Vids = Mesh.GetEdgeV(Eid);
				PDI->DrawLine(
					ModifierTransform.TransformPosition(Mesh.GetVertex(Vids.A)),
					ModifierTransform.TransformPosition(Mesh.GetVertex(Vids.B)),
					FLinearColor::White, SDPG_World, 2.0f, 0, true);
			}
		}
	}
}

FTransform UProjectMeshLayersModifier::GetProjectionTransform() const
{
	// projection transform for mesh layers is currently always the component world transform
	return GetComponentTransform();
}

Geometry::FAxisAlignedBox3d UProjectMeshLayersModifier::GetProjectionBounds(
	const Geometry::FAxisAlignedBox3d& InputBounds,
	const FTransform& InProjectionTransform) const
{
	return MegaMeshProjectSculptLayersModifierLocals::GetProjectionBounds(InputBounds, InProjectionTransform,
	                                                                      GetComponentTransform(), VerticalExtentUp, VerticalExtentDown);
}

Geometry::FAxisAlignedBox3d UProjectMeshLayersModifier::GetProjectionEditBounds() const
{
	if (ProjectMethod == ESculptLayerProjectMethod::ClosestPoints)
	{
		FVector3d AbsEditExtents = EditVolumeExtents.GetAbs();
		return Geometry::FAxisAlignedBox3d(-AbsEditExtents, AbsEditExtents);
	}
	else // ProjectMethod == ESculptLayerProjectMethod::FixedDirection
	{
		FVector2d AbsEditExtents = EditExtents.GetAbs();
		return Geometry::FAxisAlignedBox3d(
			FVector3d(-AbsEditExtents.X, -AbsEditExtents.Y, -VerticalExtentDown),
			FVector3d(AbsEditExtents.X, AbsEditExtents.Y, VerticalExtentUp)
		);
	}
}

FGuid UProjectMeshLayersModifier::GetCodeVersionKey() const
{
	return MegaMeshProjectSculptLayersModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

bool UProjectMeshLayersModifier::IsTemporarilyDisabledInEditor() const
{
	return Super::IsTemporarilyDisabledInEditor() || bDisabledByCode;
}

Tasks::FTask UProjectMeshLayersModifier::GetAsyncPrepareResourcesTask() const
{
	return MeshInstance->GetAsyncInitTask();
}

void UProjectMeshLayersModifier::SetDisabledByCode(bool bDisabledByCodeIn)
{
	if (bDisabledByCode == bDisabledByCodeIn)
	{
		return;
	}
	bDisabledByCode = bDisabledByCodeIn;

	// ComputeBounds() will give us empty bounds once we're disabled. Currently, previous bounds
	//  are automatically added by the OnChanged call.
	OnChanged(ComputeBounds(), bDisabledByCodeIn ? EChangeType::TransientStateChange : EChangeType::StateChange);
}

void UProjectMeshLayersModifier::ResetForReuse()
{
	if (ensure(MeshObject))
	{
		MeshObject->Reset();
	}
}

bool UProjectMeshLayersModifier::IsUsed() const
{
	return MeshObject && !MeshObject->IsEmpty();
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
