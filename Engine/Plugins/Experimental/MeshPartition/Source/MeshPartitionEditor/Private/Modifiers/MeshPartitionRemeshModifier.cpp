// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionRemeshModifier.h"
#include "MeshPartitionMeshView.h"
#include "PrimitiveDrawingUtils.h"
#include "SubRegionRemesher.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "MeshConstraintsUtil.h"
#include "Ops/MeshPartitionRemeshOp.h"
#include "Ops/MeshPartitionTessellateOp.h"

namespace UE::MeshPartition
{
void URemeshModifierBase::SetCurrentOperation(const ERemeshModifierOperation InOperation)
{
	if (CurrentOperation != InOperation)
	{
		CurrentOperation = InOperation;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetBoundaryMode(const EMegaMeshRemeshModifierBoundaryMode InMode)
{
	if (BoundaryMode != InMode)
	{
		BoundaryMode = InMode;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetDisallowUnsafeBoundaryEdits(const bool bDisallow)
{
	if (bDisallowUnsafeBoundaryEdits != bDisallow)
	{
		bDisallowUnsafeBoundaryEdits = bDisallow;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetTargetEdgeLength(const float InLength)
{
	if (TargetEdgeLength != InLength)
	{
		TargetEdgeLength = InLength;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetRemeshIterations(const int32 InIterations)
{
	if (RemeshIterations != InIterations)
	{
		RemeshIterations = InIterations;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetSmoothingStrength(const float InStrength)
{
	if (SmoothingStrength != InStrength)
	{
		SmoothingStrength = InStrength;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetSmoothingType(const ERemeshSmoothingType InType)
{
	if (SmoothingType != InType)
	{
		SmoothingType = InType;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetPreserveNormalSeams(const bool bPreserve)
{
	if (bPreserveNormalSeams != bPreserve)
	{
		bPreserveNormalSeams = bPreserve;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetSharpEdgeAngleThreshold(const float InThreshold)
{
	if (SharpEdgeAngleThreshold != InThreshold)
	{
		SharpEdgeAngleThreshold = FMath::Clamp(InThreshold, 0.0f, 180.0f);
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetProjectToInputMesh(const bool bProject)
{
	if (bProjectToInputMesh != bProject)
	{
		bProjectToInputMesh = bProject;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetTessellationMethod(const EMegaMeshRemeshModifierTessellateMethod InMethod)
{
	if (TessellationMethod != InMethod)
	{
		TessellationMethod = InMethod;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetUseTargetEdgeLength(const bool bUse)
{
	if (bUseTargetEdgeLength != bUse)
	{
		bUseTargetEdgeLength = bUse;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetTessellationTargetEdgeLength(const float InLength)
{
	if (TessellationTargetEdgeLength != InLength)
	{
		TessellationTargetEdgeLength = InLength;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetTessellationLevel(const int32 InLevel)
{
	if (TessellationLevel != InLevel)
	{
		TessellationLevel = InLevel;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetMaxTessellationLevel(const int32 InMaxLevel)
{
	if (MaxTessellationLevel != InMaxLevel)
	{
		MaxTessellationLevel = FMath::Clamp(InMaxLevel, 0, 10);
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetPostProcessingIterations(const int32 InIterations)
{
	if (PostProcessingIterations != InIterations)
	{
		PostProcessingIterations = FMath::Clamp(InIterations, 0, 40);
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetVertexSmoothing(const bool bSmoothing)
{
	if (bVertexSmoothing != bSmoothing)
	{
		bVertexSmoothing = bSmoothing;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetTessellateSmoothingStrength(const float InStrength)
{
	if (TessellateSmoothingStrength != InStrength)
	{
		TessellateSmoothingStrength = FMath::Clamp(InStrength, 0.0f, 1.0f);
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetResampleUVs(const bool bResample)
{
	if (bResampleUVs != bResample)
	{
		bResampleUVs = bResample;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetEdgeFlips(const bool bFlips)
{
	if (bEdgeFlips != bFlips)
	{
		bEdgeFlips = bFlips;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetUseDensityWeightChannel(const bool bUse)
{
	if (bUseDensityWeightChannel != bUse)
	{
		bUseDensityWeightChannel = bUse;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetDensityWeightChannelName(const FChannelName& InName)
{
	if (DensityWeightChannelName.GetName() != InName.GetName())
	{
		DensityWeightChannelName = InName;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::SetRelativeDensity(const float InDensity)
{
	if (RelativeDensity != InDensity)
	{
		RelativeDensity = FMath::Clamp(InDensity, -2.0f, 2.0f);
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

void URemeshModifierBase::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	Super::GatherDependencies(Dependencies);

	Dependencies += CurrentOperation;

	Dependencies += BoundaryMode;
	Dependencies += bDisallowUnsafeBoundaryEdits;
	Dependencies += bDisallowSafeEditsOutsideCoverage;
	Dependencies += TargetEdgeLength;
	Dependencies += RemeshIterations;
	Dependencies += SmoothingStrength;
	Dependencies += SmoothingType;
	Dependencies += bPreserveNormalSeams;
	Dependencies += SharpEdgeAngleThreshold;
	Dependencies += bProjectToInputMesh;
	Dependencies += bUseDensityWeightChannel;
	Dependencies += DensityWeightChannelName;
	Dependencies += RelativeDensity;
	Dependencies += bUseWeightThreshold;
	Dependencies += MinWeightThreshold;

	Dependencies += TessellationMethod;
	Dependencies += bUseTargetEdgeLength;
	Dependencies += TessellationTargetEdgeLength;
	Dependencies += TessellationLevel;
	Dependencies += MaxTessellationLevel;
	Dependencies += PostProcessingIterations;
	Dependencies += bVertexSmoothing;
	Dependencies += bResampleUVs;
	Dependencies += bEdgeFlips;
	Dependencies += TessellateSmoothingStrength;
}


URemeshModifier::URemeshModifier()
{
}

void URemeshModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	const FName ParentPreviousDefaultType = TEXT("Misc");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("Remesh");
		SetType(PreviousModifierDefaultType);
	}
}

Geometry::FAxisAlignedBox3d URemeshModifier::GetLocalCoverage() const
{
	return Geometry::FAxisAlignedBox3d(-UnscaledCoverage / 2, UnscaledCoverage / 2);
}

void URemeshModifier::SetUnscaledCoverage(const FVector3d& InUnscaledCoverage)
{
	if (UnscaledCoverage != InUnscaledCoverage)
	{
		UnscaledCoverage = InUnscaledCoverage;
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
}

FVector3d URemeshModifier::GetUnscaledCoverage() const
{
	return UnscaledCoverage;
}

TArray<FBox> URemeshModifier::ComputeBounds() const
{
	Geometry::FAxisAlignedBox3d Coverage = GetLocalCoverage();
	return { FBox(Geometry::FAxisAlignedBox3d(Coverage, GetComponentTransform())) };
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> URemeshModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	// When building the interactive base, we avoid applying remesh modifiers by default to speed-up the initial build.
	if ((InBuildType == MeshPartition::EBuildType::InteractiveBase) || (InBuildType == MeshPartition::EBuildType::SimplifiedPreviewSection))
	{
		TSharedPtr<FPassthroughBackgroundOp> PassthroughOp = MakeShared<FPassthroughBackgroundOp>(GetFName());
		PassthroughOp->GlobalBounds = ComputeCombinedBounds();
		return PassthroughOp;
	}

	TSharedPtr<MeshPartition::IModifierBackgroundOp> OutOp;

	switch (CurrentOperation)
	{
	case MeshPartition::ERemeshModifierOperation::Remesh:
	{
		TSharedPtr<FMegaMeshRemeshBackgroundOp> Op = MakeShared<FMegaMeshRemeshBackgroundOp>(GetFName());
		Op->GlobalBounds = ComputeCombinedBounds();
		Op->LocalCoverage = GetLocalCoverage();
		Op->ModifierToWorld = GetComponentTransform();
		Op->bComputeNormalSeams = bPreserveNormalSeams;
		Op->NormalSeamDotProductThreshold = FMathf::Cos(SharpEdgeAngleThreshold * FMathf::DegToRad);;
		Op->BoundaryMode = BoundaryMode;
		Op->bDisallowUnsafeBoundaryEdits = bDisallowUnsafeBoundaryEdits;
		Op->bDisallowSafeEditsOutsideCoverage = bDisallowSafeEditsOutsideCoverage;
		Op->TargetEdgeLength = TargetEdgeLength;
		Op->RemeshIterations = RemeshIterations;
		Op->SmoothingStrength = SmoothingStrength;
		Op->SmoothingType = SmoothingType;
		Op->bProjectToInputMesh = bProjectToInputMesh;

		Op->bUseDensityWeightChannel = bUseDensityWeightChannel;
		Op->DensityWeightChannelName = DensityWeightChannelName;
		Op->RelativeDensity = RelativeDensity;
		Op->bUseWeightThreshold = bUseWeightThreshold;
		Op->MinWeightThreshold = MinWeightThreshold;

		OutOp = Op;
	}
	break;
	case MeshPartition::ERemeshModifierOperation::Tessellate:
	{
		TSharedPtr<FMegaMeshTessellateBackgroundOp> Op = MakeShared<FMegaMeshTessellateBackgroundOp>(GetFName());
		Op->GlobalBounds = ComputeCombinedBounds();
		Op->LocalBounds = FBox(GetLocalCoverage());
		Op->bUseTargetEdgeLength = bUseTargetEdgeLength;
		Op->TessellationLevel = TessellationLevel;
		Op->TargetEdgeLength = TessellationTargetEdgeLength;
		Op->MaxTessellationLevel = MaxTessellationLevel;
		Op->TessellationMethod = TessellationMethod;
		Op->ModifierTransform = GetComponentToWorld();
		Op->PostProcessingIterations = PostProcessingIterations;
		Op->bVertexSmoothing = bVertexSmoothing;
		Op->SmoothingStrength = TessellateSmoothingStrength;
		Op->bResampleUVs = bResampleUVs;
		Op->bEdgeFlips = bEdgeFlips;

		Op->bUseDensityWeightChannel = bUseDensityWeightChannel;
		Op->DensityWeightChannelName = DensityWeightChannelName;
		Op->RelativeDensity = RelativeDensity;
		Op->bUseWeightThreshold = bUseWeightThreshold;
		Op->MinWeightThreshold = MinWeightThreshold;

		OutOp = Op;
	}
	break;
	};

	return OutOp;
}


void URemeshModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(URemeshModifier::GatherDependencies);

	MeshPartition::URemeshModifierBase::GatherDependencies(Dependencies);

	Dependencies += ComputeCombinedBounds();

	Geometry::FAxisAlignedBox3d Coverage = GetLocalCoverage();
	Dependencies += Coverage.Min;
	Dependencies += Coverage.Max;
}

void URemeshModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	const FColor LocalBoundsColor = FColor::Yellow;
	const FColor WorldBoundsColor = FColor::Orange;
	constexpr float BoundsThickness = 1;
	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	if (bDrawAffectedBox)
	{
		const Geometry::FAxisAlignedBox3d Coverage = GetLocalCoverage();
		DrawWireBox(PDI, GetComponentTransform().ToMatrixWithScale(), FBox(Coverage),
			LocalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
	}

	if (bWorldBounds)
	{
		const Geometry::FAxisAlignedBox3d WorldBounds = ComputeCombinedBounds();
		DrawWireBox(PDI, FBox(WorldBounds), WorldBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
	}
}

FGuid URemeshModifier::GetCodeVersionKey() const
{
	return FGuid::Combine(FMegaMeshRemeshBackgroundOp::GetCodeVersionKey(), FMegaMeshTessellateBackgroundOp::GetCodeVersionKey());
}
} // namespace UE::MeshPartition