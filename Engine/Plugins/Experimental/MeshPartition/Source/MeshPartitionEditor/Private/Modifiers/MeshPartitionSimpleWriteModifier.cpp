// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionSimpleWriteModifier.h"

#include "Async/ParallelFor.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef
#include "Math/UnrealMathUtility.h" // UE_DOUBLE_KINDA_SMALL_NUMBER

namespace UE::MeshPartition
{
namespace MegaMeshWriteModifierLocals
{
	float VertexSearchCellSize = 0.01f;
	static FAutoConsoleVariableRef CVarVertexSearchCellSize(
		TEXT("MegaMesh.PCGWrite.SearchCellSize"),
		VertexSearchCellSize,
		TEXT("When using the write modifier, how far to look for a match in source position. This should be set "
			"fairly low to avoid considering too many vertices, but high enough to be tolerant to roundoff when "
			"converting to world space."));

	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		FBox GlobalBounds;
		TSharedPtr<const Geometry::TPointHashGrid3<int32, double>> HashGrid;
		TSharedPtr<const TArray<FVector3d>> SourcePositions;
		TSharedPtr<const TArray<FVector3d>> DestPositions;
		TSharedPtr<const TArray<TPair<FName, TArray<float>>>> WeightChannelValues;

		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::USimpleWriteModifier::ApplyModifications);

			if (!ensure(HashGrid && SourcePositions))
			{
				return;
			}

			ParallelFor(InMeshView.VertexCount(), [this, &InMeshView, &InTransform](int32 VertexIndex) 
			{
				FVector3d CurrentWorldPosition = InTransform.TransformPosition(InMeshView.GetVertexPos(VertexIndex));
				
				int32 FoundIndex = HashGrid->FindNearestInRadius(CurrentWorldPosition, VertexSearchCellSize, [this, &CurrentWorldPosition](int32 CandidateIndex)
				{
					return FVector3d::DistSquared(CurrentWorldPosition, (*SourcePositions)[CandidateIndex]);
				}).Key;

				if (FoundIndex == IndexConstants::InvalidID || !ensure(SourcePositions->IsValidIndex(FoundIndex)))
				{
					return;
				}

				if (DestPositions && ensure(DestPositions->IsValidIndex(FoundIndex)))
				{
					InMeshView.SetVertexPos(VertexIndex, InTransform.InverseTransformPosition((*DestPositions)[FoundIndex]));
				}
				for (int32 ChannelIndex = 0; ChannelIndex < WeightChannelValues->Num(); ++ChannelIndex)
				{
					InMeshView.SetVertexAttributeWeight((*WeightChannelValues)[ChannelIndex].Key, VertexIndex,
						(*WeightChannelValues)[ChannelIndex].Value[FoundIndex]);
				}
			});
		}

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override
		{
			AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);

			// Add the used weight layer names
			if (!OutInstanceInfos.IsEmpty() && WeightChannelValues)
			{
				for (const TPair<FName, TArray<float>>& Entry : *WeightChannelValues)
				{
					if (!Entry.Key.IsNone())
					{
						OutInstanceInfos[0].UsedChannels.Emplace(Entry.Key);

						OutInstanceInfos[0].WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;

						// TODO: at the time of writing, the meshview had a bug where reading weights was required to write them, and merge
						//  concerns with another incoming change made it impractical to fix immediately. This line can be removed once fixed.
						OutInstanceInfos[0].ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
					}
				}
			}
		}

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("f6273b64-4fe0-e3c6-ff05-0584da367349"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};
}

USimpleWriteModifier::USimpleWriteModifier()
{
}

TArray<FBox> USimpleWriteModifier::ComputeBounds() const
{
	return TArray<FBox>{ ExpandedBounds };
}


TSharedPtr<const MeshPartition::IModifierBackgroundOp> USimpleWriteModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshWriteModifierLocals;

	if (SourcePositions.IsEmpty())
	{
		return nullptr;
	}

	// Create the cached data copies that we hand over to the background op
	if (!HashGrid_BackgroundOp)
	{
		SourcePositions_BackgroundOp = MakeShared<TArray<FVector3d>>(SourcePositions);
		if (DestPositions.IsSet())
		{
			DestPositions_BackgroundOp = MakeShared<TArray<FVector3d>>(*DestPositions);
		}

		TSharedPtr<Geometry::TPointHashGrid3<int32, double>> HashGridMutable = 
			MakeShared<Geometry::TPointHashGrid3<int32, double>>(VertexSearchCellSize, IndexConstants::InvalidID);
		HashGridMutable->Reserve(SourcePositions.Num());
		for (int32 Index = 0; Index < SourcePositions.Num(); ++Index)
		{
			HashGridMutable->InsertPointUnsafe(Index, SourcePositions[Index]);
		}

		TSharedPtr<TArray<TPair<FName, TArray<float>>>> WeightChannelValuesMutableCopy = MakeShared<TArray<TPair<FName, TArray<float>>>>();
		WeightChannelValuesMutableCopy->Reserve(WeightChannelValues.Num());
		for (const MeshPartition::FSimpleWriteModifier_ChannelValues& ChannelValues : WeightChannelValues)
		{
			WeightChannelValuesMutableCopy->Emplace(ChannelValues.Name, ChannelValues.Values);
		}

		WeightChannelValues_BackgroundOp = WeightChannelValuesMutableCopy;
		HashGrid_BackgroundOp = HashGridMutable;
	}

	TSharedPtr<FBackgroundOp> Op = MakeShared<FBackgroundOp>(GetFName());
	Op->GlobalBounds = ExpandedBounds;
	Op->HashGrid = HashGrid_BackgroundOp;
	Op->SourcePositions = SourcePositions_BackgroundOp;
	Op->DestPositions = DestPositions_BackgroundOp;
	Op->WeightChannelValues = WeightChannelValues_BackgroundOp;

	return Op;
}

void USimpleWriteModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USimpleWriteModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	Dependencies += SourcePositions;

	if (DestPositions.IsSet())
	{
		Dependencies += *DestPositions;
	}
	for (const MeshPartition::FSimpleWriteModifier_ChannelValues& Layer : WeightChannelValues)
	{
		Dependencies += Layer.Name;
		Dependencies += Layer.Values;
	}
}

void USimpleWriteModifier::SetDisabledByCode(bool bDisabledByCodeIn)
{
	if (bDisabledByCode == bDisabledByCodeIn)
	{
		return;
	}
	bDisabledByCode = bDisabledByCodeIn;

	// ComputeBounds() will give us empty bounds once we're disabled. Currently, previous bounds
	//  are automatically added by the OnChanged call.
	OnChanged(ComputeBounds(), bDisabledByCodeIn ? EChangeType::TransientStateChange : EChangeType::StateChange);
}

void USimpleWriteModifier::ResetForReuse()
{
	ClearPoints();
}

bool USimpleWriteModifier::IsUsed() const
{
	return !SourcePositions.IsEmpty();
}

void USimpleWriteModifier::ReinitializePoints(
	const TArray<FVector3d>& SourcePositionsIn,
	const TArray<FVector3d>* DestPositionsIn,
	const TArray<TPair<FName, TArray<float>>>& WeightChannelValuesIn)
{
	SourcePositions = SourcePositionsIn;
	if (DestPositionsIn)
	{
		DestPositions = *DestPositionsIn;
		if (!ensure(DestPositions->Num() == SourcePositions.Num()))
		{
			int32 PreviousNumDestPositions = DestPositions->Num();
			DestPositions->SetNum(SourcePositions.Num());
			for (int32 i = PreviousNumDestPositions; i < SourcePositions.Num(); ++i)
			{
				(*DestPositions)[i] = SourcePositions[i];
			}
		}
	}

	GlobalBounds = FBox();
	for (const FVector3d& Position : SourcePositions)
	{
		GlobalBounds += Position;
	}
	if (DestPositions)
	{
		for (const FVector3d& Position : *DestPositions)
		{
			GlobalBounds += Position;
		}
	}
	ExpandedBounds = GlobalBounds.ExpandBy(UE_DOUBLE_KINDA_SMALL_NUMBER);

	WeightChannelValues.Reset();
	for (const TPair<FName, TArray<float>>& ChannelIn : WeightChannelValuesIn)
	{
		if (!ensure(!ChannelIn.Key.IsNone()))
		{
			continue;
		}

		// Make sure the same channel name doesn't show up twice
		MeshPartition::FSimpleWriteModifier_ChannelValues* ExistingChannel = WeightChannelValues.FindByPredicate(
			[&ChannelIn](const MeshPartition::FSimpleWriteModifier_ChannelValues& Candidate) { return Candidate.Name == ChannelIn.Key; });
		if (ExistingChannel)
		{
			ensure(false);
			continue;
		}

		ExistingChannel = &WeightChannelValues.Emplace_GetRef(ChannelIn.Key, TArray<float>());		
		ExistingChannel->Values.Append(ChannelIn.Value);
		if (!ensure(ExistingChannel->Values.Num() == SourcePositions.Num()))
		{
			ExistingChannel->Values.SetNumZeroed(SourcePositions.Num());
		}
	}
	
	ResetCachedData();
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void USimpleWriteModifier::ClearPoints()
{
	SourcePositions.Reset();
	DestPositions.Reset();
	WeightChannelValues.Reset();
	GlobalBounds = FBox();
	ExpandedBounds = GlobalBounds;

	ResetCachedData();
}

void USimpleWriteModifier::ResetCachedData()
{
	SourcePositions_BackgroundOp.Reset();
	DestPositions_BackgroundOp.Reset();
	WeightChannelValues_BackgroundOp.Reset();
	HashGrid_BackgroundOp.Reset();
}

FGuid USimpleWriteModifier::GetCodeVersionKey() const
{
	return MegaMeshWriteModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

bool USimpleWriteModifier::IsTemporarilyDisabledInEditor() const
{
	return Super::IsTemporarilyDisabledInEditor() || bDisabledByCode;
}
} // namespace UE::MeshPartition