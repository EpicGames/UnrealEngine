// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE::Geometry
{
	class FDynamicMesh3;
	template<typename T> class TMeshAABBTree3;
	typedef TMeshAABBTree3<FDynamicMesh3> FDynamicMeshAABBTree3;
}

namespace UE::MeshPartition
{
	// Holds surface patch data that gets generated on a background thread and potentially passed to newly created background
	// threads if that work doesn't change. Also used for debug visualization.
	struct FSplineCachedSurfaceData
	{
		TSharedPtr<Geometry::FDynamicMesh3> MeshedLoop;
		TSharedPtr<Geometry::FDynamicMeshAABBTree3> MeshedLoopSpatial;

		// In cases where the same cache data is needed for multiple applications, this bool guards it
		//  against being initialized more than once.
		std::atomic<bool> bNeedsInitializing = false;

		// This is only set to true once the struct is fully initialized. Once true, the struct is not
		//  touched (and is therefore safe to use on all threads).
		std::atomic<bool> bInitializationCompleted = false;
	};
} // namespace UE::MeshPartition
