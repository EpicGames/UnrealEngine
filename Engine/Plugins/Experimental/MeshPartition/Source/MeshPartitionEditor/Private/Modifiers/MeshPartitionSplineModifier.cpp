// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionSplineModifier.h"

#include "Modifiers/MeshPartitionSplineCachedData.h"
#include "BoxTypes.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionMeshView.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Components/SplineComponent.h"
#include "MeshPartitionEditorComponent.h"
#include "Solvers/ConstrainedMeshSmoother.h"
#include "Solvers/ConstrainedMeshDeformer.h"
#include "ConstrainedDelaunay2.h"
#include "Generators/FlatTriangulationMeshGenerator.h"
#include "Remesher.h"
#include "MeshConstraintsUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Algo/Reverse.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "CleaningOps/RemeshMeshOp.h"
#include "Curves/CurveFloat.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "Misc/TransactionObjectEvent.h"
#include "PrimitiveDrawInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "SplineUtil.h"

namespace UE::MeshPartition
{
namespace MegaMeshSplineModifierLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		FTransform ComponentTransform;
		FBox LocalBounds;

		FSplineCurves SplineCurves;
		FTransform SplineTransform;
		FVector3d SplineDefaultUpVector;

		bool bMeshClosedInterior = false;
		TArray<FVector3d> WorldSampledLoopPoints;
		double MeshedInteriorNumTriTarget = 1000;

		MeshPartition::ESplineModifierInteriorSmoothMode InteriorSmoothMode;
		
		double FalloffDistance;
		float PlateauDistance = 0.0f;
		bool bUseSplineScaleForFalloff;
		bool bUseSplineScaleForPlateau;
		bool bUseEdgeFalloffCurve;
		TUniquePtr<FRichCurve> EdgeFalloffCurve;
		bool bSplineIsLeftOfFalloffCurve;
		FVector ProjectionDirection;
		bool bUseNearestSplineFrameForDisplacement;
		bool bProjectSplineToSurface;

		ESplineModifierBlendMode BlendMode;

		ESplineModifierWriteMode WriteMode = ESplineModifierWriteMode::Positions | ESplineModifierWriteMode::Weights;

		struct FWeightEntry
		{
			FName WeightChannelName;
			double Value;
			ESplineWeightBlendMode BlendMode;
		};
		TArray<FWeightEntry> WeightEntries;

		// When CachedDataIn is set, we want to use it. When ExternalCachedDataToInitialize is set, we want to
		//  initialize it. Only one of them should be set.
		TSharedPtr<const FSplineCachedSurfaceData> CachedDataIn;
		TSharedPtr<FSplineCachedSurfaceData> ExternalCachedDataToInitialize;

		TWeakObjectPtr<const USplineModifier> ParentModifier = nullptr;

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("147a0079-e4c0-4c77-9a74-6b99a4b9b621"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override 
		{ 
			return false;
		}

	private:
		bool ShouldFillInterior() const;
		void ComputeLoopMesh(FSplineCachedSurfaceData& CachedData) const;
		TUniquePtr<Geometry::FDynamicMeshOperator> CreateRemeshingOperator(FSplineCachedSurfaceData& CachedData, double TargetEdgeLength) const;
	};

	inline double BlendValues(
		const double CurrentValue, const double NewValue, const double Alpha,
		const MeshPartition::ESplineWeightBlendMode BlendMode)
	{
		double DesiredValue = NewValue;
		switch (BlendMode)
		{
		case MeshPartition::ESplineWeightBlendMode::AlphaBlend:
			// DesiredValue is already NewValue; Lerp(Current, New, Alpha) gives standard alpha blend.
			break;
		case MeshPartition::ESplineWeightBlendMode::Additive:
			DesiredValue = CurrentValue + NewValue;
			break;
		case MeshPartition::ESplineWeightBlendMode::Min:
			DesiredValue = FMath::Min(CurrentValue, NewValue);
			break;
		case MeshPartition::ESplineWeightBlendMode::Max:
			DesiredValue = FMath::Max(CurrentValue, NewValue);
			break;
		default:
			ensure(false);
		}

		return FMath::Lerp(CurrentValue, DesiredValue, Alpha);
	}
}

USplineModifier::USplineModifier()
	: SplineDelegateHelper(MakeUnique<FSplineDelegateHelper>(this, [this]()
		{
			CurrentCachedData.Reset();
			OnChanged(ComputeBounds(), EChangeType::StateChange);
		}))
{
}

USplineModifier::~USplineModifier() = default;

void USplineModifier::PreEditChange(FProperty* InPropertyAboutToChange)
{
	Super::PreEditChange(InPropertyAboutToChange);

	if (InPropertyAboutToChange->GetFName() == GET_MEMBER_NAME_CHECKED(USplineModifier, EdgeFalloffCurve))
	{
		if (EdgeFalloffCurve && EdgeFalloffUpdateHandle.IsValid())
		{
			EdgeFalloffCurve->OnUpdateCurve.Remove(EdgeFalloffUpdateHandle);
			EdgeFalloffUpdateHandle.Reset();
		}
	}
	
}

void USplineModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USplineModifier, EdgeFalloffCurve) && EdgeFalloffCurve != nullptr)
	{
		EdgeFalloffUpdateHandle = EdgeFalloffCurve->OnUpdateCurve.AddUObject(this, &USplineModifier::UpdateWhenEdgeFalloffChanges);
	}
	else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USplineModifier, MeshedInteriorNumTriTarget)
		|| InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USplineModifier, SplinePolygonErrorTolerance))
	{
		CurrentCachedData.Reset();
	}
	else if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(USplineModifier, SplinePtr))
	{
		// Handle when the bound spline component is changed; Register handles unregistering any previously bound spline as well
		CurrentCachedData.Reset();
		SplineDelegateHelper->Register(GetSplineComponent());
	}
}

void USplineModifier::PostEditUndo()
{
	Super::PostEditUndo();

	// Bulk restore may have swapped SplinePtr; refresh cache and delegates.
	CurrentCachedData.Reset();
	SplineDelegateHelper->Register(GetSplineComponent());
}

void USplineModifier::PostLoad()
{
	Super::PostLoad();

	if (SplinePtr.IsNull())
	{
		if (USplineComponent* LegacySplineComponent = Cast<USplineComponent>(SplineRef.GetComponent(GetOwner())))
		{
			SetSplineComponent(LegacySplineComponent);
			SplineRef = FComponentReference();
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (bSingleSidedProjection_DEPRECATED)
	{
		BlendMode = ESplineModifierBlendMode::Max;
		bSingleSidedProjection_DEPRECATED = false;
	}

	if (FalloffPlateauRatio != 0.0f)
	{
		// Old behavior: plateau was a ratio of (scaled) FalloffDistance, carved out of it.
		// New behavior: plateau and falloff are additive (total width = PlateauDistance + FalloffDistance).
		// Adjust both values to preserve the same total width and plateau size.
		// Also match the scale flag so that the plateau continues to scale with the spline
		// the same way it did when it was a ratio of the falloff.
		PlateauDistance = FalloffPlateauRatio * FalloffDistance;
		FalloffDistance -= PlateauDistance;
		bUseSplineScaleForPlateau = bUseSplineScaleForFalloff;
		FalloffPlateauRatio = 0.0f;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void USplineModifier::InitializeModifier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USplineModifier::InitializeModifier);

	Super::InitializeModifier();
	CurrentCachedData.Reset();

	// Set spline pointer from name if not already done

	if (TemplateSplineComponentName.IsEmpty())
	{
		SplineDelegateHelper->Register(GetSplineComponent());
		return;
	}

	if (SplinePtr.IsValid() && SplinePtr->GetName() == TemplateSplineComponentName)
	{
		SplineDelegateHelper->Register(GetSplineComponent());
		return;
	}

	// Retrieve the actual spline component from the name
	if (const AActor* const Owner = GetOwner())
	{
		TArray<USplineComponent*> ActorSplines;
		Owner->GetComponents(ActorSplines, /*bIncludeFromChildActors = */ true);

		for (USplineComponent* const SplineComponent : ActorSplines)
		{
			if ((SplineComponent->GetName() == TemplateSplineComponentName) && (SplineComponent->GetOwner() == GetOwner()))
			{
				SetSplineComponent(SplineComponent);
				// SetSplineComponent registers spline delegates
				return;
			}
		}
	}

	SplineDelegateHelper->Register(GetSplineComponent());
}

void USplineModifier::UninitializeModifier()
{
	SplineDelegateHelper->Unregister();

	if (EdgeFalloffCurve && EdgeFalloffUpdateHandle.IsValid())
	{
		EdgeFalloffCurve->OnUpdateCurve.Remove(EdgeFalloffUpdateHandle);
	}

	Super::UninitializeModifier();
}


void USplineModifier::UpdateWhenEdgeFalloffChanges(UCurveBase* Curve, EPropertyChangeType::Type ChangeType)
{
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void USplineModifier::SetSplineComponent(USplineComponent* Spline, bool bUpdate)
{
	if (!ensure(Spline))
	{
		return;
	}

	if (!ensure(Spline->GetOwner() == this->GetOwner()))
	{
		return;
	}

	if (bUpdate)
	{
		// PostEditChangeProperty's SplinePtr branch handles delegate (re)registration.
		FProperty* const Prop = FindFProperty<FProperty>(USplineModifier::StaticClass(), GET_MEMBER_NAME_CHECKED(USplineModifier, SplinePtr));
		PreEditChange(Prop);

		SplinePtr = TSoftObjectPtr<USplineComponent>(FSoftObjectPath(Spline));

		FPropertyChangedEvent E(Prop, EPropertyChangeType::ValueSet);
		PostEditChangeProperty(E);

		CurrentCachedData.Reset();
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
	else
	{
		// Silent mode: assign without firing PostEditChangeProperty, so register here directly.
		SplinePtr = TSoftObjectPtr<USplineComponent>(FSoftObjectPath(Spline));
		SplineDelegateHelper->Register(GetSplineComponent());
	}
}

void USplineModifier::BP_SetSplineComponent(USplineComponent* InSplineComponent)
{
	if (!InSplineComponent)
	{
		return;
	}

	const bool bIsTemplate = (IsTemplate() || HasAnyFlags(RF_ArchetypeObject));
	if (bIsTemplate)
	{
		TemplateSplineComponentName = InSplineComponent->GetName();
	}
	else
	{
		SetSplineComponent(InSplineComponent, /*bUpdate =*/ false);
	}

	// Update the UI but use EPropertyChangeType::Interactive so we don't trigger a rerun of the construction script
	if (GIsEditor)
	{
		if (FProperty* const Prop = FindFProperty<FProperty>(StaticClass(), GET_MEMBER_NAME_CHECKED(USplineModifier, SplinePtr)))
		{
			FPropertyChangedEvent Ev(Prop, EPropertyChangeType::Interactive);
			PostEditChangeProperty(Ev);
		}
	}

}


void USplineModifier::SetProjectedSplineCurve(FInterpCurveVector&& NewProjectedCurve) const
{
	FScopeLock Lock(&ProjectedSplineCurveMutex);
	ProjectedSplineCurve = MoveTemp(NewProjectedCurve);
}

void USplineModifier::UpdateSplineData()
{
	CurrentCachedData.Reset();
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void MegaMeshSplineModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	AddDefaultInstanceIfIntersects(LocalBounds.TransformBy(ComponentTransform), InBounds, OutInstanceInfos);
	if (OutInstanceInfos.IsEmpty())
	{
		return;
	}

	// Read components
	OutInstanceInfos[0].ReadViewComponents = EMeshViewComponents::VertexPos;
	if (ShouldFillInterior() && InteriorSmoothMode != MeshPartition::ESplineModifierInteriorSmoothMode::Simple)
	{
		OutInstanceInfos[0].ReadViewComponents |= EMeshViewComponents::DynamicSubmesh;
	}
	else if (bUseNearestSplineFrameForDisplacement && bProjectSplineToSurface)
	{
		OutInstanceInfos[0].ReadViewComponents |= EMeshViewComponents::DynamicSubmesh;
	}

	// Write components
	OutInstanceInfos[0].WriteViewComponents = EnumHasAnyFlags(WriteMode, ESplineModifierWriteMode::Positions) ? EMeshViewComponents::VertexPos : EMeshViewComponents::None;

	// Weight channels
	if (EnumHasAnyFlags(WriteMode, ESplineModifierWriteMode::Weights))
	{
		for (const FWeightEntry& Entry : WeightEntries)
		{
			if (!Entry.WeightChannelName.IsNone())
			{
				OutInstanceInfos[0].ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				OutInstanceInfos[0].WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				OutInstanceInfos[0].UsedChannels.Emplace(Entry.WeightChannelName);
			}
		}
	}

}

bool MegaMeshSplineModifierLocals::FBackgroundOp::ShouldFillInterior() const
{
	return SplineCurves.Position.bIsLooped && bMeshClosedInterior && WorldSampledLoopPoints.Num() > 2;
}

bool USplineModifier::ShouldFillInterior() const
{
	USplineComponent* Spline = GetSplineComponent();
	if (!Spline)
	{
		return false;
	}

	return Spline->IsClosedLoop() && bMeshClosedInterior && Spline->GetNumberOfSplinePoints() > 2;
}

USplineComponent* USplineModifier::GetSplineComponent() const
{
	return SplinePtr.Get();
}



TSharedPtr<const MeshPartition::IModifierBackgroundOp> USplineModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshSplineModifierLocals;

	USplineComponent* SplineComponent = GetSplineComponent();
	if (!SplineComponent)
	{
		return nullptr;
	}
	int32 PointsNeeded = SplineComponent->IsClosedLoop() ? 2 : 1;
	if (SplineComponent->GetNumberOfSplinePoints() < PointsNeeded)
	{
		return nullptr;
	}

	TSharedPtr<FBackgroundOp> Op = MakeShared<FBackgroundOp>(GetFName());
	Op->ParentModifier = this;
	Op->ComponentTransform = GetComponentToWorld();
	Op->LocalBounds = ComputeLocalBounds();
	Op->ProjectionDirection = SplineComponent->GetComponentToWorld().GetRotation().GetAxisZ();
	Op->SplineCurves = SplineComponent->GetSplineCurves();
	Op->SplineTransform = SplineComponent->GetComponentToWorld();
	Op->SplineDefaultUpVector = SplineComponent->DefaultUpVector;
	Op->SplineDefaultUpVector.Normalize(); // Not sure if necessary, but might as well
	
	Op->bUseNearestSplineFrameForDisplacement = bUseNearestSplineFrameForDisplacement;
	Op->bProjectSplineToSurface = !bNearestFrameFastApproximation;
	Op->BlendMode = BlendMode;

	Op->FalloffDistance = FalloffDistance;
	Op->PlateauDistance = PlateauDistance;
	Op->bUseSplineScaleForFalloff = bUseSplineScaleForFalloff;
	Op->bUseSplineScaleForPlateau = bUseSplineScaleForPlateau;
	Op->bUseEdgeFalloffCurve = bUseEdgeFalloffCurve;
	if (EdgeFalloffCurve && bUseEdgeFalloffCurve)
	{
		Op->EdgeFalloffCurve = MakeUnique<FRichCurve>(EdgeFalloffCurve->FloatCurve);
	}
	Op->bSplineIsLeftOfFalloffCurve = bSplineIsLeftOfFalloffCurve;

	Op->WriteMode = static_cast<ESplineModifierWriteMode>(WriteMode);

	Op->WeightEntries.Reserve(WeightChannels.Num());
	for (const FSplineModifierWeightEntry& WeightEntry : WeightChannels)
	{
		FBackgroundOp::FWeightEntry& OpEntry = Op->WeightEntries.AddDefaulted_GetRef();
		OpEntry.WeightChannelName = WeightEntry.WeightChannelName.GetName();
		OpEntry.Value = WeightEntry.Value;
		OpEntry.BlendMode = WeightEntry.BlendMode;
	}

	Op->bMeshClosedInterior = bMeshClosedInterior;
	// TODO: We shouldn't need to do this sampling here- we should be able to do it from the spline
	//  curve data, but it currently happens to be easier to do while we still have the spline. 
	if (ShouldFillInterior())
	{
		// We'll sample in world space so that the error is in world space
		SplineComponent->ConvertSplineToPolyLine(ESplineCoordinateSpace::World,
			SplinePolygonErrorTolerance * SplinePolygonErrorTolerance,
			Op->WorldSampledLoopPoints);

		if (Op->WorldSampledLoopPoints.Num() > 2 && Op->WorldSampledLoopPoints.Last() == Op->WorldSampledLoopPoints[0])
		{
			Op->WorldSampledLoopPoints.Pop();
		}
	}
	Op->MeshedInteriorNumTriTarget = MeshedInteriorNumTriTarget;
	
	// When the spline modifier is interactive, we keep smoothing mode simple by default for better performances.
	Op->InteriorSmoothMode = (InBuildType == MeshPartition::EBuildType::InteractiveModifier) ? MeshPartition::ESplineModifierInteriorSmoothMode::Simple : InteriorSmoothMode;

	if (CurrentCachedData &&
		CurrentCachedData->bInitializationCompleted.load(std::memory_order_relaxed)) 
	{
		Op->CachedDataIn = CurrentCachedData;
	}
	else
	{
		Op->ExternalCachedDataToInitialize = MakeShared<FSplineCachedSurfaceData>();
		CurrentCachedData = Op->ExternalCachedDataToInitialize;
		LastSplineVersion = SplineComponent->GetVersion();
	}

	return Op;
}

void MegaMeshSplineModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::USplineModifier::ApplyModifications);

	if (SplineCurves.Position.Points.Num() < (SplineCurves.Position.bIsLooped ? 3 : 1))
	{
		return;
	}

	bool bFillInterior = ShouldFillInterior();
	
	// If we're not meshing the interior, then we need the falloff distance to be positive, or else we won't
	//  actually affect any vertices unless they fall exactly on the spline.
	if (!bFillInterior && FalloffDistance <= 0)
	{
		return;
	}

	// Used if we end up using the falloff curve
	FVector2f CurveDomain = FVector2f(0, 1);
	FVector2f CurveRange = FVector2f(0, 1);
	if (bUseEdgeFalloffCurve && EdgeFalloffCurve.IsValid())
	{
		EdgeFalloffCurve->GetTimeRange(CurveDomain.X, CurveDomain.Y);
		EdgeFalloffCurve->GetValueRange(CurveRange.X, CurveRange.Y);
	}

	// Used unless we end up using bUseNearestSplineFrameForDisplacement
	FVector WorldProjectionDirection = ProjectionDirection;
	FVector3d ProjectionFrameX = FVector3d::XAxisVector;
	FVector3d ProjectionFrameY = FVector3d::YAxisVector;
	FVector3d::CreateOrthonormalBasis(ProjectionFrameX, ProjectionFrameY, WorldProjectionDirection);
	auto GetProjectedCoordinate = [&ProjectionFrameX, ProjectionFrameY](const FVector3d& Point) -> FVector2d
	{
		return FVector2d(ProjectionFrameX.Dot(Point), ProjectionFrameY.Dot(Point));
	};
	const FVector3d SplineSpaceProjectionDirection = SplineTransform.InverseTransformVectorNoScale(WorldProjectionDirection);
	
	const bool bMeshIsEmpty = InMeshView.VertexCount() == 0;

	// Used for getting projected distance from spline
	FInterpCurveVector ProjectedSplineCurve;
	if (bFillInterior || !bUseNearestSplineFrameForDisplacement || bMeshIsEmpty)
	{
		ProjectedSplineCurve = SplineCurves.Position;
		for (FInterpCurvePointVector& Point : ProjectedSplineCurve.Points)
		{
			Point.ArriveTangent -= Point.ArriveTangent.Dot(SplineSpaceProjectionDirection) * SplineSpaceProjectionDirection;
			Point.LeaveTangent -= Point.LeaveTangent.Dot(SplineSpaceProjectionDirection) * SplineSpaceProjectionDirection;
			Point.OutVal -= Point.OutVal.Dot(SplineSpaceProjectionDirection) * SplineSpaceProjectionDirection;
		}
	}
	else if (bUseNearestSplineFrameForDisplacement && bProjectSplineToSurface)
	{
		const FDynamicMesh3& Submesh = InMeshView.GetSubmesh();
		const Geometry::FDynamicMeshAABBTree3 SurfaceAABBTree(&Submesh, true);

		Geometry::SplineUtil::ProjectSplineToSurface(ProjectedSplineCurve, SplineCurves.Position, SurfaceAABBTree, SplineTransform, InTransform);
	}

	// Used if bFillInterior is true
	Geometry::FPolygon2d InteriorPolygon;
	TArray<bool> IsInsideFlags;
	TSharedPtr<const FSplineCachedSurfaceData> CachedDataToUse;
	TUniquePtr<UE::Solvers::IConstrainedMeshSolver> Smoother;
	if (bFillInterior)
	{
		for (const FVector3d& Point : WorldSampledLoopPoints)
		{
			// The polygon will be in component space so that the projection is easier
			FVector3d LocalPoint = ComponentTransform.InverseTransformPosition(Point);
			InteriorPolygon.AppendVertex(FVector2d(LocalPoint));
		}
		IsInsideFlags.SetNumZeroed(InMeshView.VertexCount());

		if (CachedDataIn)
		{
			CachedDataToUse = CachedDataIn;
		}
		else if (ExternalCachedDataToInitialize 
			// if no other running instance of this op started caching the data
			&& !ExternalCachedDataToInitialize->bNeedsInitializing.exchange(true))
		{
			ComputeLoopMesh(*ExternalCachedDataToInitialize);
			CachedDataToUse = ExternalCachedDataToInitialize;
		}
		else
		{
			TSharedPtr<FSplineCachedSurfaceData> CachedData = MakeShared<FSplineCachedSurfaceData>();
			ComputeLoopMesh(*CachedData);
			CachedDataToUse = CachedData;
		}

		if (!ensure(CachedDataToUse && CachedDataToUse->MeshedLoop && CachedDataToUse->MeshedLoopSpatial))
		{
			bFillInterior = false;
		}

		// NOTE: It is important that the smoother, if used, be set up here before we start modifying vertices. This is
		//  because it is not really used for smoothing, but for detail transfer. We initialize it with the original submesh
		//  positions now, then update positions, then constrain everything except for the interior of the spline and run the
		//  smoother, which sort-of brings the original mesh up to the boundary.
		switch (InteriorSmoothMode)
		{
		case MeshPartition::ESplineModifierInteriorSmoothMode::Simple:
			Smoother = nullptr;
			break;
		case MeshPartition::ESplineModifierInteriorSmoothMode::Smooth:
			Smoother = MeshDeformation::ConstructConstrainedMeshSmoother(ELaplacianWeightScheme::IDTCotanget, InMeshView.GetSubmesh());
			break;
		case MeshPartition::ESplineModifierInteriorSmoothMode::DetailPreserving:
			Smoother = MeshDeformation::ConstructSoftMeshDeformer(InMeshView.GetSubmesh());
			break;
		default:
			return;
		}
	}

	int32 VertexCount = InMeshView.VertexCount();
	//for (int VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
	ParallelFor(VertexCount, [&WorldProjectionDirection, &InMeshView, &InTransform, this, bFillInterior, 
		&InteriorPolygon, &IsInsideFlags, &SplineSpaceProjectionDirection, &CachedDataToUse, &ProjectedSplineCurve, 
		&CurveDomain, &CurveRange](int VertexIndex)
	{
		// May change if using spline-dependent direction
		FVector3d WorldProjectionDirectionToUse = WorldProjectionDirection;

		FVector3d MeshVertex = InMeshView.GetVertexPos(VertexIndex);
		FVector3d WorldVertex = InTransform.TransformPosition(MeshVertex);
		FVector3d LocalVertex = ComponentTransform.InverseTransformPosition(WorldVertex);
		if (!LocalBounds.IsInside(LocalVertex))
		{
			return;
		}

		FVector3d SplineSpaceVertex = SplineTransform.InverseTransformPosition(WorldVertex);

		// Handle interior of closed and meshed polygon by just projecting to the mesh
		if (bFillInterior && InteriorPolygon.Contains(FVector2d(ComponentTransform.InverseTransformPosition(WorldVertex))))
		{
			IsInsideFlags[VertexIndex] = true;
			double TravelTime;
			int TriangleID;
			FVector BaryCoords;
			FRay3d MeshRay(SplineSpaceVertex, SplineSpaceProjectionDirection);
			CachedDataToUse->MeshedLoopSpatial->FindNearestHitTriangle(MeshRay, TravelTime, TriangleID, BaryCoords);
			if (TriangleID == FDynamicMesh3::InvalidID)
			{
				MeshRay.Direction *= -1;
				CachedDataToUse->MeshedLoopSpatial->FindNearestHitTriangle(MeshRay, TravelTime, TriangleID, BaryCoords);
			}

			if (TriangleID != FDynamicMesh3::InvalidID && ensure(CachedDataToUse->MeshedLoop->IsTriangle(TriangleID)))
			{
				if (EnumHasAnyFlags(WriteMode, ESplineModifierWriteMode::Positions))
				{
					const FVector3d HitMeshPoint = CachedDataToUse->MeshedLoop->GetTriBaryPoint(TriangleID, BaryCoords.X, BaryCoords.Y, BaryCoords.Z);
					const FVector3d WorldHitPoint = SplineTransform.TransformPosition(HitMeshPoint);
					const FVector3d VertToSurface = HitMeshPoint - SplineSpaceVertex;
					const double SignedHeightToSpline = VertToSurface.Dot(SplineSpaceProjectionDirection);

					bool bVertexClamped = false;
					if (BlendMode == ESplineModifierBlendMode::Min && SignedHeightToSpline > 0.0)
					{
						bVertexClamped = true;
					}
					if (BlendMode == ESplineModifierBlendMode::Max && SignedHeightToSpline < 0.0)
					{
						bVertexClamped = true;
					}

					if (!bVertexClamped)
					{
						InMeshView.SetVertexPos(VertexIndex, InTransform.InverseTransformPosition(WorldHitPoint));
					}
				}

				// Interior vertices are at full influence
				if (EnumHasAnyFlags(WriteMode, ESplineModifierWriteMode::Weights))
				{
					for (const FWeightEntry& WeightEntry : WeightEntries)
					{
						if (WeightEntry.WeightChannelName.IsNone())
						{
							continue;
						}
						const double CurrentValue = InMeshView.GetVertexAttributeWeight(WeightEntry.WeightChannelName, VertexIndex);
						const double BlendedValue = BlendValues(CurrentValue, WeightEntry.Value, 1.0, WeightEntry.BlendMode);
						InMeshView.SetVertexAttributeWeight(WeightEntry.WeightChannelName, VertexIndex, BlendedValue);
					}
				}

				return;
			}

			// We can end up missing the mesh in some cases, such as when the loop had self intersections. We'll fall through
			//  to the other processing further below.
		}

		// If we got here, we have to get a projected distance to our spline
	
		float NearestSplineT = -1;
		float ActualDistanceToProjectedSpline = -1.0f;
		if (bUseNearestSplineFrameForDisplacement && !bFillInterior)
		{
			// Get closest point, then use the frame at that point for projection
			FQuat Quat;
			if (bProjectSplineToSurface)
			{
				float DistanceSq;
				NearestSplineT = ProjectedSplineCurve.FindNearest(SplineSpaceVertex, DistanceSq);
				Quat = SplineCurves.Rotation.Eval(NearestSplineT);
				Quat.Normalize();

				ActualDistanceToProjectedSpline = FMath::Sqrt(DistanceSq);
			}
			else
			{
				float DistanceSq;
				NearestSplineT = SplineCurves.Position.FindNearest(SplineSpaceVertex, DistanceSq);
				Quat = SplineCurves.Rotation.Eval(NearestSplineT);
				Quat.Normalize();
			}

			FVector3d NewSplineSpaceProjectionDirection = Quat.RotateVector(SplineDefaultUpVector);
			WorldProjectionDirectionToUse = SplineTransform.TransformVectorNoScale(NewSplineSpaceProjectionDirection);
		}
		else // if using single projection direction
		{
			float DistanceSq;
			NearestSplineT = ProjectedSplineCurve.FindNearest(SplineSpaceVertex, DistanceSq);
		}

		// Expect to have found the nearest (potentially projected) spline point
		if (!ensure(NearestSplineT >= 0))
		{
			return;
		}

		FVector3d SplinePoint = SplineCurves.Position.Eval(NearestSplineT);
		FVector3d SplineWorldPoint = SplineTransform.TransformPosition(SplinePoint);
		FVector3d VertToSpline = SplineWorldPoint - WorldVertex;

		const double SignedHeightToSpline = VertToSpline.Dot(WorldProjectionDirectionToUse);

		bool bVertexClamped = false;
		if (BlendMode == ESplineModifierBlendMode::Min && SignedHeightToSpline > 0.0)
		{
			bVertexClamped = true;
		}
		if (BlendMode == ESplineModifierBlendMode::Max && SignedHeightToSpline < 0.0)
		{
			bVertexClamped = true;
		}

		double ProjectedDistanceToSpline = (VertToSpline - SignedHeightToSpline * WorldProjectionDirectionToUse).Length();

		double FalloffToUse = FalloffDistance;
		if (bUseSplineScaleForFalloff)
		{
			FalloffToUse *= SplineCurves.Scale.Eval(NearestSplineT).Y;
		}
			
		double PlateauToUse = PlateauDistance;
		if (bUseSplineScaleForPlateau)
		{
			PlateauToUse *= SplineCurves.Scale.Eval(NearestSplineT).Y;
		}

		// Total coverage is plateau + falloff. Vertices beyond this range are unaffected.
		const double TotalDistance = PlateauToUse + FalloffToUse;

		// Now apply our results
		if (ProjectedDistanceToSpline >= TotalDistance)
		{
			// Vertex is too far away
			return;
		}

		float HeightAlpha;
		if (FalloffToUse <= 0.0)
		{
			// No falloff region: full influence everywhere within the plateau, hard edge
			HeightAlpha = 1.0f;
		}
		else if (bUseNearestSplineFrameForDisplacement && bProjectSplineToSurface)
		{
			HeightAlpha = 1.0 - FMath::Clamp((ActualDistanceToProjectedSpline - PlateauToUse) / FalloffToUse, 0.0, 1.0);
		}
		else
		{
			HeightAlpha = 1.0 - FMath::Clamp((ProjectedDistanceToSpline - PlateauToUse) / FalloffToUse, 0.0, 1.0);
		}

		if (bUseEdgeFalloffCurve && EdgeFalloffCurve)
		{
			float InputAlpha = bSplineIsLeftOfFalloffCurve ? 1.0 - HeightAlpha : HeightAlpha;
			float Time = FMath::Lerp(CurveDomain.X, CurveDomain.Y, InputAlpha);
			float CurveValue = EdgeFalloffCurve->Eval(Time);
			HeightAlpha = FMath::Clamp((CurveValue - CurveRange.X) / (CurveRange.Y - CurveRange.X), 0.0, 1.0);
		}

		if (EnumHasAnyFlags(WriteMode, ESplineModifierWriteMode::Positions) && !bVertexClamped)
		{
			FVector3d NewWorldPosition = WorldVertex + HeightAlpha * SignedHeightToSpline * WorldProjectionDirectionToUse;

			// We could just use the instance desc bounds to clamp our output, but we'll be proper and use our local
			//  bounds.
			NewWorldPosition = ComponentTransform.TransformPosition(
				LocalBounds.GetClosestPointTo(ComponentTransform.InverseTransformPosition(NewWorldPosition)));

			InMeshView.SetVertexPos(VertexIndex, InTransform.InverseTransformPosition(NewWorldPosition));
		}

		if (EnumHasAnyFlags(WriteMode, ESplineModifierWriteMode::Weights))
		{
			for (const FWeightEntry& WeightEntry : WeightEntries)
			{
				if (WeightEntry.WeightChannelName.IsNone())
				{
					continue;
				}
				const double CurrentValue = InMeshView.GetVertexAttributeWeight(WeightEntry.WeightChannelName, VertexIndex);
				const double BlendedValue = BlendValues(CurrentValue, WeightEntry.Value, HeightAlpha, WeightEntry.BlendMode);
				InMeshView.SetVertexAttributeWeight(WeightEntry.WeightChannelName, VertexIndex, BlendedValue);
			}
		}
	});

	// The smoother is used for detail transfer- it gets initialized with an unmodified mesh, gets everything outside
	//  of the interior constrained to updated locations, and then gets run to bring the interior up.
	if (Smoother)
	{
		//TODO: Find a way to include this step in the above parallel for...
		for (int VertexIndex = 0; VertexIndex < VertexCount; ++VertexIndex)
		{
			const int SubmeshVID = InMeshView.GetSubmeshVID(VertexIndex);
			FVector3d MeshVertex = InMeshView.GetVertexPos(VertexIndex);
			const bool bIsInside = IsInsideFlags[VertexIndex];

			if (!bIsInside)
			{
				Smoother->AddConstraint(SubmeshVID, 1.0, MeshVertex, false);
			}
		}

		TArray<FVector3d> VertexPositions;
		Smoother->Deform(VertexPositions);

		ParallelFor(VertexCount, [this, &IsInsideFlags, &InMeshView, &InTransform, &VertexPositions](int VertexIndex)
		{
			const int SubmeshVID = InMeshView.GetSubmeshVID(VertexIndex);
			if (IsInsideFlags[VertexIndex])
			{
				// We'll clamp in local space
				FVector3d NewLocalPosition = ComponentTransform.InverseTransformPosition(
					InTransform.TransformPosition(VertexPositions[VertexIndex]));
				NewLocalPosition = LocalBounds.GetClosestPointTo(NewLocalPosition);
				FVector3d NewWorldPosition = ComponentTransform.TransformPosition(NewLocalPosition);
				InMeshView.SetVertexPos(VertexIndex, InTransform.InverseTransformPosition(NewWorldPosition));
			}
		});

	}

	if (const USplineModifier* const SplineModifier = ParentModifier.Get())
	{
		SplineModifier->SetProjectedSplineCurve(MoveTemp(ProjectedSplineCurve));
	}
}

void USplineModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USplineModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	USplineComponent* SplineComponent = GetSplineComponent();
	if (!SplineComponent)
	{
		// does nothing if there's no spline
		return;
	}

	// instead of hashing the entire serialized spline component, we just record the package dependency,
	// and then hash the specific data that we use from the spline below
	Dependencies.AddPackageDependency(SplineComponent);

	int32 PointsNeeded = SplineComponent->IsClosedLoop() ? 2 : 1;
	if (SplineComponent->GetNumberOfSplinePoints() < PointsNeeded)
	{
		return;
	}

	Dependencies += ComputeLocalBounds();

	// serialize spline curves (via tagged properties), without defaulting as we want to detect changes to default data too
	FSplineCurves SplineCurves = SplineComponent->GetSplineCurves();
	UStruct* SplineCurveStruct = FSplineCurves::StaticStruct();
	SplineCurveStruct->SerializeTaggedProperties(
		Dependencies.GetDependentDataArchive(), 
		reinterpret_cast<uint8*>(&SplineCurves),
		SplineCurveStruct, nullptr);

	Dependencies += SplineComponent->GetComponentToWorld();
	Dependencies += SplineComponent->DefaultUpVector;

	Dependencies += bUseNearestSplineFrameForDisplacement;
	Dependencies += bNearestFrameFastApproximation;
	Dependencies += BlendMode;

	Dependencies += FalloffDistance;
	Dependencies += PlateauDistance;
	Dependencies += bUseSplineScaleForFalloff;
	Dependencies += bUseSplineScaleForPlateau;
	Dependencies += bExpandBoundsBySplineScale;
	Dependencies += FalloffBoundsMultiplier;
	Dependencies += bUseEdgeFalloffCurve;
	if (EdgeFalloffCurve && bUseEdgeFalloffCurve)
	{
		Dependencies += EdgeFalloffCurve;
	}
	Dependencies += bSplineIsLeftOfFalloffCurve;
	Dependencies += bMeshClosedInterior;
	Dependencies += ShouldFillInterior();
	if (ShouldFillInterior())
	{
		Dependencies += SplinePolygonErrorTolerance;
		// as long as the whole spline curve is hashed above,
		// we shouldn't need to pull out and hash individual poly points
	}
	Dependencies += MeshedInteriorNumTriTarget;
	Dependencies += InteriorSmoothMode;

	Dependencies += WriteMode;

	for (const FSplineModifierWeightEntry& WeightEntry : WeightChannels)
	{
		Dependencies += WeightEntry.WeightChannelName.GetName();
		Dependencies += WeightEntry.Value;
		Dependencies += WeightEntry.BlendMode;
	}
}

TArray<FBox> USplineModifier::ComputeBounds() const
{
	USplineComponent* SplineComponent = GetSplineComponent();

	if (!SplineComponent)
	{
		return {};
	}

	FBox LocalBounds = ComputeLocalBounds();

	return { LocalBounds.TransformBy(GetComponentToWorld()) };
}

FBox USplineModifier::ComputeLocalBounds() const
{
	USplineComponent* SplineComponent = GetSplineComponent();
	if (!SplineComponent)
	{
		return FBox();
	}

	Geometry::FAxisAlignedBox3d SplineBounds = SplineComponent->GetLocalBounds().GetBox();
	const FTransform& SplineTransform = SplineComponent->GetComponentToWorld();

	Geometry::FAxisAlignedBox3d LocalBounds;
	for (int i = 0; i < 8; ++i)
	{
		LocalBounds.Contain(GetComponentToWorld().InverseTransformPosition(
			SplineTransform.TransformPosition(SplineBounds.GetCorner(i))));
	}

	float BoundsMultiplier = FalloffBoundsMultiplier;
	if ((bUseSplineScaleForFalloff || bUseSplineScaleForPlateau) && bExpandBoundsBySplineScale)
	{
		// Find a safe upper bound for Scale.Y along the spline. The Hermite curve is equivalent to a
		// Bezier curve, which is bounded by its convex hull, so checking the Bezier control points for
		// each segment guarantees we capture any overshoot from tangents.
		const FSplineCurves& SplineCurves = SplineComponent->GetSplineCurves();
		const TArray<FInterpCurvePoint<FVector>>& ScalePoints = SplineCurves.Scale.Points;
		float MaxAbsScaleY = 0.0f;
		for (int32 ScalePointIndex = 0; ScalePointIndex < ScalePoints.Num(); ++ScalePointIndex)
		{
			const FInterpCurvePoint<FVector>& CurrentPoint = ScalePoints[ScalePointIndex];
			const float P0Y = CurrentPoint.OutVal.Y;
			MaxAbsScaleY = FMath::Max(MaxAbsScaleY, FMath::Abs(P0Y));

			// For Constant/Linear interp the curve can't overshoot the endpoint values,
			// so the Bezier control point check is only needed for curve interp modes.
			const bool bHasTangents = (CurrentPoint.InterpMode != CIM_Constant && CurrentPoint.InterpMode != CIM_Linear);
			if (bHasTangents && ScalePointIndex + 1 < ScalePoints.Num())
			{
				const FInterpCurvePoint<FVector>& NextPoint = ScalePoints[ScalePointIndex + 1];
				const float P1Y = NextPoint.OutVal.Y;
				const float Diff = NextPoint.InVal - CurrentPoint.InVal;

				// Hermite-to-Bezier: B1 = P0 + T0*Diff/3, B2 = P1 - T1*Diff/3
				const float B1Y = P0Y + CurrentPoint.LeaveTangent.Y * Diff / 3.0f;
				const float B2Y = P1Y - NextPoint.ArriveTangent.Y * Diff / 3.0f;

				MaxAbsScaleY = FMath::Max(MaxAbsScaleY, FMath::Max(FMath::Abs(B1Y), FMath::Abs(B2Y)));
			}
		}
		BoundsMultiplier = MaxAbsScaleY;
	}

	// Total lateral reach is plateau + falloff. Each component is scaled independently
	// based on whether it uses spline scale.
	const float FalloffBounds = FalloffDistance * (bUseSplineScaleForFalloff ? BoundsMultiplier : 1.0f);
	const float PlateauBounds = PlateauDistance * (bUseSplineScaleForPlateau ? BoundsMultiplier : 1.0f);
	const float TotalLateralDistance = PlateauBounds + FalloffBounds;

	// If BoundsMultiplier is zero due to all scale points being zero this will return the bounds around the spline points
	FBox ToReturn(LocalBounds);
	if (bUseNearestSplineFrameForDisplacement && !ShouldFillInterior())
	{
		ToReturn = ToReturn.ExpandBy(FMath::Max(TotalLateralDistance, MaxProjectionHeightExtent));
	}
	else
	{
		ToReturn = ToReturn.ExpandBy(FVector(TotalLateralDistance, TotalLateralDistance, MaxZDistance));
	}

	return ToReturn;
}

void MegaMeshSplineModifierLocals::FBackgroundOp::ComputeLoopMesh(FSplineCachedSurfaceData& CachedData) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::USplineModifier::ComputeLoopMesh);

	using namespace Geometry;

	if (!ensure(WorldSampledLoopPoints.Num() > 2))
	{
		return;
	}

	TArray<FIndex3i> Tris;
	PolygonTriangulation::TriangulateSimplePolygon<double>(WorldSampledLoopPoints, Tris);

	if (Tris.Num() == 0)
	{
		return;
	}

	CachedData.MeshedLoop = MakeShared<FDynamicMesh3>();
	double MeshedLoopArea = 0;
	for (const FVector3d& WorldPoint : WorldSampledLoopPoints)
	{
		CachedData.MeshedLoop->AppendVertex(SplineTransform.InverseTransformPosition(WorldPoint));
	}
	for (const FIndex3i& Triangle : Tris)
	{
		int32 Tid = CachedData.MeshedLoop->AppendTriangle(Triangle);
		MeshedLoopArea += CachedData.MeshedLoop->GetTriArea(Tid);
	}

	CachedData.MeshedLoopSpatial = MakeShared<FDynamicMeshAABBTree3>(CachedData.MeshedLoop.Get());

	double TargetTriArea = MeshedLoopArea / FMath::Max(0, MeshedInteriorNumTriTarget);
	double TargetEdgeLength = TriangleUtil::EquilateralEdgeLengthForArea(TargetTriArea);

	TUniquePtr<Geometry::FDynamicMeshOperator> RemeshOperator = CreateRemeshingOperator(CachedData, TargetEdgeLength);
	RemeshOperator->CalculateResult(nullptr);
	if (RemeshOperator->GetResultInfo().Result == EGeometryResultType::Success)
	{
		TUniquePtr<FDynamicMesh3> ResultMesh = RemeshOperator->ExtractResult();
		CachedData.MeshedLoop->Copy(*ResultMesh);
		CachedData.MeshedLoopSpatial->Build();
	}

	// memory_order_release to make sure this doesn't get moved up by the compiler.
	CachedData.bInitializationCompleted.store(true, std::memory_order_release);
}

TUniquePtr<Geometry::FDynamicMeshOperator> MegaMeshSplineModifierLocals::FBackgroundOp::CreateRemeshingOperator(
	FSplineCachedSurfaceData& CachedData, double TargetEdgeLength) const
{
	using namespace Geometry;

	TUniquePtr<FRemeshMeshOp> Op = MakeUnique<FRemeshMeshOp>();

	Op->RemeshType = ERemeshType::Standard;
	Op->TargetEdgeLength = TargetEdgeLength;
	Op->bCollapses = true;
	Op->bDiscardAttributes = true;
	// We always want attributes enabled on result even if we discard them initially
	Op->bResultMustHaveAttributesEnabled = true;
	Op->bFlips = true;
	Op->bPreserveSharpEdges = true;
	Op->MeshBoundaryConstraint = EEdgeRefineFlags::FullyConstrained;
	Op->GroupBoundaryConstraint = EEdgeRefineFlags::NoConstraint;
	Op->MaterialBoundaryConstraint = EEdgeRefineFlags::NoConstraint;
	Op->bPreventNormalFlips = true;
	Op->bPreventTinyTriangles = true;
	Op->bReproject = true;
	Op->bReprojectConstraints = false;
	Op->BoundaryCornerAngleThreshold = 45.0;
	Op->bSplits = true;
	Op->RemeshIterations = 20;
	Op->MaxRemeshIterations = 20;
	Op->ExtraProjectionIterations = 5;
	Op->SmoothingStrength = 0.25;
	Op->SmoothingType = ERemeshSmoothingType::MeanValue;

	FTransform LocalToWorld = FTransform::Identity;
	Op->SetTransform(LocalToWorld);

	Op->OriginalMesh = CachedData.MeshedLoop;
	Op->OriginalMeshSpatial = CachedData.MeshedLoopSpatial;

	Op->ProjectionTarget = nullptr;
	Op->ProjectionTargetSpatial = nullptr;

	return Op;
}

void USplineModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	using namespace MegaMeshSplineModifierLocals;

	const FColor RectangleColor = FColor::Red;
	const FColor LocalBoundsColor = FColor::Yellow;
	const FColor GlobalBoundsColor = FColor::Orange;
	const float RectangleThickness = 3;
	const float BoundsThickness = 1;
	const float DepthBias = 1;
	const bool bScreenSpace = true;

	const USplineComponent* const Spline = GetSplineComponent();

	if (bShowDebugSurface && Spline
		&& CurrentCachedData && CurrentCachedData->bInitializationCompleted.load(std::memory_order_acquire)
		&& CurrentCachedData->MeshedLoop)
	{
		const FTransform SplineTransform = Spline->GetComponentToWorld();
		for (Geometry::FDynamicMesh3::FEdge Edge : CurrentCachedData->MeshedLoop->EdgesItr())
		{
			FVector3d A, B;
			A = CurrentCachedData->MeshedLoop->GetVertexRef(Edge.Vert.A);
			B = CurrentCachedData->MeshedLoop->GetVertexRef(Edge.Vert.B);
			PDI->DrawLine(SplineTransform.TransformPosition(A), SplineTransform.TransformPosition(B), FColor::Magenta, 0, 3.0f, 1.0f, true);
		}
	}
	
	if (bDrawLocalBounds)
	{
		const FBox LocalBounds = ComputeLocalBounds();
		DrawWireBox(PDI, GetComponentToWorld().ToMatrixWithScale(), LocalBounds, LocalBoundsColor, SDPG_World, 1.0, 0, true);
	}

	if (bDrawSplineFrames && Spline)
	{
		const double DeltaParam = 1.0 / (double)(NumSplineFrames + 1);
		const float SplineLength = Spline->GetSplineLength();
		for (int32 PointIndex = 0; PointIndex < NumSplineFrames; ++PointIndex)
		{
			const double Param = DeltaParam * (PointIndex + 1);		// 0 - 1
			const double Distance = Param * SplineLength;			// 0 - SplineLength
			const FTransform SampleTransform = Spline->GetTransformAtDistanceAlongSpline(Distance, ESplineCoordinateSpace::World);
			
			DrawCoordinateSystem(PDI, SampleTransform.GetLocation(), FRotator(SampleTransform.GetRotation()), FrameScale, SDPG_World, 4.0);
		}
	}

	if (bDrawProjectionPlane && !bUseNearestSplineFrameForDisplacement && Spline)
	{
		const FBox LocalBounds = ComputeLocalBounds();
		const FTransform SplineTransform = Spline->GetComponentToWorld();
		DrawRectangle(PDI, SplineTransform.GetTranslation(), SplineTransform.GetUnitAxis(EAxis::X), SplineTransform.GetUnitAxis(EAxis::Y),
			RectangleColor, 2.0*LocalBounds.GetExtent().X * SplineTransform.GetScale3D().X, 2.0 * LocalBounds.GetExtent().Y * SplineTransform.GetScale3D().Y, SDPG_Foreground,
			RectangleThickness, DepthBias, bScreenSpace);
	}

	if (bDrawProjectedSpline && bUseNearestSplineFrameForDisplacement && !bNearestFrameFastApproximation)
	{
		FScopeLock Lock(&ProjectedSplineCurveMutex);

		for (int32 PointIndex = 0; PointIndex < ProjectedSplineCurve.Points.Num() - 1; ++PointIndex)
		{
			const FInterpCurvePointVector& Point = ProjectedSplineCurve.Points[PointIndex];
			const FInterpCurvePointVector& NextPoint = ProjectedSplineCurve.Points[PointIndex +1];

			const int32 NumSubSteps = 20;
			double DT = 1.0 / (NumSubSteps+1);
			for (int32 SubStep = 0; SubStep <= NumSubSteps; ++SubStep)
			{
				const double SubT = SubStep * DT;
				const double FullT = Point.InVal + SubT * (NextPoint.InVal - Point.InVal);
				
				const double NextSubT = (SubStep + 1) * DT;
				const double NextFullT = Point.InVal + NextSubT * (NextPoint.InVal - Point.InVal);

				const FVector3d SubPoint = ProjectedSplineCurve.Eval(FullT, FVector3d(0));
				const FVector3d NextSubPoint = ProjectedSplineCurve.Eval(NextFullT, FVector3d(0));

				PDI->DrawLine(GetComponentToWorld().TransformPosition(SubPoint), GetComponentToWorld().TransformPosition(NextSubPoint), FColor::Cyan, SDPG_World, 3.0f, 15.0f, true);
			}		
		}
	}
}

FGuid USplineModifier::GetCodeVersionKey() const
{
	return MegaMeshSplineModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

bool USplineModifier::IsContiguous() const
{
	return bMeshClosedInterior && ((InteriorSmoothMode == MeshPartition::ESplineModifierInteriorSmoothMode::Smooth) || (InteriorSmoothMode == MeshPartition::ESplineModifierInteriorSmoothMode::DetailPreserving));
}

void USplineModifier::SetMaxZDistance(float InMaxZDistance)
{
	MaxZDistance = InMaxZDistance;
}

void USplineModifier::SetUseNearestSplineFrameForDisplacement(bool InUseNearestSplineFrameForDisplacement)
{
	bUseNearestSplineFrameForDisplacement = InUseNearestSplineFrameForDisplacement;
}

void USplineModifier::SetFalloffDistance(float InFalloffDistance)
{
	FalloffDistance = InFalloffDistance;
}

void USplineModifier::SetSplinePolygonErrorTolerance(float InSplinePolygonErrorTolerance)
{
	SplinePolygonErrorTolerance = InSplinePolygonErrorTolerance;
}

void USplineModifier::SetMeshedInteriorNumTriTarget(double InMeshedInteriorNumTriTarget)
{
	if (!FMath::IsFinite(InMeshedInteriorNumTriTarget))
	{
		UE_LOGFMT(LogUnrealMath, Warning, "USplineModifier::SetMeshedInteriorNumTriTarget: Invalid infinite input value: {Value}", InMeshedInteriorNumTriTarget);
		return;
	}

	MeshedInteriorNumTriTarget = FMath::Max(1.0, InMeshedInteriorNumTriTarget);
}

void USplineModifier::SetFalloffBoundsMultiplier(float InFalloffBoundsMultiplier)
{
	if (!FMath::IsFinite(InFalloffBoundsMultiplier))
	{
		UE_LOGFMT(LogUnrealMath, Warning, "USplineModifier::InFalloffBoundsMultiplier: Invalid infinite input value: {Value}", InFalloffBoundsMultiplier);
		return;
	}

	FalloffBoundsMultiplier = FMath::Max(1.0, InFalloffBoundsMultiplier);
}
void USplineModifier::SetInteriorSmoothMode(ESplineModifierInteriorSmoothMode InInteriorSmoothMode)
{
	InteriorSmoothMode = InInteriorSmoothMode;
}

void USplineModifier::SetMaxProjectionHeightExtent(float InMaxProjectionHeightExtent)
{
	if (!FMath::IsFinite(InMaxProjectionHeightExtent))
	{
		UE_LOGFMT(LogUnrealMath, Warning, "USplineModifier::SetMaxProjectionHeightExtent: Invalid infinite input value: {Value}", InMaxProjectionHeightExtent);
		return;
	}

	MaxProjectionHeightExtent = FMath::Max(1.0, InMaxProjectionHeightExtent);
}

void USplineModifier::SetMeshClosedInterior(bool InbMeshClosedInterior)
{
	bMeshClosedInterior = InbMeshClosedInterior;
}

void USplineModifier::SetUseSplineScaleForFalloff(bool bInUseSplineScaleForFalloff)
{
	bUseSplineScaleForFalloff = bInUseSplineScaleForFalloff;
}

void USplineModifier::SetExpandBoundsBySplineScale(bool bInExpandBoundsBySplineScale)
{
	bExpandBoundsBySplineScale = bInExpandBoundsBySplineScale;
}

} // namespace UE::MeshPartition