// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionSplineRemeshModifier.h"
#include "Modifiers/MeshPartitionSplineCachedData.h"
#include "Components/SplineComponent.h"
#include "Misc/TransactionObjectEvent.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Kismet2/ComponentEditorUtils.h"
#include "PrimitiveDrawInterface.h"
#include "PrimitiveDrawingUtils.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "SubRegionRemesher.h"
#include "MeshConstraintsUtil.h"
#include "Ops/MeshPartitionRemeshOp.h"
#include "Ops/MeshPartitionTessellateOp.h"
#include "DynamicMesh/MeshNormals.h"

namespace UE::MeshPartition
{
namespace MegaMeshSplineRemeshModifierLocals
{
	/**
	* A class that handles getting all triangles within a certain distance to the spline, including distance to the surface patch defined by a closed spline.
	* Uses FSplineCachedSurfaceData to avoid recomputing the surface, and for sharing it with the modifier.
	* TODO: Replace similar code in MegaMeshSplineModifierLocals::FBackgroundOp with this.
	**/
	class FSplineROI
	{
	public:

		FSplineCurves SplineCurves;
		FTransform SplineTransform;
		double SplineRadius;
		TArray<FVector> WorldSampledLoopPoints;
		double MeshedInteriorNumTriTarget;
		bool bMeshInterior;

		// When CachedDataIn is set, we want to use it. When ExternalCachedDataToInitialize is set, we want to
		//  initialize it. Only one of them should be set.
		TSharedPtr<const FSplineCachedSurfaceData> CachedDataIn;
		TSharedPtr<FSplineCachedSurfaceData> ExternalCachedDataToInitialize;

		bool ShouldFillInterior() const
		{
			return SplineCurves.Position.bIsLooped && bMeshInterior && WorldSampledLoopPoints.Num() > 2;
		}

		void CreateLoopMesh(FSplineCachedSurfaceData& CachedData) const
		{
			using namespace Geometry;

			auto CreateRemeshingOperator = [](FSplineCachedSurfaceData& CachedData, double TargetEdgeLength) -> TUniquePtr<Geometry::FDynamicMeshOperator>
			{
				TUniquePtr<FRemeshMeshOp> Op = MakeUnique<FRemeshMeshOp>();

				Op->RemeshType = ERemeshType::Standard;
				Op->TargetEdgeLength = TargetEdgeLength;
				Op->bCollapses = true;
				Op->bDiscardAttributes = true;
				// We always want attributes enabled on result even if we discard them initially
				Op->bResultMustHaveAttributesEnabled = true;
				Op->bFlips = true;
				Op->bPreserveSharpEdges = true;
				Op->MeshBoundaryConstraint = EEdgeRefineFlags::FullyConstrained;
				Op->GroupBoundaryConstraint = EEdgeRefineFlags::NoConstraint;
				Op->MaterialBoundaryConstraint = EEdgeRefineFlags::NoConstraint;
				Op->bPreventNormalFlips = true;
				Op->bPreventTinyTriangles = true;
				Op->bReproject = true;
				Op->bReprojectConstraints = false;
				Op->BoundaryCornerAngleThreshold = 45.0;
				Op->bSplits = true;
				Op->RemeshIterations = 20;
				Op->MaxRemeshIterations = 20;
				Op->ExtraProjectionIterations = 5;
				Op->SmoothingStrength = 0.25;
				Op->SmoothingType = ERemeshSmoothingType::MeanValue;

				Op->SetTransform(FTransform::Identity);
				Op->OriginalMesh = CachedData.MeshedLoop;
				Op->OriginalMeshSpatial = CachedData.MeshedLoopSpatial;

				Op->ProjectionTarget = nullptr;
				Op->ProjectionTargetSpatial = nullptr;

				return Op;
			};

			if (!ensure(WorldSampledLoopPoints.Num() > 2))
			{
				return;
			}

			TArray<FIndex3i> Tris;
			PolygonTriangulation::TriangulateSimplePolygon<double>(WorldSampledLoopPoints, Tris);

			if (Tris.Num() == 0)
			{
				return;
			}

			CachedData.MeshedLoop = MakeShared<FDynamicMesh3>();
			double MeshedLoopArea = 0;
			for (const FVector3d& WorldPoint : WorldSampledLoopPoints)
			{
				CachedData.MeshedLoop->AppendVertex(SplineTransform.InverseTransformPosition(WorldPoint));
			}
			for (const FIndex3i& Triangle : Tris)
			{
				int32 Tid = CachedData.MeshedLoop->AppendTriangle(Triangle);
				MeshedLoopArea += CachedData.MeshedLoop->GetTriArea(Tid);
			}

			CachedData.MeshedLoopSpatial = MakeShared<FDynamicMeshAABBTree3>(CachedData.MeshedLoop.Get());

			const double TargetTriArea = MeshedLoopArea / MeshedInteriorNumTriTarget;
			const double TargetEdgeLength = TriangleUtil::EquilateralEdgeLengthForArea(TargetTriArea);

			const TUniquePtr<Geometry::FDynamicMeshOperator> RemeshOperator = CreateRemeshingOperator(CachedData, TargetEdgeLength);
			RemeshOperator->CalculateResult(nullptr);
			if (RemeshOperator->GetResultInfo().Result == EGeometryResultType::Success)
			{
				const TUniquePtr<const FDynamicMesh3> ResultMesh = RemeshOperator->ExtractResult();
				CachedData.MeshedLoop->Copy(*ResultMesh);
				CachedData.MeshedLoopSpatial->Build();
			}

			FMeshNormals MeshNormals(CachedData.MeshedLoop.Get());
			MeshNormals.ComputeVertexNormals();
			MeshNormals.CopyToVertexNormals(CachedData.MeshedLoop.Get());

			// memory_order_release to make sure this doesn't get moved up by the compiler.
			CachedData.bInitializationCompleted.store(true, std::memory_order_release);
		}

		void GetTrianglesInSplineROI(const FDynamicMesh3& Mesh, TSet<int32>& OutTriangles) const
		{
			if (SplineCurves.Position.Points.Num() < (SplineCurves.Position.bIsLooped ? 3 : 1))
			{
				return;
			}

			if (SplineRadius <= 0)
			{
				return;
			}

			const double SplineRadiusSq = SplineRadius * SplineRadius;

			if (ShouldFillInterior())
			{
				TSharedPtr<const FSplineCachedSurfaceData> CachedDataToUse;
				if (CachedDataIn)
				{
					CachedDataToUse = CachedDataIn;
				}
				else if (ExternalCachedDataToInitialize
					// if no other running instance of this op started caching the data
					&& !ExternalCachedDataToInitialize->bNeedsInitializing.exchange(true))
				{
					CreateLoopMesh(*ExternalCachedDataToInitialize);
					CachedDataToUse = ExternalCachedDataToInitialize;
				}
				else
				{
					TSharedPtr<FSplineCachedSurfaceData> CachedData = MakeShared<FSplineCachedSurfaceData>();
					CreateLoopMesh(*CachedData);
					CachedDataToUse = CachedData;
				}

				for (const int32 VertexID : Mesh.VertexIndicesItr())
				{
					const FVector3d MeshVertexPosition = Mesh.GetVertex(VertexID);
					const FVector3d SplineSpaceVertex = SplineTransform.InverseTransformPosition(MeshVertexPosition);

					int TriangleID;
					if (CachedDataToUse->MeshedLoopSpatial->IsWithinDistanceSquared(SplineSpaceVertex, SplineRadiusSq, TriangleID))
					{
						for (const int32 TriIndex : Mesh.VtxTrianglesItr(VertexID))
						{
							OutTriangles.Add(TriIndex);
						}
					}
				}
			}
			else
			{
				for (const int32 VertexID : Mesh.VertexIndicesItr())
				{
					const FVector3d MeshVertexPosition = Mesh.GetVertex(VertexID);
					const FVector3d SplineSpaceVertexPosition = SplineTransform.InverseTransformPosition(MeshVertexPosition);

					float DistanceSq;
					SplineCurves.Position.FindNearest(SplineSpaceVertexPosition, DistanceSq);

					if (DistanceSq < SplineRadiusSq)
					{
						for (const int32 TriIndex : Mesh.VtxTrianglesItr(VertexID))
						{
							OutTriangles.Add(TriIndex);
						}
					}
				}
			}
		}
	};


	//
	// Remesh Op
	//

	class FMegaMeshSplineRemeshOp : public FMegaMeshRemeshBackgroundOpBase
	{
	public:
		FMegaMeshSplineRemeshOp(const FName& InOperationName);
		~FMegaMeshSplineRemeshOp() = default;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("b9ac53a9-d55b-4c97-96c2-958ef8caad5c"));
			return VersionKey;
		}

		FSplineROI SplineROI;

	private:

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override
		{
			return false;
		}
	};


	FMegaMeshSplineRemeshOp::FMegaMeshSplineRemeshOp(const FName& InOperationName) :
		FMegaMeshRemeshBackgroundOpBase(InOperationName)
	{}

	void FMegaMeshSplineRemeshOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
	{
		FInstanceInfo InstanceInfo;
		InstanceInfo.InstanceID = 0;
		InstanceInfo.Bounds = GlobalBounds;
		InstanceInfo.ReadViewComponents = EMeshViewComponents::DynamicSubmesh;
		InstanceInfo.WriteViewComponents = EMeshViewComponents::DynamicSubmesh;

		if (InstanceInfo.Bounds.Intersect(InBounds))
		{
			OutInstanceInfos.Add(InstanceInfo);
		}
	}

	void FMegaMeshSplineRemeshOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::USplineRemeshModifier::Remesh::ApplyModifications);

		FDynamicMesh3& SubMesh = InMeshView.GetSubmeshMutable();
		TSet<int32> InsideTriangles;

		SplineROI.GetTrianglesInSplineROI(SubMesh, InsideTriangles);

		RemeshROI(SubMesh, InTransform, InsideTriangles);
	}

	//
	// Tessellate Op
	//

	class FMegaMeshSplineTessellateOp : public FMegaMeshTessellateBackgroundOpBase
	{
	public:
		FMegaMeshSplineTessellateOp(const FName& InOperationName);
		~FMegaMeshSplineTessellateOp() = default;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("409803a8-a07e-4e3d-a4e5-ba7260bdd545"));
			return VersionKey;
		}

		FSplineROI SplineROI;

	private:

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override
		{
			return false;
		}
	};


	FMegaMeshSplineTessellateOp::FMegaMeshSplineTessellateOp(const FName& InOperationName) :
		FMegaMeshTessellateBackgroundOpBase(InOperationName)
	{
	}

	void FMegaMeshSplineTessellateOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
	{
		FInstanceInfo InstanceInfo;
		InstanceInfo.InstanceID = 0;
		InstanceInfo.Bounds = GlobalBounds;
		InstanceInfo.ReadViewComponents = EMeshViewComponents::DynamicSubmesh;
		InstanceInfo.WriteViewComponents = EMeshViewComponents::DynamicSubmesh;

		if (InstanceInfo.Bounds.Intersect(InBounds))
		{
			OutInstanceInfos.Add(InstanceInfo);
		}
	}

	void FMegaMeshSplineTessellateOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::USplineRemeshModifier::Tessellate::ApplyModifications);

		FDynamicMesh3& SubMesh = InMeshView.GetSubmeshMutable();
		TSet<int32> InsideTriangles;

		SplineROI.GetTrianglesInSplineROI(SubMesh, InsideTriangles);

		TessellateROI(SubMesh, InTransform, InsideTriangles);
	}

}

//
// Modifier
// 

void USplineRemeshModifier::PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent)
{
	Super::PostEditChangeProperty(InPropertyChangedEvent);

	if (InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(MeshPartition::USplineRemeshModifier, ROIVolumeTargetResolution)
		|| InPropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(MeshPartition::USplineRemeshModifier, ROIVolumePolygonErrorTolerance))
	{
		CurrentCachedData.Reset();
	}
}


USplineRemeshModifier::USplineRemeshModifier()
	: SplineDelegateHelper(MakeUnique<FSplineDelegateHelper>(this, [this]()
		{
			CurrentCachedData.Reset();
			OnChanged(ComputeBounds(), EChangeType::StateChange);
		}))
{
}

void USplineRemeshModifier::InitializeModifier()
{
	Super::InitializeModifier();
	SplineDelegateHelper->Register(GetSplineComponent());
}

void USplineRemeshModifier::UninitializeModifier()
{
	SplineDelegateHelper->Unregister();
	Super::UninitializeModifier();
}

void USplineRemeshModifier::SetSplineComponent(USplineComponent* Spline, bool bUpdate)
{
	if (!ensure(Spline))
	{
		return;
	}

	if (!ensure(Spline->GetOwner() == this->GetOwner()))
	{
		return;
	}

	SplineRef = FComponentEditorUtils::MakeComponentReference(Spline->GetOwner(), Spline);
	SplineDelegateHelper->Register(GetSplineComponent());
	if (bUpdate)
	{
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}
	CurrentCachedData.Reset();
}

void USplineRemeshModifier::UpdateSplineData()
{
	OnChanged(ComputeBounds(), EChangeType::TransientChange);
	CurrentCachedData.Reset();
}

USplineComponent* USplineRemeshModifier::GetSplineComponent() const
{
	return Cast<USplineComponent>(SplineRef.GetComponent(GetOwner()));
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> USplineRemeshModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshSplineRemeshModifierLocals;

	const USplineComponent* const SplineComponent = GetSplineComponent();
	if (!SplineComponent)
	{
		return nullptr;
	}

	const int32 PointsNeeded = SplineComponent->IsClosedLoop() ? 2 : 1;
	if (SplineComponent->GetNumberOfSplinePoints() < PointsNeeded)
	{
		return nullptr;
	}

	TSharedPtr<const MeshPartition::IModifierBackgroundOp> OutOp;

	auto InitSplineROI = [this, &SplineComponent](FSplineROI& SplineROI)
	{
		SplineROI.SplineCurves = SplineComponent->GetSplineCurves();
		SplineROI.SplineTransform = SplineComponent->GetComponentToWorld();
		SplineROI.SplineRadius = SplineRadius;
		SplineROI.bMeshInterior = bCreateVolumeFromClosedSpline;
		SplineROI.MeshedInteriorNumTriTarget = ROIVolumeTargetResolution;

		if (CurrentCachedData &&
			CurrentCachedData->bInitializationCompleted.load(std::memory_order_relaxed))
		{
			SplineROI.CachedDataIn = CurrentCachedData;
		}
		else
		{
			SplineROI.ExternalCachedDataToInitialize = MakeShared<FSplineCachedSurfaceData>();
			CurrentCachedData = SplineROI.ExternalCachedDataToInitialize;
			LastSplineVersion = SplineComponent->GetVersion();
		}
	};


	switch (CurrentOperation)
	{
	case MeshPartition::ERemeshModifierOperation::Remesh:
	{
		const TSharedPtr<FMegaMeshSplineRemeshOp> Op = MakeShared<FMegaMeshSplineRemeshOp>(GetFName());

		Op->GlobalBounds = ComputeCombinedBounds();
		Op->LocalCoverage = Geometry::FAxisAlignedBox3d(GetLocalBounds().GetBox());
		Op->ModifierToWorld = GetComponentTransform();
		Op->bComputeNormalSeams = bPreserveNormalSeams;
		Op->NormalSeamDotProductThreshold = FMathf::Cos(SharpEdgeAngleThreshold * FMathf::DegToRad);
		Op->BoundaryMode = BoundaryMode;
		Op->bDisallowUnsafeBoundaryEdits = bDisallowUnsafeBoundaryEdits;
		Op->bDisallowSafeEditsOutsideCoverage = bDisallowSafeEditsOutsideCoverage;
		Op->TargetEdgeLength = TargetEdgeLength;
		Op->RemeshIterations = RemeshIterations;
		Op->SmoothingStrength = SmoothingStrength;
		Op->SmoothingType = SmoothingType;
		Op->bProjectToInputMesh = bProjectToInputMesh;
		Op->bUseDensityWeightChannel = bUseDensityWeightChannel;
		Op->DensityWeightChannelName = DensityWeightChannelName;
		Op->RelativeDensity = RelativeDensity;

		SplineComponent->ConvertSplineToPolyLine(ESplineCoordinateSpace::World, ROIVolumePolygonErrorTolerance * ROIVolumePolygonErrorTolerance, Op->SplineROI.WorldSampledLoopPoints);

		InitSplineROI(Op->SplineROI);

		OutOp = Op;
	}
	break;
	case MeshPartition::ERemeshModifierOperation::Tessellate:
	{
		const TSharedPtr<FMegaMeshSplineTessellateOp> Op = MakeShared<FMegaMeshSplineTessellateOp>(GetFName());

		Op->GlobalBounds = ComputeCombinedBounds();
		Op->LocalBounds = GetLocalBounds().GetBox();
		Op->bUseTargetEdgeLength = bUseTargetEdgeLength;
		Op->TessellationLevel = TessellationLevel;
		Op->TargetEdgeLength = TessellationTargetEdgeLength;
		Op->MaxTessellationLevel = MaxTessellationLevel;
		Op->TessellationMethod = TessellationMethod;
		Op->ModifierTransform = GetComponentToWorld();
		Op->PostProcessingIterations = PostProcessingIterations;
		Op->bVertexSmoothing = bVertexSmoothing;
		Op->SmoothingStrength = TessellateSmoothingStrength;
		Op->bResampleUVs = bResampleUVs;
		Op->bEdgeFlips = bEdgeFlips;

		Op->bUseDensityWeightChannel = bUseDensityWeightChannel;
		Op->DensityWeightChannelName = DensityWeightChannelName;
		Op->RelativeDensity = RelativeDensity;

		SplineComponent->ConvertSplineToPolyLine(ESplineCoordinateSpace::World, ROIVolumePolygonErrorTolerance * ROIVolumePolygonErrorTolerance, Op->SplineROI.WorldSampledLoopPoints);

		InitSplineROI(Op->SplineROI);

		OutOp = Op;
	}
	break;
	};

	return OutOp;
}


void USplineRemeshModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(USplineRemeshModifier::GatherDependencies);

	MeshPartition::URemeshModifierBase::GatherDependencies(Dependencies);

	const USplineComponent* const SplineComponent = GetSplineComponent();
	if (!SplineComponent)
	{
		// does nothing if there's no spline
		return;
	}

	// instead of hashing the entire serialized spline component, we just record the package dependency,
	// and then hash the specific data that we use from the spline below
	Dependencies.AddPackageDependency(SplineComponent);

	const int32 PointsNeeded = SplineComponent->IsClosedLoop() ? 2 : 1;
	if (SplineComponent->GetNumberOfSplinePoints() < PointsNeeded)
	{
		return;
	}

	// serialize spline curves (via tagged properties), without defaulting as we want to detect changes to default data too
	FSplineCurves SplineCurves = SplineComponent->GetSplineCurves();
	UStruct* const SplineCurveStruct = FSplineCurves::StaticStruct();
	SplineCurveStruct->SerializeTaggedProperties(
		Dependencies.GetDependentDataArchive(), 
		reinterpret_cast<uint8*>(&SplineCurves),
		SplineCurveStruct, nullptr);

	Dependencies += SplineComponent->GetComponentToWorld();
	Dependencies += SplineComponent->DefaultUpVector;
	Dependencies += SplineRadius;

	Dependencies += bCreateVolumeFromClosedSpline;

	Dependencies += ComputeLocalBounds();
}

TArray<FBox> USplineRemeshModifier::ComputeBounds() const
{
	const USplineComponent* const SplineComponent = GetSplineComponent();
	if (!SplineComponent)
	{
		return {};
	}

	const FBox LocalBounds = ComputeLocalBounds();

	return { LocalBounds.TransformBy(GetComponentToWorld()) };
}

FBox USplineRemeshModifier::ComputeLocalBounds() const
{
	const USplineComponent* const SplineComponent = GetSplineComponent();
	if (!SplineComponent)
	{
		return FBox();
	}

	const Geometry::FAxisAlignedBox3d SplineBounds = SplineComponent->GetLocalBounds().GetBox();
	const FTransform& SplineTransform = SplineComponent->GetComponentToWorld();

	Geometry::FAxisAlignedBox3d LocalBounds;
	for (int i = 0; i < 8; ++i)
	{
		LocalBounds.Contain(GetComponentToWorld().InverseTransformPosition(
			SplineTransform.TransformPosition(SplineBounds.GetCorner(i))));
	}

	FBox ToReturn(LocalBounds);
	ToReturn = ToReturn.ExpandBy(SplineRadius);

	return ToReturn;
}


void USplineRemeshModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	using namespace MegaMeshSplineRemeshModifierLocals;

	const USplineComponent* const Spline = GetSplineComponent();
	
	const FColor LocalBoundsColor = FColor::Yellow;
	const FColor WorldBoundsColor = FColor::Orange;
	constexpr float BoundsThickness = 1;
	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	if (bDrawAffectedBox)
	{
		const Geometry::FAxisAlignedBox3d Coverage = ComputeLocalBounds();
		DrawWireBox(PDI, GetComponentTransform().ToMatrixWithScale(), FBox(Coverage),
			LocalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
	}

	if (bWorldBounds)
	{
		const Geometry::FAxisAlignedBox3d WorldBounds = ComputeCombinedBounds();
		DrawWireBox(PDI, FBox(WorldBounds), WorldBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
	}

	if (bDrawSplineRadius && Spline)
	{
		const double DeltaParam = 1.0 / (double)(SplineRadiusSamples + 1);
		const float SplineLength = Spline->GetSplineLength();
		for (int32 PointIndex = 0; PointIndex < SplineRadiusSamples; ++PointIndex)
		{
			const double Param = DeltaParam * (PointIndex + 1);		// 0 - 1
			const double Distance = Param * SplineLength;			// 0 - SplineLength
			const FTransform SampleTransform = Spline->GetTransformAtDistanceAlongSpline(Distance, ESplineCoordinateSpace::World);
			DrawCircle(PDI, SampleTransform.GetLocation(), SampleTransform.GetUnitAxis(EAxis::Y), SampleTransform.GetUnitAxis(EAxis::Z), FColor::Magenta, SplineRadius, 16, SDPG_World, 3.0, 0.0, true);
		}

		const FTransform BeginTransform = Spline->GetTransformAtDistanceAlongSpline(0.0, ESplineCoordinateSpace::World);
		DrawCircle(PDI, BeginTransform.GetLocation(), BeginTransform.GetUnitAxis(EAxis::Y), BeginTransform.GetUnitAxis(EAxis::Z), FColor::Magenta, SplineRadius, 16, SDPG_World, 3.0, 0.0, true);

		const FTransform EndTransform = Spline->GetTransformAtDistanceAlongSpline(SplineLength, ESplineCoordinateSpace::World);
		DrawCircle(PDI, EndTransform.GetLocation(), EndTransform.GetUnitAxis(EAxis::Y), EndTransform.GetUnitAxis(EAxis::Z), FColor::Magenta, SplineRadius, 16, SDPG_World, 3.0, 0.0, true);

		// TODO: Draw end caps
	}

	// draw surface if is exists
	const bool bCacheDataIsValid = CurrentCachedData && CurrentCachedData->bInitializationCompleted.load(std::memory_order_acquire) && CurrentCachedData->MeshedLoop;

	if (bDrawSurface && Spline && bCacheDataIsValid)
	{
		const FTransform SplineTransform = Spline->GetComponentToWorld();
		for (const FDynamicMesh3::FEdge Edge : CurrentCachedData->MeshedLoop->EdgesItr())
		{
			const FVector3d A = CurrentCachedData->MeshedLoop->GetVertex(Edge.Vert.A);
			const FVector3d NormalA = (FVector3d)CurrentCachedData->MeshedLoop->GetVertexNormal(Edge.Vert.A);
			const FVector3d B = CurrentCachedData->MeshedLoop->GetVertex(Edge.Vert.B);
			const FVector3d NormalB = (FVector3d)CurrentCachedData->MeshedLoop->GetVertexNormal(Edge.Vert.B);

			PDI->DrawLine(SplineTransform.TransformPosition(A + SplineRadius * NormalA), SplineTransform.TransformPosition(B + SplineRadius * NormalB), FColor::Magenta, 0, 3.0f, 1.0f, true);
			PDI->DrawLine(SplineTransform.TransformPosition(A - SplineRadius * NormalA), SplineTransform.TransformPosition(B - SplineRadius * NormalB), FColor::Magenta, 0, 3.0f, 1.0f, true);
		}
	}
}

FGuid USplineRemeshModifier::GetCodeVersionKey() const
{
	return FGuid::Combine(MegaMeshSplineRemeshModifierLocals::FMegaMeshSplineRemeshOp::GetCodeVersionKey(), MegaMeshSplineRemeshModifierLocals::FMegaMeshSplineTessellateOp::GetCodeVersionKey());
}
} // namespace UE::MeshPartition