// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Modifiers/Tessellation/MeshPostProcessing.h"
#include "Modifiers/Ops/MeshPartitionTexturePatchOp.h"

#include "AssetUtils/Texture2DUtil.h"
#include "Async/ParallelFor.h"
#include "Curves/CurveFloat.h"
#include "Engine/Texture.h"
#include "Spatial/SampledScalarField2.h"
#include "MathUtil.h" // SmoothMin
#include "MeshPartitionMeshView.h"
#include "MeshPartitionModule.h" // FCustomVersion
#include "Modifiers/MeshPartitionPatchModifier.h"
#include "OrientedBoxTypes.h"
#include "TextureResource.h"
#include "PrimitiveDrawingUtils.h"
#include "Templates/UnrealTemplate.h"
#include "DynamicMesh/MeshNormals.h"
#include "Tessellation/AdaptiveDisplacement.h"
#include "Image/DisplacementMap.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Image/ImageBlur.h"
#include "ImageCoreUtils.h"

#include "PropertyHandle.h"

#include <bit>
#include <limits>

#define ENABLE_VECTOR_INTRINSICS 1

namespace UE::MeshPartition
{

static Geometry::FSampledScalarField2f MoveIntoScalarField(Geometry::TDenseGrid2<float>&& Values)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexturePatchModifier::MoveIntoScalarField);

	const int64 TextureWidth = Values.Width();
	const int64 TextureHeight = Values.Height();
	// Need to be at least 2x2 to cover any ground between extremal pixels
	if (TextureWidth < 2 || TextureHeight < 2)
	{
		return {};
	}

	// We set things up such that we can index into the grid using Local2DCoordinates/UnscaledPatchCoverage,
	//  where the patch is centered at local (0,0). So, our domain is -0.5 to 0.5. The centers of the extremal
	//  pixels are at the corners, so the size in number of cells covering this range is actually one smaller
	//  than the dimensions.
	const FVector2f LocalGridOffset(-0.5f, -0.5f);
	const FVector2f LocalGridCellSize(1.0f / (TextureWidth - 1), 1.0f / (TextureHeight - 1));

	Geometry::FSampledScalarField2f Field;
	Field.GridValues = MoveTemp(Values);
	Field.GridOrigin = LocalGridOffset;
	Field.CellDimensions = LocalGridCellSize;

	return Field;
}

Geometry::FDisplacementMap MakeFilteredDisplacementMap(
                                   const Geometry::FSampledScalarField2f& ScalarField, const double SampleRate,
                                   const FVector2D UnscaledPatchCoverage)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexturePatchModifier::MakeFilteredDisplacementMap);

	Geometry::FSampledScalarField2f Prefiltered;
	Prefiltered.CopyConfiguration(ScalarField);
	Prefiltered.GridValues = ScalarField.GridValues;

	constexpr float PrefilteringFactor = 0.5;

	if (PrefilteringFactor > 0.f)
	{
		const float SigmaX  = PrefilteringFactor * 0.5f * Prefiltered.Width()  * SampleRate / UnscaledPatchCoverage[0]; 
		const float SigmaY  = PrefilteringFactor * 0.5f * Prefiltered.Height() * SampleRate / UnscaledPatchCoverage[1];
		Geometry::HeatEquationImplicitAOS(Prefiltered, SigmaX, SigmaY);
	}

	FImage PrefilteredImage(Prefiltered.Width(), Prefiltered.Height(), ERawImageFormat::R32F);
	float *const Dst = reinterpret_cast<float*>(PrefilteredImage.RawData.GetData());
	FMemory::Memcpy(Dst, Prefiltered.GridValues.GridValues().GetData(), Prefiltered.Width() * Prefiltered.Height() * sizeof(float));

	return Geometry::FDisplacementMap(
		MoveTemp(PrefilteredImage.RawData), FImageCoreUtils::ConvertToTextureSourceFormat(PrefilteredImage.Format),
		PrefilteredImage.SizeX, PrefilteredImage.SizeY,
		1.f, 0.f, TextureAddress::TA_Clamp, TextureAddress::TA_Clamp);
}

class FReadTextureAdapter
{
public:
	explicit FReadTextureAdapter(const UTexturePatchEntry& Channel);
	FReadTextureAdapter(FReadTextureAdapter&& Other);

	void MoveResultIntoScalarAndAlphaValues(TSharedPtr<Geometry::TDenseGrid2<float>>& Values, TSharedPtr<Geometry::TDenseGrid2<float>>& AlphaValues);

	bool HasValidData() const
	{
		return ScalarHash.IsValid();
	}

	FGuid ScalarHash;
	FGuid AlphaHash;

private:
	static bool IsTextureAssetValid(UTexture2D* InTexture);
	static bool ReadTexture(UTexture2D* TextureMap, uint8 ChannelMask, Geometry::TDenseGrid2<float>& DestImageOut);
	static bool ReadTexture(UTexture2D* TextureMap, uint8 ChannelMask, uint8 ChannelIndex0, uint8 ChannelIndex1, Geometry::TDenseGrid2<FVector2f>& DestImageOut);

	void ReadTextures(const UTexturePatchEntry& Channel);

	using FSingleValue = TSharedPtr<Geometry::TDenseGrid2<float>>;
	using FSingleValuePair = TPair<FSingleValue, FSingleValue>;
	using FTwoValues = TSharedPtr<Geometry::TDenseGrid2<FVector2f>>;
	using FResult = TVariant<FSingleValuePair, FTwoValues>;

	FResult ReadTextureResult;
};

FReadTextureAdapter::FReadTextureAdapter(const UTexturePatchEntry& Channel)
{
	ReadTextures(Channel);
}

FReadTextureAdapter::FReadTextureAdapter(FReadTextureAdapter&& Other)
	: ScalarHash(Other.ScalarHash)
	, AlphaHash(Other.AlphaHash)
	, ReadTextureResult(MoveTemp(Other.ReadTextureResult))
{
}

void FReadTextureAdapter::ReadTextures(const UTexturePatchEntry& Channel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexturePatchModifier::FReadTextureAdapter::ReadTextures);

	auto GetChannelMask = [](int32 ChannelIndex)
	{
		if (!ensure(0 <= ChannelIndex && ChannelIndex < 4))
		{
			ChannelIndex = FMath::Clamp(ChannelIndex, 0, 3);
		}
		return 0b0001 << (3 - ChannelIndex);
	};

	auto GetHash = [](UTexture2D& TextureAsset)
	{
		// instead of hashing the actual contents of the ScalarField, we will just use the texture source's already-computed hash
		// this includes unused channel data, but that's unlikely to be a big issue
		TextureAsset.Source.UseHashAsGuid();
		return TextureAsset.Source.GetId();
	};

	if (IsTextureAssetValid(Channel.TextureAsset))
	{
		switch (Channel.AlphaMode)
		{
		case ETexturePatchAlphaMode::AlwaysOne:
		case ETexturePatchAlphaMode::SelfMask:
			{
				FSingleValue ScalarValues = MakeShared<Geometry::TDenseGrid2<float>>();
				if (ReadTexture(Channel.TextureAsset, GetChannelMask(Channel.TextureChannelIndex), *ScalarValues))
				{
					ReadTextureResult = FResult{TInPlaceType<FSingleValuePair>(), FSingleValuePair{ScalarValues, FSingleValue{}}};
					ScalarHash = GetHash(*Channel.TextureAsset);
					AlphaHash = {};
					return;
				}
			}
			break;
		case ETexturePatchAlphaMode::ThisAlphaChannel:
			{
				if (Channel.TextureChannelIndex != Channel.AlphaChannelIndex)
				{
					FTwoValues ScalarAndAlphaValues = MakeShared<Geometry::TDenseGrid2<FVector2f>>();
					if (ReadTexture(Channel.TextureAsset, GetChannelMask(Channel.TextureChannelIndex) | GetChannelMask(Channel.AlphaChannelIndex), Channel.TextureChannelIndex, Channel.AlphaChannelIndex, *ScalarAndAlphaValues))
					{
						ReadTextureResult = FResult{TInPlaceType<FTwoValues>(), ScalarAndAlphaValues};
						ScalarHash = GetHash(*Channel.TextureAsset);
						AlphaHash = ScalarHash;
						return;
					}
				}
				else
				{
					FSingleValue ScalarAndAlphaValues = MakeShared<Geometry::TDenseGrid2<float>>();
					if (ReadTexture(Channel.TextureAsset, GetChannelMask(Channel.TextureChannelIndex), *ScalarAndAlphaValues))
					{
						ReadTextureResult = FResult{TInPlaceType<FSingleValuePair>(), FSingleValuePair{ScalarAndAlphaValues, ScalarAndAlphaValues}};
						ScalarHash = GetHash(*Channel.TextureAsset);
						AlphaHash = ScalarHash;
						return;
					}
				}
			}
			break;
		case ETexturePatchAlphaMode::OtherAlphaChannel:
			{
				ScalarHash = {};
				AlphaHash = {};

				FSingleValue ScalarValues = MakeShared<Geometry::TDenseGrid2<float>>();
				if (ReadTexture(Channel.TextureAsset, GetChannelMask(Channel.TextureChannelIndex), *ScalarValues))
				{
					ScalarHash = GetHash(*Channel.TextureAsset);
				}

				FSingleValue AlphaValues = MakeShared<Geometry::TDenseGrid2<float>>();
				if (IsTextureAssetValid(Channel.AlphaTextureAsset))
				{
					if (ReadTexture(Channel.AlphaTextureAsset, GetChannelMask(Channel.AlphaChannelIndex), *AlphaValues))
					{
						AlphaHash = GetHash(*Channel.AlphaTextureAsset);
					}
				}

				if (!ScalarHash.IsValid())
				{
					ScalarValues.Reset();
				}

				if (!AlphaHash.IsValid())
				{
					AlphaValues.Reset();
				}

				ReadTextureResult = FResult{TInPlaceType<FSingleValuePair>(), FSingleValuePair{ScalarValues, AlphaValues}};
				return;
			}
		default:
			ensure(false);
		}
	}

	ReadTextureResult = FResult{TInPlaceType<FSingleValuePair>(), FSingleValuePair{FSingleValue{}, FSingleValue{}}};
	ScalarHash = {};
	AlphaHash = {};
}

bool FReadTextureAdapter::IsTextureAssetValid(UTexture2D* InTexture)
{
	return (InTexture != nullptr) && (InTexture->GetPlatformData() != nullptr) && (InTexture->GetPlatformData()->Mips.Num() >= 1);
}

bool FReadTextureAdapter::ReadTexture(UTexture2D* TextureMap, uint8 ChannelMask, Geometry::TDenseGrid2<float>& DestImageOut)
{
	if (!ensureMsgf(ChannelMask == 0b1000 || ChannelMask == 0b0100 || ChannelMask == 0b0010 || ChannelMask == 0b0001,
					TEXT("Channel mask must have exactly one channel set.")))
	{
		return false;
	}

	Geometry::TImageBuilder<FVector4f> TempImage;
	if (!AssetUtils::ReadTexture(TextureMap, TempImage))
	{
		return false;
	}

	const int32 ChannelIndex = 3 - std::countr_zero(ChannelMask);

	DestImageOut.Resize(TempImage.GetDimensions().GetWidth(), TempImage.GetDimensions().GetHeight(), EAllowShrinking::Yes);
	for (int64 Index = 0, Num = TempImage.GetDimensions().Num(); Index < Num; ++Index)
	{
		const FVector4f& Pixel = TempImage.GetPixel(Index);
		DestImageOut[Index] = Pixel.Component(ChannelIndex);
	}

	return true;
}

bool FReadTextureAdapter::ReadTexture(UTexture2D* TextureMap, uint8 ChannelMask, uint8 ChannelIndex0, uint8 ChannelIndex1, Geometry::TDenseGrid2<FVector2f>& DestImageOut)
{
	if (!ensureMsgf(ChannelMask == 0b1100 || ChannelMask == 0b1010 || ChannelMask == 0b1001 ||
					ChannelMask == 0b0110 || ChannelMask == 0b0101 || ChannelMask == 0b0011,
					TEXT("Channel mask must have exactly two channels set.")))
	{
		return false;
	}
	
	if (!ensure(ChannelIndex0 < 4 && ChannelIndex1 < 4))
	{
		return false;
	}

	Geometry::TImageBuilder<FVector4f> TempImage;
	if (!AssetUtils::ReadTexture(TextureMap, TempImage))
	{
		return false;
	}

	DestImageOut.Resize(TempImage.GetDimensions().GetWidth(), TempImage.GetDimensions().GetHeight(), EAllowShrinking::Yes);
	for (int64 Index = 0, Num = TempImage.GetDimensions().Num(); Index < Num; ++Index)
	{
		const FVector4f& Pixel = TempImage.GetPixel(Index);
		DestImageOut[Index] = FVector2f(Pixel.Component(ChannelIndex0), Pixel.Component(ChannelIndex1));
	}

	return true;
}

void FReadTextureAdapter::MoveResultIntoScalarAndAlphaValues(TSharedPtr<Geometry::TDenseGrid2<float>>& ScalarValues,
                                                             TSharedPtr<Geometry::TDenseGrid2<float>>& AlphaValues)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexturePatchModifier::FReadTextureAdapter::MoveResultIntoScalarAndAlphaValues);

	if (ReadTextureResult.IsType<FSingleValuePair>())
	{
		FSingleValuePair& Pair = ReadTextureResult.Get<FSingleValuePair>();
		ScalarValues = MoveTemp(Pair.Key);
		AlphaValues = MoveTemp(Pair.Value);
	}
	else
	{
		check(ReadTextureResult.IsType<FTwoValues>());
		FTwoValues TwoValues = MoveTemp(ReadTextureResult.Get<FTwoValues>());
		check(TwoValues.IsValid());

		ScalarValues = MakeShared<Geometry::TDenseGrid2<float>>();
		AlphaValues = MakeShared<Geometry::TDenseGrid2<float>>();

		const int64 Width = TwoValues->Width();
		const int64 Height = TwoValues->Height();
		const int64 NumPixels = Width * Height;
		ScalarValues->Resize(Width, Height, EAllowShrinking::Yes);
		AlphaValues->Resize(Width, Height, EAllowShrinking::Yes);

#if ENABLE_VECTOR_INTRINSICS && PLATFORM_ENABLE_VECTORINTRINSICS

		const float* TwoValuesPtr = reinterpret_cast<const float*>(TwoValues->GridValues().GetData());
		float* ScalarValuesPtr = ScalarValues->GridValues().GetData();
		float* AlphaValuesPtr = AlphaValues->GridValues().GetData();

		// Iterate 4 pixels at a time, i.e. split 8 input floats into 4 output floats each.
		const int64 NumInputValues = NumPixels * 2;
		const int64 NumInputValuesMultipleOfEight = NumInputValues & ~7;
		int64 InputIndex, OutputIndex;
		for (InputIndex = 0, OutputIndex = 0; InputIndex < NumInputValuesMultipleOfEight; InputIndex += 8, OutputIndex += 4)
		{
			const VectorRegister4Float V0 = VectorLoad(&TwoValuesPtr[InputIndex]);
			const VectorRegister4Float V1 = VectorLoad(&TwoValuesPtr[InputIndex + 4]);

			const VectorRegister4Float VScalar = VectorShuffle(V0, V1, 0, 2, 0, 2);
			const VectorRegister4Float VAlpha = VectorShuffle(V0, V1, 1, 3, 1, 3);

			VectorStore(VScalar, &ScalarValuesPtr[OutputIndex]);
			VectorStore(VAlpha, &AlphaValuesPtr[OutputIndex]);
		}

		// Process remining pixels one by one, i.e. split 2 input floats into 1 output float each.
		for (; InputIndex < NumInputValues; InputIndex += 2, OutputIndex += 1)
		{
			ScalarValuesPtr[OutputIndex] = TwoValuesPtr[InputIndex];
			AlphaValuesPtr[OutputIndex] = TwoValuesPtr[InputIndex + 1];
		}

#else

		const FVector2f* TwoValuesPtr = TwoValues->GridValues().GetData();
		float* ScalarValuesPtr = ScalarValues->GridValues().GetData();
		float* AlphaValuesPtr = AlphaValues->GridValues().GetData();

		for (int64 Index = 0; Index < NumPixels; ++Index)
		{
			ScalarValuesPtr[Index] = TwoValuesPtr[Index].X;
			AlphaValuesPtr[Index] = TwoValuesPtr[Index].Y;
		}

#endif

		TwoValues.Reset();
	}
}

struct FChannelFieldData
{
	explicit FChannelFieldData(FReadTextureAdapter ReadTextureAdapter)
		: ScalarFieldHash(ReadTextureAdapter.ScalarHash)
		, AlphaFieldHash(ReadTextureAdapter.AlphaHash)
		, Data(MoveTemp(ReadTextureAdapter), [](FReadTextureAdapter ReadTextureAdapter)
			{
				TSharedPtr<Geometry::TDenseGrid2<float>> ScalarValues;
				TSharedPtr<Geometry::TDenseGrid2<float>> AlphaValues;
				ReadTextureAdapter.MoveResultIntoScalarAndAlphaValues(ScalarValues, AlphaValues);

				if (ensure(ScalarValues.IsValid()))
				{
					FData Result = {MoveIntoScalarField(MoveTemp(*ScalarValues))};
					// Avoid initializing the alpha field when we should be reusing the scalar field data for the alpha channel.
					if (ScalarValues != AlphaValues && AlphaValues.IsValid())
					{
						Result.AlphaField = MoveIntoScalarField(MoveTemp(*AlphaValues));
					}
					return Result;
				}
				return FData{};
			})
	{
	}

	const Geometry::FSampledScalarField2f& GetScalarField() const
	{
		return Data.GetResult().ScalarField;
	}

	const Geometry::FSampledScalarField2f* GetAlphaField() const
	{
		if (AlphaFieldHash.IsValid())
		{
			const FData& Result = Data.GetResult();
			if (Result.AlphaField.IsSet())
			{
				return &Result.AlphaField.GetValue();
			}
			else
			{
				return &Result.ScalarField;
			}
		}
		return nullptr;
	}

	const FGuid& GetScalarFieldHash() const
	{
		return ScalarFieldHash;
	}

	const FGuid& GetAlphaFieldHash() const
	{
		return AlphaFieldHash;
	}

	Tasks::FTask GetAsyncInitTask() const
	{
		return Data.GetAsyncInitTask();
	}

	TSharedRef<Utils::TAsyncTransform<Geometry::FDisplacementMap>> MakeDisplacementMap(double InFilterWidth, FVector2D InCoverage) const
	{
		return MakeShared<Utils::TAsyncTransform<Geometry::FDisplacementMap>>(Data.Chain<Geometry::FDisplacementMap>(
			[InFilterWidth, InCoverage](const FData& Payload) -> Geometry::FDisplacementMap
			{
				return MakeFilteredDisplacementMap(Payload.ScalarField, InFilterWidth, InCoverage);
			}));
	}

private:
	struct FData
	{
		Geometry::FSampledScalarField2f ScalarField;
		TOptional<Geometry::FSampledScalarField2f> AlphaField;
	};

	FGuid ScalarFieldHash;
	FGuid AlphaFieldHash;
	Utils::TAsyncTransform<FData> Data;
};

// Combines values in using the alpha
inline double BlendValues(
	double CurrentValue, double NewValue, double Alpha, MeshPartition::ETexturePatchBlendMode BlendMode,
	double MinMaxBlendDistance, double* MinMaxDeltaOut = nullptr)
{
	double DesiredValue = NewValue;
	switch (BlendMode)
	{
	case MeshPartition::ETexturePatchBlendMode::AlphaBlend:
		// Already initialized
		break;
	case MeshPartition::ETexturePatchBlendMode::Additive:
		DesiredValue = CurrentValue + NewValue;
		break;
	case MeshPartition::ETexturePatchBlendMode::Min:
		DesiredValue = FMathd::SmoothMin(CurrentValue, NewValue, MinMaxBlendDistance);
		if (MinMaxDeltaOut)
		{
			*MinMaxDeltaOut = DesiredValue - CurrentValue;
		}
		break;
	case MeshPartition::ETexturePatchBlendMode::Max:
		DesiredValue = FMathd::SmoothMax(CurrentValue, NewValue, MinMaxBlendDistance);
		if (MinMaxDeltaOut)
		{
			*MinMaxDeltaOut = CurrentValue - NewValue;
		}
		break;
	default:
		ensure(false);
	}

	return FMath::Lerp(CurrentValue, DesiredValue, Alpha);
}


double CalculateSampleRate(const MeshPartition::ETexturePatchTessellationErrorMode TessellationErrorMode, 
		                   const double TessellationError, const FVector2D UnscaledPatchCoverage)
{
	if (TessellationErrorMode == MeshPartition::ETexturePatchTessellationErrorMode::Relative)
	{
		const double PatchArea = UnscaledPatchCoverage.X * UnscaledPatchCoverage.Y;
		return 1.5f * TessellationError * 0.01f * FMath::Sqrt(2.f * PatchArea);
	}
	else
	{
		return TessellationError;
	}
}

// Acts as a functor to apply the vector displacement to vertices.
// 
// FMeshAccess provides get/set for vertex positions, weights and texture sampling.
template <typename FMeshAccessPolicy>
class FVertexDisplacement
{
public:
	using Op = FTexturePatchModifierOp;

	using FHeightEntry = FTexturePatchModifierOp::FHeightEntry;
	using FWeightEntry = FTexturePatchModifierOp::FWeightEntry;
	using FFalloffData = FTexturePatchModifierOp::FFalloffData;

	FVertexDisplacement(
		FMeshAccessPolicy& InMeshAccessPolicy,
		const Op& InBackgroundOp,
		const FTransform& InPatchTransform,
		FVector2D InLocalPatchExtent,
		const FTransform3d& InMeshToWorld)
		: MeshAccessPolicy(InMeshAccessPolicy)
		, BackgroundOp(InBackgroundOp)
		, PatchTransform(InPatchTransform)
		, LocalPatchExtent(InLocalPatchExtent)
		, LocalBounds(FVector3d(-LocalPatchExtent, -InBackgroundOp.VerticalPatchExtentDown),
					FVector3d( LocalPatchExtent,  InBackgroundOp.VerticalPatchExtentUp))
		, MeshToWorld(InMeshToWorld)
	{
		if (BackgroundOp.HeightChannel)
		{
			FTexturePatchModifierOp::PrepFalloffData(LocalPatchExtent, BackgroundOp.HeightChannel.GetValue(), HeightFalloff);
		}

		for (const FTexturePatchModifierOp::FWeightEntry& Entry : BackgroundOp.WeightChannels)
		{
			FTexturePatchModifierOp::FFalloffData& FalloffData = WeightFalloffs.Emplace_GetRef();
			FTexturePatchModifierOp::PrepFalloffData(LocalPatchExtent, Entry, FalloffData);
		}
	}

	inline FVector3d MeshToPatch(const FVector3d& MeshPosition) const
	{
		return PatchTransform.InverseTransformPosition(MeshToWorld.TransformPosition(MeshPosition));
	}

	inline FVector3d PatchToMesh(const FVector3d& PatchPosition) const
	{
		return MeshToWorld.InverseTransformPosition(PatchTransform.TransformPosition(PatchPosition));
	}

	inline FVector2f GetSamplingCoordinates(int32 VertexIndex) const
	{
		FVector PatchLocalVertPosition = MeshToPatch(MeshAccessPolicy.GetMeshVertexPosition(VertexIndex));
		FVector2f Local2DPosition = FVector2f(PatchLocalVertPosition.X, PatchLocalVertPosition.Y);
		FVector2f SamplingCoordinates = Local2DPosition / FVector2f(BackgroundOp.UnscaledPatchCoverage.X, BackgroundOp.UnscaledPatchCoverage.Y);
		return SamplingCoordinates;
	}

	// @return (vertex in component coordinates, sampling coordinates) iff inside local bounds, otherwise empty
	TOptional<TPair<FVector3d, FVector2f>> GetSamplingCoordinates(const FVector3d& MeshPosition) const
	{
		FVector3d PatchLocalVertPosition = MeshToPatch(MeshPosition);

		if (!LocalBounds.Contains(PatchLocalVertPosition))
		{
			// Vertex is outside of patch
			return {};
		}

		const FVector2f Local2DPosition = FVector2f(PatchLocalVertPosition.X, PatchLocalVertPosition.Y);
		const FVector2f SamplingCoordinates = Local2DPosition / FVector2f(BackgroundOp.UnscaledPatchCoverage.X, BackgroundOp.UnscaledPatchCoverage.Y);

		return { { PatchLocalVertPosition, SamplingCoordinates } };
	}

	// Apply the displacement to a single vertex
	TOptional<FVector3d> Displace(int32 VertexIndex, FVector3d& PatchLocalVertPosition, const FVector2f SamplingCoordinates, double& MinMaxHeightDelta )
	{
		// Amount by which the current value is below/above new value in min/max blend mode, respectively. Used to falloff weights when
		// using bApplyHeightMinMaxBlend (a higher value means that the weight should be masked out more).
		MinMaxHeightDelta = 0.;

		if (BackgroundOp.HeightChannel.IsSet())
		{
			const FVector2d Local2DPosition = FVector2d(PatchLocalVertPosition.X, PatchLocalVertPosition.Y);
			const FHeightEntry& HeightEntry = BackgroundOp.HeightChannel.GetValue();
			const double TextureValue = FTexturePatchModifierOp::ApplyCurve(HeightEntry, MeshAccessPolicy.GetTextureValue(HeightEntry, SamplingCoordinates));

			// Height is frequently not given in the same scale inside the texture, so it gets adjusted.
			double AdjustedValue = (TextureValue - BackgroundOp.HeightChannel->ZeroInEncoding) * BackgroundOp.HeightChannel->EncodingScale;
			AdjustedValue *= MeshAccessPolicy.GetHeightScale(VertexIndex);

			double Alpha = FTexturePatchModifierOp::GetAlphaValue(HeightEntry, SamplingCoordinates, TextureValue);
			Alpha *= FTexturePatchModifierOp::GetFalloffAlpha(LocalPatchExtent, Local2DPosition, HeightEntry, HeightFalloff);

			// This is not in the if statement below because the weight application may need to know the MinMaxHeightDelta
			//  if we're using Min/Max blend mode.
			const double BlendedValue = BlendValues(PatchLocalVertPosition.Z, AdjustedValue, Alpha, HeightEntry.BlendMode,
				HeightEntry.SoftnessParameter, &MinMaxHeightDelta);

			if (Alpha != UE_DOUBLE_SMALL_NUMBER)
			{
				PatchLocalVertPosition.Z = BlendedValue;
				PatchLocalVertPosition.Z = FMath::Clamp(PatchLocalVertPosition.Z, LocalBounds.Min.Z, LocalBounds.Max.Z);

				return PatchToMesh(PatchLocalVertPosition);
			}
		}

		return {};
	}


	// Apply the displacement to a single vertex
	void ProcessVertex(const int32 VertexIndex)
	{
		const FVector3d MeshPosition = MeshAccessPolicy.GetMeshVertexPosition(VertexIndex);

		TOptional<TPair<FVector3d, FVector2f>> ComponentPosAndCoords = GetSamplingCoordinates(MeshPosition);

		if (!ComponentPosAndCoords)
		{
			// Vertex is outside of patch
			return;
		}
		FVector3d PatchLocalVertPosition = ComponentPosAndCoords->Get<0>();
		const FVector2f SamplingCoordinates = ComponentPosAndCoords->Get<1>();

		const FVector2d Local2DPosition = FVector2d(PatchLocalVertPosition.X, PatchLocalVertPosition.Y);
		
		double MinMaxHeightDelta = 0.;
		TOptional<FVector3d> DisplacedPos = Displace(VertexIndex, PatchLocalVertPosition, SamplingCoordinates, MinMaxHeightDelta);
		if (DisplacedPos)
		{
			MeshAccessPolicy.SetMeshVertexPosition(VertexIndex, *DisplacedPos);
		}
		
		for (int32 EntryIndex = 0; EntryIndex < BackgroundOp.WeightChannels.Num(); ++EntryIndex)
		{
			const FWeightEntry& WeightChannelEntry = BackgroundOp.WeightChannels[EntryIndex];
			if (WeightChannelEntry.WeightChannelName.IsNone())
			{
				continue;
			}

			const double TextureValue = FTexturePatchModifierOp::ApplyCurve(WeightChannelEntry, WeightChannelEntry.FieldData->GetScalarField().BilinearSampleClamped(SamplingCoordinates));

			// As with height, additional zero and scale adjustment happens after the curve-based adjustment above
			const double Value = (TextureValue - WeightChannelEntry.ZeroInEncoding) * WeightChannelEntry.EncodingScale;

			double Alpha = FTexturePatchModifierOp::GetAlphaValue(WeightChannelEntry, SamplingCoordinates, Value);
			Alpha *= FTexturePatchModifierOp::GetFalloffAlpha(LocalPatchExtent, Local2DPosition, WeightChannelEntry, WeightFalloffs[EntryIndex]);

			if (WeightChannelEntry.bApplyHeightMinMaxBlend && MinMaxHeightDelta > -WeightChannelEntry.HeightMinMaxBlendDistance)
			{
				Alpha *= (WeightChannelEntry.HeightMinMaxBlendDistance == 0.0) ? 0.0 // Known to be positive if HeightMinMaxBlendDistance was 0, so mask out
					// Already clamped to be lower than 1 by the if statement above
					: FMath::Max(0.5 - MinMaxHeightDelta / (2.0 * WeightChannelEntry.HeightMinMaxBlendDistance), 0.0);
			}

			if (Alpha >= UE_DOUBLE_SMALL_NUMBER)
			{
				const double CurrentValue = MeshAccessPolicy.GetVertexAttributeWeight(EntryIndex, WeightChannelEntry.WeightChannelName, VertexIndex);
				const double BlendedValue = BlendValues(CurrentValue, Value, Alpha, WeightChannelEntry.BlendMode, WeightChannelEntry.SoftnessParameter);
				MeshAccessPolicy.SetVertexAttributeWeight(EntryIndex, WeightChannelEntry.WeightChannelName, VertexIndex, BlendedValue);
			} 
		}
	}
private:
	FMeshAccessPolicy& MeshAccessPolicy;
	const Op& BackgroundOp;
	const FTransform& PatchTransform;
	FVector2D LocalPatchExtent;
	const Geometry::FAxisAlignedBox3d LocalBounds;
	const FTransform3d& MeshToWorld;
	FTexturePatchModifierOp::FFalloffData HeightFalloff;
	TArray<FTexturePatchModifierOp::FFalloffData> WeightFalloffs;
};

// Mesh access for FVertexDisplacement on FDynamicMesh3 directly
class FMeshAccessPolicy : public FNoncopyable
{
public:
	FMeshAccessPolicy(
		FDynamicMesh3& InMesh,
		TArray<Geometry::FDynamicMeshWeightAttribute*>& InWeightLayers,
		const FName& InHeightScaleWeightChannel)
		: Mesh(InMesh)
		, WeightLayers(InWeightLayers)
		, HeightScaleWeightChannel(InHeightScaleWeightChannel)
	{
		if (!HeightScaleWeightChannel.IsNone())
		{
			const Geometry::FDynamicMeshAttributeSet* const AttributeSet = Mesh.Attributes();
			if (AttributeSet)
			{
				for (int32 LayerIdx = 0; LayerIdx < AttributeSet->NumWeightLayers(); ++LayerIdx)
				{
					const Geometry::FDynamicMeshWeightAttribute* WeightLayer = AttributeSet->GetWeightLayer(LayerIdx);
					if (WeightLayer->GetName() == HeightScaleWeightChannel)
					{
						HeightScaleLayer = WeightLayer;
					}
				}
			}
		}
	}

	FVector3d GetMeshVertexPosition(const int32 VertexIndex) const
	{
		return Mesh.GetVertex(VertexIndex);
	}

	void SetMeshVertexPosition(const int32 VertexIndex, const FVector3d Position)
	{
		Mesh.SetVertex(VertexIndex, Position);
	}

	double GetVertexAttributeWeight(const int32 WeightChannelIndex, const FName& WeightChannelName, int32 VertexIndex) const
	{
		float ValueToReturn;
		WeightLayers[WeightChannelIndex]->GetValue<float* const>(VertexIndex, &ValueToReturn);
		return ValueToReturn;
	}

	void SetVertexAttributeWeight(const int32 WeightChannelIndex, const FName& WeightChannelName, int32 VertexIndex, double Value)
	{
		WeightLayers[WeightChannelIndex]->SetScalarValue(VertexIndex, Value);
	}

	float GetTextureValue(const FTexturePatchModifierOp::FHeightEntry& HeightEntry, const FVector2f& Coords) const
	{
		return HeightEntry.DisplacementMap->GetResult().Sample(Coords + 0.5f);
	}

	float GetTextureValue(const FTexturePatchModifierOp::FHeightEntry& HeightEntry, const int32 VertexIndex, const FVector2f& Coords) const
	{
		return GetTextureValue(HeightEntry, Coords);
	}

	float GetHeightScale(const int32 VertexIndex) const
	{
		if (HeightScaleWeightChannel.IsNone())
		{
			return 1.f;
		}

		if (!HeightScaleLayer)
		{
			return 0.f;
		}

		float ValueToReturn;
		HeightScaleLayer->GetValue<float* const>(VertexIndex, &ValueToReturn);
		return ValueToReturn;
	}

protected:
	FDynamicMesh3& Mesh;
	TArray<Geometry::FDynamicMeshWeightAttribute*>& WeightLayers;
	const FName HeightScaleWeightChannel;
	const Geometry::FDynamicMeshWeightAttribute* HeightScaleLayer { nullptr };
};

// When applying EWA, the sampling coordinates need to be immutable while displacement is applied.
// Here we store a sampling coordinates per-vertex used during filtered GetTextureValue.
class FMeshAccessPolicyEWA : public FMeshAccessPolicy
{
public:
	FMeshAccessPolicyEWA(
		FDynamicMesh3& InMesh,
		TArray<Geometry::FDynamicMeshWeightAttribute*>& InWeightLayers,
		const FName& InHeightScaleWeightChannel,
		const bool bInEWAFiltering)
		: FMeshAccessPolicy(InMesh, InWeightLayers, InHeightScaleWeightChannel)
		, bEWAFiltering(bInEWAFiltering)
	{
		SamplingCoordinates.SetNumUninitialized(InMesh.MaxVertexID());
	}

	void SetSamplingCoordinates(int32 VertexIndex, const FVector2f VertexSamplingCoords)
	{
		SamplingCoordinates[VertexIndex] = VertexSamplingCoords;
	}

	using FMeshAccessPolicy::GetTextureValue;

	// potentially EWA-filtered texture lookup
	double GetTextureValue(const FTexturePatchModifierOp::FHeightEntry& HeightEntry, const int32 VertexIndex, const FVector2D& ) const
	{
		const FVector2f Center = SamplingCoordinates[VertexIndex];

		if (bEWAFiltering)
		{
			// extract elliptic filter region
			float MaxDistSqr = 0.f;
			float MinDistSqr = std::numeric_limits<float>::max();

			FVector2f MajorAxis(0.f, 0.f); 
							
			// walk through one-ring to get major and minor axes
			for (int32 NbVertexIdx : Mesh.VtxVerticesItr(VertexIndex))
			{
				const FVector2f UV = SamplingCoordinates[NbVertexIdx];
				const float DistSqr = (UV - Center).SquaredLength();
				if (DistSqr > MaxDistSqr)
				{
					MaxDistSqr = DistSqr;
					MajorAxis = UV - Center;
				}
				MinDistSqr = FMath::Min(MinDistSqr, DistSqr);
			}

			float MaxDist = FMath::Sqrt(MaxDistSqr);
			float MinDist = FMath::Sqrt(MinDistSqr);
			
			// maximum anisotropy 4x
			if (MaxDist > 4.f * MinDist)
			{
				MajorAxis *= 4.f * MinDist / MaxDist;
				MaxDist = 4.f * MinDist;
			}

			// Minor axis is rescaled version of major axis rotated by 90 degrees 
			const FVector2f MinorAxis = FVector2f(MajorAxis.Y, -MajorAxis.X) / MaxDist * MinDist; 
			return HeightEntry.DisplacementMap->GetResult().SampleEWA(FVector2f(Center.X + 0.5f, Center.Y + 0.5f), MajorAxis, MinorAxis);
		}
		else
		{
			return HeightEntry.DisplacementMap->GetResult().Sample(FVector2f(Center.X + 0.5f, Center.Y + 0.5f));
		}
	}

private:
	TArray<FVector2f> SamplingCoordinates;
	const bool bEWAFiltering;
};

// Mesh access for FVertexDisplacement on MegaMeshMeshView (for non-adaptive displacement)
class FMeshViewAccessPolicy
{
public:
	FMeshViewAccessPolicy(
		MeshPartition::FMeshView& InMeshView,
		const FName InHeightScaleChannelName)
		: MeshView(InMeshView)
		, HeightScaleChannelName(InHeightScaleChannelName)
	{
	}

	FVector3d GetMeshVertexPosition(const int32 VertexIndex) const
	{
		return MeshView.GetVertexPos(VertexIndex);
	}

	void SetMeshVertexPosition(const int32 VertexIndex, const FVector3d Position)
	{
		MeshView.SetVertexPos(VertexIndex, Position);
	}

	double GetVertexAttributeWeight(const int32 WeightChannelIndex, const FName& WeightChannelName, int32 VertexIndex) const
	{
		return MeshView.GetVertexAttributeWeight(WeightChannelName, VertexIndex);
	}

	void SetVertexAttributeWeight(const int32 WeightChannelIndex, const FName& WeightChannelName, int32 VertexIndex, double Value)
	{
		MeshView.SetVertexAttributeWeight(WeightChannelName, VertexIndex, Value);
	}

	float GetTextureValue(const FTexturePatchModifierOp::FHeightEntry& HeightEntry, const FVector2f& SamplingCoordinates) const
	{
		return HeightEntry.DisplacementMap->GetResult().Sample(SamplingCoordinates + 0.5f);
	}

	float GetTextureValue(const FTexturePatchModifierOp::FHeightEntry& HeightEntry, const int32 VertexIndex, const FVector2f& SamplingCoordinates) const
	{
		return GetTextureValue(HeightEntry, SamplingCoordinates);
	}

	float GetHeightScale(const int32 VertexIndex) const
	{
		if (HeightScaleChannelName.IsNone())
		{
			return 1.f;
		}
		return MeshView.GetVertexAttributeWeight(HeightScaleChannelName, VertexIndex);
	}

private:
	MeshPartition::FMeshView& MeshView;
	const FName HeightScaleChannelName;
};

double FTexturePatchModifierOp::GetFalloffAlpha(const FVector2d LocalPatchExtent, const FVector2d Local2DPosition, const FTextureEntry& Entry, const FFalloffData& FalloffData)
{
	if (Entry.FalloffDistance <= 0 && FalloffData.ClampedRadius <= 0)
	{
		return 1.0;
	}

	double DistanceFromEdge = -1;
	FVector2D Abs2DCoordinates(FMath::Abs(Local2DPosition.X), FMath::Abs(Local2DPosition.Y));

	// See if we're in the portion where we have to worry about the rounded corners
	if (FalloffData.ClampedRadius > 0
		&& Abs2DCoordinates.X > FalloffData.CornerCenter.X
		&& Abs2DCoordinates.Y > FalloffData.CornerCenter.Y)
	{
		DistanceFromEdge = FalloffData.ClampedRadius - (Abs2DCoordinates - FalloffData.CornerCenter).Length();
		if (DistanceFromEdge < 0)
		{
			// This means we're outside the corner radius, where we should not apply at all.
			return 0.0;
		}
	}

	if (Entry.FalloffDistance <= 0)
	{
		// Don't have regular falloff to apply, and already checked corner containment
		return 1.0;
	}

	if (DistanceFromEdge < 0) // If we didn't already get a distance from the corner check
	{
		DistanceFromEdge = (LocalPatchExtent - Abs2DCoordinates).GetMin();
	}

	// Should be the case because we shouldn't be dealing with points outside our extent, and
	//  we already have an early out above if we're outside the rounded corners or there is
	//  no falloff.
	checkSlow(DistanceFromEdge >= 0);

	// We handle the case of falloff being larger than our extents. For linear falloff we could just
	//  divide distance from edge by unclamped falloff distance and call it a day, because the center
	//  will be appropriately lowered (imagine a truncated pyramid's sides becoming less steep until
	//  we have a pyramid shrinking in height).
	// However this isn't good enough for any other falloff function, where we lose the smoothness of
	//  the end by doing it this way (imagine a sigmoid being cut in the middle). Instead, we divide
	//  distance from edge by clamped falloff distance, and then multiply by a factor by which a linear
	//  falloff would have decreased at the innermost end of the falloff (to lower our "pyramid").
	double FalloffMultiplier = FalloffData.ClampedFalloff / Entry.FalloffDistance;

	if (DistanceFromEdge >= FalloffData.ClampedFalloff)
	{
		// We're in the region that is "full" value
		return FalloffMultiplier;
	}

	double FalloffInput = DistanceFromEdge / FalloffData.ClampedFalloff;

	switch (FalloffData.FalloffModeToUse)
	{
	case MeshPartition::ETexturePatchFalloffMode::Linear:
		return FalloffMultiplier * FalloffInput;
	case MeshPartition::ETexturePatchFalloffMode::Smooth:
		return FalloffMultiplier * FMath::SmoothStep(0.0, 1.0, FalloffInput);
	case MeshPartition::ETexturePatchFalloffMode::CustomCurve:
		return FalloffMultiplier * Entry.FalloffCurve->Eval(FalloffInput);
	}
	ensure(false);
	return 1.0;
}

double FTexturePatchModifierOp::ApplyCurve(const FTextureEntry& InTextureEntry, double Value)
{
	if (InTextureEntry.ValueCurve != nullptr)
	{
		Value = InTextureEntry.ValueCurve->Eval((Value - InTextureEntry.ValueCurveOffset) / InTextureEntry.ValueCurveScale)
			* InTextureEntry.ValueCurveScale + InTextureEntry.ValueCurveOffset;
	}
	return Value;
}

float FTexturePatchModifierOp::GetAlphaValue(const FTextureEntry& InTextureEntry, const FVector2f& InSamplingCoordinates, float Value)
{
	switch (InTextureEntry.AlphaMode)
	{
	case ETexturePatchAlphaMode::AlwaysOne:
		return 1.0f;
	case ETexturePatchAlphaMode::SelfMask:
		return FMath::Abs(Value) <= InTextureEntry.SelfMaskTolerance ? 0.0f : 1.0f;
	case ETexturePatchAlphaMode::ThisAlphaChannel:
	case ETexturePatchAlphaMode::OtherAlphaChannel:
		if (const Geometry::FSampledScalarField2f* AlphaField = InTextureEntry.FieldData->GetAlphaField())
		{
			return AlphaField->BilinearSampleClamped(InSamplingCoordinates);
		}
		// If we didn't have an alpha channel, we'll just assume it's all 1.0
		return 1.0f;
	}
	return 0.0f;
}

void FTexturePatchModifierOp::PrepFalloffData(const FVector2d& LocalPatchExtent, const FTextureEntry& TextureEntry, FFalloffData& Data)
{
	double ExtentMin = LocalPatchExtent.GetMin();
	Data.ClampedRadius = FMath::Min(TextureEntry.CornerRadius, ExtentMin);
	Data.ClampedFalloff = FMath::Min(TextureEntry.FalloffDistance, ExtentMin);
	Data.CornerCenter = LocalPatchExtent - Data.ClampedRadius;
	// Switch falloff mode to smooth if we are set to curve but don't have curve data
	Data.FalloffModeToUse =
		TextureEntry.FalloffMode == MeshPartition::ETexturePatchFalloffMode::CustomCurve && !TextureEntry.FalloffCurve ?
		MeshPartition::ETexturePatchFalloffMode::Smooth : TextureEntry.FalloffMode;
}

void FTexturePatchModifierOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	// Don't need to check HasScalarField because that is done when creating the op
	
	for (int32 InstanceID = 0; InstanceID < GlobalBounds.Num(); ++InstanceID)
	{
		const FBox& Bounds = GlobalBounds[InstanceID];
		
		if (!Bounds.Intersect(InBounds))
		{
			continue;
		}

		FInstanceInfo& Info = OutInstanceInfos.Emplace_GetRef();
		Info.Bounds = Bounds;
		Info.InstanceID = InstanceID;

		if (ShouldApplyAdaptiveDisplacement())
		{
			Info.ReadViewComponents  = EMeshViewComponents::DynamicSubmesh;
			Info.WriteViewComponents = EMeshViewComponents::DynamicSubmesh;
		}
		else
		{
			// We need the vert positions regardless of whether we're modifying height or weights
			Info.ReadViewComponents = EMeshViewComponents::VertexPos;

			if (HeightChannel.IsSet())
			{
				if (!HeightChannel->HeightScaleWeightChannel.IsNone())
				{
					Info.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
					Info.UsedChannels.Emplace(HeightChannel->HeightScaleWeightChannel);
				}
				Info.WriteViewComponents |= EMeshViewComponents::VertexPos;
			}
		}

		for (const FWeightEntry& WeightChannelEntry : WeightChannels)
		{
			if (!WeightChannelEntry.WeightChannelName.IsNone())
			{
				if (!ShouldApplyAdaptiveDisplacement())
				{
					Info.ReadViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				}
				Info.WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
				Info.UsedChannels.Emplace(WeightChannelEntry.WeightChannelName);
			}
		}
	}
}

void FTexturePatchModifierOp::ApplyAdaptiveDisplacement(MeshPartition::FMeshView& InMeshView,
	const FTransform3d& MeshToWorld, const FInstanceInfo& InInstanceDesc) const
{
	if (!ensure(HeightChannel) || !ensure(PatchTransforms.IsValidIndex(InInstanceDesc.InstanceID)))
	{
		// we still would need to apply the weight layers, but in case no height channel is specified we go through the non-adaptive
		// code path which handles that
		return;
	}
	
	FTransform PatchTransform = PatchTransforms[InInstanceDesc.InstanceID];

	FDynamicMesh3& Mesh = InMeshView.GetSubmeshMutable();
	const FHeightEntry& HeightEntry = HeightChannel.GetValue();

	const double SampleRate = CalculateSampleRate(TessellationErrorMode, TessellationError, UnscaledPatchCoverage);

	// Build the weight layers list that we need to write to
	Geometry::FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	check(AttributeSet != nullptr);

	const Geometry::TDynamicMeshVertexAttribute<float, 1>* HeightScaleLayer = nullptr;
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* SampleRateScaleLayer = nullptr;
	if (AttributeSet && (!HeightEntry.HeightScaleWeightChannel.IsNone() || !DetailAdjustmentChannelName.IsNone()))
	{
		for (int32 LayerIdx = 0; LayerIdx < AttributeSet->NumWeightLayers(); ++LayerIdx)
		{
			const Geometry::FDynamicMeshWeightAttribute* WeightLayer = AttributeSet->GetWeightLayer(LayerIdx);
			if (WeightLayer->GetName() == HeightEntry.HeightScaleWeightChannel)
			{
				HeightScaleLayer = WeightLayer;
			}
			
			if (WeightLayer->GetName() == DetailAdjustmentChannelName)
			{
				SampleRateScaleLayer = WeightLayer;
			}
		}
	}

	Geometry::TDynamicMeshVertexAttribute<float, 1>* ZeroHeightScaleLayer = nullptr;
	const FName ZeroHeightScaleLayerName = "AdaptiveTess.ZeroHeightScaleLayer";
	if (AttributeSet && !HeightEntry.HeightScaleWeightChannel.IsNone() && !HeightScaleLayer)
	{
		// if the height scale channel name is set, but the channel is not existing we need to ensure behavior 
		// consistent with the non-adaptive case, where values are queried from the meshview is to return the background value.
		// therefore we supply a dummy layer here to avoid passing through another flag through the interfaces.
		// otherwise the displacement would still be zero, because it's applied after the tessellation, but unnecessarily
		// refining.
		 
		ZeroHeightScaleLayer = new Geometry::TDynamicMeshVertexAttribute<float, 1>(&Mesh);
		AttributeSet->AttachAttribute(ZeroHeightScaleLayerName, ZeroHeightScaleLayer);
		HeightScaleLayer = ZeroHeightScaleLayer;
	}

	// apply the adaptive tessellation (without displacement). this should preserve existing UV and weight layers
	TessellateAdaptive(
		Mesh, 
		MeshToWorld,
		PatchTransform, 
		UnscaledPatchCoverage,
		HeightEntry.DisplacementMap->GetResult(), 
		HeightEntry.ZeroInEncoding,
		HeightEntry.EncodingScale,
		TessellationMode == MeshPartition::ETexturePatchTessellationMode::AdaptiveFast, 
		SampleRate, 
		MinimumEdgeLength, 
		MaximumEdgeLength,
		FeatureSensitivity,
		SampleRateScaleLayer,
		HeightScaleLayer);

	// Update AttributeSet pointer as the TessellateAdaptive call above may have invalidated it
	AttributeSet = Mesh.Attributes();
	check(AttributeSet != nullptr);

	if (AttributeSet && ZeroHeightScaleLayer)
	{
		AttributeSet->RemoveAttribute(ZeroHeightScaleLayerName);
	}

	TArray<Geometry::FDynamicMeshWeightAttribute*> WeightLayers;
	WeightLayers.Init(nullptr, WeightChannels.Num());

	if (AttributeSet)
	{
		for (int32 EntryIndex = 0; EntryIndex < WeightChannels.Num(); ++EntryIndex)
		{
			const FWeightEntry& WeightChannelEntry = WeightChannels[EntryIndex];

			if (WeightChannelEntry.WeightChannelName.IsNone())
			{
				continue;
			}

			// search the weight layer in the existing attributes of the submesh
			bool NeedsInsert = true;
			for (int32 i = 0; i < AttributeSet->NumWeightLayers(); ++i)
			{
				Geometry::FDynamicMeshWeightAttribute* weightLayer = AttributeSet->GetWeightLayer(i);
				if (weightLayer->GetName() == WeightChannelEntry.WeightChannelName)
				{
					WeightLayers[EntryIndex] = weightLayer;
					NeedsInsert = false;
					break;
				}
			}

			if (NeedsInsert)
			{
				// doesn't exist -> insert 
				AttributeSet->SetNumWeightLayers(AttributeSet->NumWeightLayers() + 1);
				Geometry::FDynamicMeshWeightAttribute* weightLayer = AttributeSet->GetWeightLayer(AttributeSet->NumWeightLayers() - 1);
				weightLayer->SetName(WeightChannelEntry.WeightChannelName);
				WeightLayers[EntryIndex] = weightLayer;
			}
		}
	}
	
	const FVector2D LocalPatchExtent = UnscaledPatchCoverage / 2.0;

	
	// Re-extract height scale layer from new mesh, the previous may be invalid as the parallel tessellation
	// reassigns a new interpolated one at the final resolution.
	if (AttributeSet && !HeightEntry.HeightScaleWeightChannel.IsNone())
	{
		for (int32 LayerIdx = 0; LayerIdx < AttributeSet->NumWeightLayers(); ++LayerIdx)
		{
			const Geometry::FDynamicMeshWeightAttribute* WeightLayer = AttributeSet->GetWeightLayer(LayerIdx);
			if (WeightLayer->GetName() == HeightEntry.HeightScaleWeightChannel)
			{
				HeightScaleLayer = WeightLayer;
			}
		}
	}

	if (bMeshRegularization)
	{
		FMeshAccessPolicy MeshAccessPolicy(Mesh, WeightLayers, HeightEntry.HeightScaleWeightChannel);
		FVertexDisplacement<FMeshAccessPolicy> VertexDisplacement(MeshAccessPolicy, *this, PatchTransform, LocalPatchExtent, MeshToWorld);

		TArray<FVector3d> Displacements;
		Displacements.SetNum(Mesh.MaxVertexID());

		const auto DisplaceFunc = [&VertexDisplacement, &Mesh](const int32 VID) 
		{
			const FVector3d MeshPosition = Mesh.GetVertex(VID);
			TOptional<TPair<FVector3d, FVector2f>> ComponentPosAndCoords = VertexDisplacement.GetSamplingCoordinates(MeshPosition);

			if (!ComponentPosAndCoords)
			{
				// Vertex is outside of patch
				return FVector3d(0.);
			}

			double MinMaxHeightDelta;
			TOptional<FVector3d> Displacement = VertexDisplacement.Displace(VID, ComponentPosAndCoords->Get<0>(), ComponentPosAndCoords->Get<1>(), MinMaxHeightDelta);
			return Displacement ? *Displacement : FVector3d(0.);
		};

		for (int32 VID : Mesh.VertexIndicesItr())
		{	
			Displacements[VID] = DisplaceFunc(VID);
		}

		PostProcessMesh(Mesh, Displacements, DisplaceFunc, MinimumEdgeLength, MaximumEdgeLength);
	}

	constexpr bool bEWAFiltering = true;
	FMeshAccessPolicyEWA MeshAccessPolicy(Mesh, WeightLayers, HeightEntry.HeightScaleWeightChannel, bEWAFiltering);
	
	ParallelFor(Mesh.MaxVertexID(), [&](int32 VertexIndex) 
	{
		if (!ensure(Mesh.IsVertex(VertexIndex))) // expect new mesh to be compact
		{
			return;
		}
		const FVector PatchLocalVertPosition = PatchTransform.InverseTransformPosition(MeshToWorld.TransformPosition(MeshAccessPolicy.GetMeshVertexPosition(VertexIndex)));
		const FVector2d Local2DPosition = FVector2d(PatchLocalVertPosition.X, PatchLocalVertPosition.Y);
		MeshAccessPolicy.SetSamplingCoordinates(VertexIndex, FVector2f(Local2DPosition / UnscaledPatchCoverage));
	});

	FVertexDisplacement<FMeshAccessPolicy> VertexDisplacement(MeshAccessPolicy, *this, PatchTransform, LocalPatchExtent, MeshToWorld);

	ParallelFor(Mesh.MaxVertexID(), [&VertexDisplacement](int32 VertexIndex)
	{ 
		VertexDisplacement.ProcessVertex(VertexIndex);	
	});

	if (bMeshRegularization)
	{
		RegularizeMesh(Mesh);
	}

	// After displacement, make sure normals overlay exists, compute normals
	{
		Geometry::FMeshNormals MeshNormals(&Mesh);
		// recompute normals
		MeshNormals.ComputeVertexNormals();
		Geometry::FDynamicMeshNormalOverlay* DestNormalsOverlay = Mesh.Attributes()->GetNormalLayer(0);
		DestNormalsOverlay->InitializeTriangles(Mesh.MaxTriangleID());

		for (int32 VertexIdx : Mesh.VertexIndicesItr())
		{
			DestNormalsOverlay->AppendElement(FVector3f(MeshNormals[VertexIdx]));
		}

		for (int32 TriIdx : Mesh.TriangleIndicesItr())
		{
			DestNormalsOverlay->SetTriangle(TriIdx, Mesh.GetTriangle(TriIdx));
		}
	}
}

void FTexturePatchModifierOp::ApplyDisplacement(MeshPartition::FMeshView& InMeshView,
	const FTransform3d& MeshToWorld, const FInstanceInfo& InInstanceDesc) const
{
	FMeshViewAccessPolicy MeshAccessPolicy(InMeshView, HeightChannel.IsSet() ? HeightChannel->HeightScaleWeightChannel : FName());
	
	if (!ensure(PatchTransforms.IsValidIndex(InInstanceDesc.InstanceID)))
	{
		return;
	}
	
	FTransform PatchTransform = PatchTransforms[InInstanceDesc.InstanceID];

	const FVector2D LocalPatchExtent = UnscaledPatchCoverage / 2.0;
	FVertexDisplacement<FMeshViewAccessPolicy> VertexDisplacement(MeshAccessPolicy, *this, PatchTransform, LocalPatchExtent, MeshToWorld);

	ParallelFor(InMeshView.VertexCount(), [&VertexDisplacement](int32 VertexIndex) { 
		VertexDisplacement.ProcessVertex(VertexIndex); 
	});
}

void FTexturePatchModifierOp::ApplyModifications(MeshPartition::FMeshView& InMeshView,
	const FTransform3d& MegameshTransform, const FInstanceInfo& InInstanceDesc) const
{
	if (ShouldApplyAdaptiveDisplacement())
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UTexturePatchModifier::ApplyModifications::AdaptiveDisplacement);
		ensure(EnumHasAnyFlags(InInstanceDesc.WriteViewComponents, EMeshViewComponents::DynamicSubmesh));

		ApplyAdaptiveDisplacement(InMeshView, MegameshTransform, InInstanceDesc);
	}
	else
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UTexturePatchModifier::ApplyModifications::Displacement);
		ApplyDisplacement(InMeshView, MegameshTransform, InInstanceDesc);
	}
}




TArray<FName> UTexturePatchHeightEntry::GetMegaMeshDefinitionChannels() const
{
	const MeshPartition::UModifierComponent* OwningModifier = Cast<const MeshPartition::UModifierComponent>(GetOuter());
	if (OwningModifier == nullptr)
	{
		return {};
	}

	return OwningModifier->GetMegaMeshDefinitionChannels();
}

void UTexturePatchWeightEntry::CopyFalloffSettings()
{
	const MeshPartition::UTexturePatchModifier* OwningModifier = Cast<const MeshPartition::UTexturePatchModifier>(GetOuter());
	if (OwningModifier == nullptr)
	{
		return;
	}

	if (!OwningModifier->HeightChannel)
	{
		return;
	}

	FalloffMode = OwningModifier->HeightChannel->FalloffMode;
	FalloffDistance = OwningModifier->HeightChannel->FalloffDistance;
	CornerRadius = OwningModifier->HeightChannel->CornerRadius;
	FalloffCurve = OwningModifier->HeightChannel->FalloffCurve;

	TriggerModifierUpdate();
}

TArray<FName> UTexturePatchWeightEntry::GetMegaMeshDefinitionChannels() const
{
	const MeshPartition::UModifierComponent* OwningModifier = Cast<const MeshPartition::UModifierComponent>(GetOuter());
	if (OwningModifier == nullptr)
	{
		return {};
	}

	return OwningModifier->GetMegaMeshDefinitionChannels();
}

UTexturePatchModifier::UTexturePatchModifier()
{
	HeightChannel = CreateDefaultSubobject<MeshPartition::UTexturePatchHeightEntry>(TEXT("HeightChannel"));
}

void UTexturePatchModifier::SetUnscaledCoverage(const FVector2D Coverage)
{
	if (UnscaledPatchCoverage != Coverage)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, UnscaledPatchCoverage));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetUnscaledCoverage: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		UnscaledPatchCoverage = Coverage;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

void UTexturePatchModifier::SetApplyComponentZScale(const bool bInApplyComponentZScale)
{
	if (bApplyComponentZScale != bInApplyComponentZScale)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, bApplyComponentZScale));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetApplyComponentZScale: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		bApplyComponentZScale = bInApplyComponentZScale;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

bool UTexturePatchModifier::GetApplyComponentZScale() const
{
	return bApplyComponentZScale;
}

void UTexturePatchModifier::SetHeightChannel(MeshPartition::UTexturePatchHeightEntry* InHeightEntry)
{
	if (HeightChannel != InHeightEntry)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, HeightChannel));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetHeightChannel: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		HeightChannel = InHeightEntry;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

UTexturePatchHeightEntry* UTexturePatchModifier::GetHeightChannel() const
{
	return HeightChannel;
}

void UTexturePatchModifier::SetAdaptiveTessellationMode(const MeshPartition::ETexturePatchTessellationMode InTessellationMode)
{
	if (TessellationMode != InTessellationMode)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationMode));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetAdaptiveTessellationMode: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		TessellationMode = InTessellationMode;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

ETexturePatchTessellationMode UTexturePatchModifier::GetAdaptiveTesselationMode() const
{
	return TessellationMode;
}

void UTexturePatchModifier::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	const FName ParentPreviousDefaultType = TEXT("Misc");
	if (Ar.IsLoading()
		&& GetLinkerCustomVersion(UE::MeshPartition::FCustomVersion::GUID) < UE::MeshPartition::FCustomVersion::DefaultPriorityLayerSetToNone
		&& GetType() == ParentPreviousDefaultType) // Super::Serialize would set default to this
	{
		const FName PreviousModifierDefaultType = TEXT("Patch");
		SetType(PreviousModifierDefaultType);
	}
}

void UTexturePatchModifier::InitializeModifier()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexturePatchModifier::InitializeModifier);

	Super::InitializeModifier();

	UpdateAllTextureData();

	if (HeightChannel)
	{
		HeightChannel->AttachCurveListeners();
	}
	for (MeshPartition::UTexturePatchWeightEntry* Entry : WeightChannels)
	{
		if (Entry)
		{
			Entry->AttachCurveListeners();
		}
	}
}

void UTexturePatchModifier::UninitializeModifier()
{
	if (HeightChannel)
	{
		HeightChannel->DetachCurveListeners();
	}
	for (MeshPartition::UTexturePatchWeightEntry* Entry : WeightChannels)
	{
		if (Entry)
		{
			Entry->DetachCurveListeners();
		}
	}

	Super::UninitializeModifier();
}

void UTexturePatchEntry::SetTextureAsset(UTexture2D* InTextureAsset)
{
	if (TextureAsset != InTextureAsset)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, TextureAsset));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetTextureAsset: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		TextureAsset = InTextureAsset;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

UTexture2D* UTexturePatchEntry::GetTextureAsset() const
{
	return TextureAsset;
}

void UTexturePatchEntry::SetTextureChannelIndex(const int32 InTextureChannelIndex)
{
	if (TextureChannelIndex != InTextureChannelIndex)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, TextureChannelIndex));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetTextureChannelIndex: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		TextureChannelIndex = InTextureChannelIndex;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

int32 UTexturePatchEntry::GetTextureChannelIndex() const
{
	return TextureChannelIndex;
}

void UTexturePatchEntry::SetAlphaBlendingMode(const ETexturePatchAlphaMode InTexturePatchAlphaMode)
{
	if (AlphaMode != InTexturePatchAlphaMode)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, AlphaMode));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetAlphaBlendingMode: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		AlphaMode = InTexturePatchAlphaMode;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

ETexturePatchAlphaMode UTexturePatchEntry::GetAlphaBlendingMode() const
{
	return AlphaMode;
}

void UTexturePatchEntry::SetTexturePatchBlendMode(const MeshPartition::ETexturePatchBlendMode InTexturePatchBlendMode)
{
	if (BlendMode != InTexturePatchBlendMode)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, BlendMode));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetTexturePatchBlendMode: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		BlendMode = InTexturePatchBlendMode;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

MeshPartition::ETexturePatchBlendMode UTexturePatchEntry::GetTexturePatchBlendMode() const
{
	return BlendMode;
}

void UTexturePatchEntry::SetUseValueCurve(const bool bInUseValueCurve)
{
	if (bUseValueCurve != bInUseValueCurve)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, bUseValueCurve));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetUseValueCurve: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		bUseValueCurve = bInUseValueCurve;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

bool UTexturePatchEntry::GetUseValueCurve() const
{
	return bUseValueCurve;
}

void UTexturePatchEntry::SetValueCurve(UCurveFloat* InValueCurve)
{
	if (ValueCurve != InValueCurve)
	{
		FProperty* Property = StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, ValueCurve));

		if (!Property)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "SetValueCurve: Failed to find property, operation aborted");
			return;
		}

		FEditPropertyChain EditChain;
		EditChain.AddHead(Property);
		EditChain.SetActivePropertyNode(Property);

		PreEditChange(EditChain);

		Modify();
		ValueCurve = InValueCurve;

		FPropertyChangedEvent ChangedEvent(Property, EPropertyChangeType::ValueSet);
		FPropertyChangedChainEvent ChangedChainEvent(EditChain, ChangedEvent);

		PostEditChangeChainProperty(ChangedChainEvent);
	}
}

UCurveFloat* UTexturePatchEntry::GetValueCurve() const
{
	return ValueCurve;
}

void UTexturePatchEntry::AttachCurveListeners()
{
	if (FalloffCurve && !FalloffCurveListenerHandle.IsValid())
	{
		FalloffCurveListenerHandle = FalloffCurve->OnUpdateCurve.AddUObject(this, &UTexturePatchEntry::OnCurveChanged);
	}
	if (ValueCurve && !ValueCurveListenerHandle.IsValid())
	{
		ValueCurveListenerHandle = ValueCurve->OnUpdateCurve.AddUObject(this, &UTexturePatchEntry::OnCurveChanged);
	}
}

void UTexturePatchEntry::DetachCurveListeners()
{
	if (FalloffCurve && FalloffCurveListenerHandle.IsValid())
	{
		FalloffCurve->OnUpdateCurve.Remove(FalloffCurveListenerHandle);
		FalloffCurveListenerHandle.Reset();
	}
	if (ValueCurve && ValueCurveListenerHandle.IsValid())
	{
		ValueCurve->OnUpdateCurve.Remove(ValueCurveListenerHandle);
		ValueCurveListenerHandle.Reset();
	}
}

void UTexturePatchEntry::OnCurveChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType)
{
	if (MeshPartition::UTexturePatchModifier* OwningModifier = Cast<MeshPartition::UTexturePatchModifier>(GetOuter()))
	{
		OwningModifier->OnChanged(OwningModifier->ComputeBounds(), EChangeType::StateChange);
	}
}

void UTexturePatchEntry::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	// A null property means we don't know which member is changing (undo, duplicate, bulk set, etc.).
	// Treat that as "anything might be touched" and run every branch defensively.
	FProperty* Property = PropertyAboutToChange.GetTail() ? PropertyAboutToChange.GetTail()->GetValue() : nullptr;

	// Disconnect our curve listener if we're about to change the curve
	if (!Property || Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, FalloffCurve))
	{
		if (FalloffCurve && FalloffCurveListenerHandle.IsValid())
		{
			FalloffCurve->OnUpdateCurve.Remove(FalloffCurveListenerHandle);
			FalloffCurveListenerHandle.Reset();
		}
	}
	if (!Property || Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, ValueCurve))
	{
		if (ValueCurve && ValueCurveListenerHandle.IsValid())
		{
			ValueCurve->OnUpdateCurve.Remove(ValueCurveListenerHandle);
			ValueCurveListenerHandle.Reset();
		}
	}
}

void UTexturePatchEntry::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	ON_SCOPE_EXIT{ Super::PostEditChangeProperty(PropertyChangedEvent); };

	// A null property means we don't know which member changed (undo, duplicate, bulk set, etc.).
	// Treat that as "anything might be touched" and run every branch.
	FProperty* Property = PropertyChangedEvent.PropertyChain.GetTail() ? PropertyChangedEvent.PropertyChain.GetTail()->GetValue() : nullptr;

	if (!Property
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, TextureAsset)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, TextureChannelIndex)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, AlphaMode)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, AlphaTextureAsset)
		|| Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, AlphaChannelIndex))
	{
		if (UTexturePatchModifier* OwningModifier = Cast<UTexturePatchModifier>(GetOuter()))
		{
			OwningModifier->UpdateChannelData(this);
		}
	}
	// Connect a curve listener if we are adding a curve
	if (!Property || Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, FalloffCurve))
	{
		if (FalloffCurve)
		{
			FalloffCurveListenerHandle = FalloffCurve->OnUpdateCurve.AddUObject(this, &UTexturePatchEntry::OnCurveChanged);
		}
	}
	if (!Property || Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchEntry, ValueCurve))
	{
		if (ValueCurve)
		{
			ValueCurveListenerHandle = ValueCurve->OnUpdateCurve.AddUObject(this, &UTexturePatchEntry::OnCurveChanged);
		}
	}

	UTexturePatchModifier* OwningModifier = Cast<UTexturePatchModifier>(GetOuter());
	if (OwningModifier != nullptr)
	{
		OwningModifier->OnInnerObjectModified(PropertyChangedEvent);
	}
}

TArray<FBox> UTexturePatchModifier::ComputeBounds() const
{
	FVector2d Coverage = GetUnscaledCoverage();

	const float Padding = 0.2f;

	Geometry::FAxisAlignedBox3d LocalBounds(
		FVector3d(-Coverage / 2. * (1. + Padding), -VerticalPatchExtentDown),
		FVector3d( Coverage / 2. * (1. + Padding),  VerticalPatchExtentUp));

	FTransform PatchToWorld = GetComponentTransform();
	if (!bApplyComponentZScale)
	{
		FVector3d Scale = PatchToWorld.GetScale3D();
		Scale.Z = 1.0;
		PatchToWorld.SetScale3D(Scale);
	}

	FBox GlobalBounds;
	for (int i = 0; i < 8; ++i)
	{
		GlobalBounds += PatchToWorld.TransformPosition(LocalBounds.GetCorner(i));
	}

	return { GlobalBounds };
}

void UTexturePatchModifier::PreEditChange(FEditPropertyChain& PropertyAboutToChange)
{
	Super::PreEditChange(PropertyAboutToChange);

	if (!PropertyAboutToChange.GetTail())
	{
		return;
	}

	FProperty* Property = PropertyAboutToChange.GetTail()->GetValue();
	if (!Property)
	{
		return;
	}

	// We're using the tail of the property chain, so this handles the case of the actual top level pointer for
	//  the height/weight entry being changed (vs internals, which we also get notified of)
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, HeightChannel) && HeightChannel)
	{
		HeightChannel->DetachCurveListeners();
		HeightChannelField.Reset();
	}
	else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, WeightChannels))
	{
		for (int32 Index = 0; Index < WeightChannels.Num(); ++Index)
		{
			if (WeightChannels[Index])
			{
				WeightChannels[Index]->DetachCurveListeners();
			}
			if (Index < WeightChannelFields.Num())
			{
				WeightChannelFields[Index].Reset();
			}
		}
	}
}

void UTexturePatchModifier::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	ON_SCOPE_EXIT{ Super::PostEditChangeProperty(PropertyChangedEvent); };

	if (!PropertyChangedEvent.PropertyChain.GetTail())
	{
		return;
	}
	FProperty* Property = PropertyChangedEvent.PropertyChain.GetTail()->GetValue();
	if (!Property)
	{
		return;
	}

	// We're using the tail of the property chain, so this handles the case of the actual top level pointer for
	//  the height/weight entry being changed (vs internals, which we also get notified of)
	if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, HeightChannel) && HeightChannel)
	{
		UpdateChannelData(HeightChannel);
		HeightChannel->AttachCurveListeners();
	}
	else if (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, WeightChannels))
	{
		bool bItemAdded = false;
		for (TObjectPtr<MeshPartition::UTexturePatchWeightEntry>& WeightChannelEntry : WeightChannels)
		{
			if (WeightChannelEntry)
			{
				UpdateChannelData(WeightChannelEntry);
				WeightChannelEntry->AttachCurveListeners();
			}
			else
			{
				// populate newly added items
				FName SubobjectName = MakeUniqueObjectName(this, MeshPartition::UTexturePatchWeightEntry::StaticClass(), "MegaMeshTexturePatchWeightEntry");
				WeightChannelEntry = NewObject<MeshPartition::UTexturePatchWeightEntry>(this, MeshPartition::UTexturePatchWeightEntry::StaticClass(), SubobjectName, RF_Transactional);
				bItemAdded = true;
			}
		}

#if WITH_EDITOR
		if (bItemAdded && GIsEditor)
		{
			// ensure that detail customization is called again	
			FPropertyEditorModule& PropertyEditorModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
			PropertyEditorModule.NotifyCustomizationModuleChanged();
		}
#endif
	}
	if (HeightChannel && ((Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationMode)) || 
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationErrorMode)) || 
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationError)) ||
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, MinimumEdgeLength)) ||
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, MaximumEdgeLength)) ||
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationFeatureSensitivity)) ||
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, bTessellationMeshRegularization)) ||
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, DetailAdjustmentChannelName))))
	{
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}

	if (HeightChannel)
	{
		for (auto* Node = PropertyChangedEvent.PropertyChain.GetHead(); Node != nullptr; Node = Node->GetNextNode())
		{
			if ( Node->GetValue()->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, UnscaledPatchCoverage))
			{
				UpdateDisplacementMap();
				break;
			}
		}
	}

	if (HeightChannel && ((Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationMode)) ||
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationErrorMode)) ||
						(Property->GetFName() == GET_MEMBER_NAME_CHECKED(UTexturePatchModifier, TessellationError))))
	{
		UpdateDisplacementMap();
	}
}

void UTexturePatchModifier::PostEditUndo()
{
	Super::PostEditUndo();

	// We could probably store extra data to determine whether our cached data needs updating,
	//  but unclear whether it's worth it vs just re-updating on any undo that touches this
	//  object.
	UpdateAllTextureData();
}

void UTexturePatchModifier::OnInnerObjectModified(const FPropertyChangedEvent& InnerPropertyChangedEvent)
{
	// When changing properties of sub-objects in details panel, PostEditChangeProperty of the modifier is not called.
	// To trigger an update in the task graph, propagate the notification.
	// 
	FPropertyChangedEvent Event(nullptr, InnerPropertyChangedEvent.ChangeType);
    PostEditChangeProperty(Event);
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UTexturePatchModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	if (!HasScalarField())
	{
		return nullptr;
	}

	TSharedPtr<FTexturePatchModifierOp> Op = AllocateBackgroundOp<FTexturePatchModifierOp>(GetFName());
	Op->GlobalBounds = { ComputeCombinedBounds() };

	FTransform Transform = GetComponentTransform();
	if (!bApplyComponentZScale)
	{
		FVector3d Scale = Transform.GetScale3D();
		Scale.Z = 1.0;
		Transform.SetScale3D(Scale);
	}
	Op->PatchTransforms = { Transform };

	Op->UnscaledPatchCoverage = UnscaledPatchCoverage;
	Op->VerticalPatchExtentUp = VerticalPatchExtentUp;
	Op->VerticalPatchExtentDown = VerticalPatchExtentDown;
	Op->TessellationMode = TessellationMode;
	Op->TessellationErrorMode = TessellationErrorMode;
	Op->TessellationError = TessellationError;
	Op->MinimumEdgeLength = MinimumEdgeLength;
	Op->MaximumEdgeLength = MaximumEdgeLength;
	Op->FeatureSensitivity = TessellationFeatureSensitivity;
	Op->bMeshRegularization = bTessellationMeshRegularization;
	Op->DetailAdjustmentChannelName = DetailAdjustmentChannelName;

	// Helper to avoid duplicating copied curve data across weight/height entries.
	// TODO: Could have a cache to share across modifiers, but might not be worth it
	TMap<UCurveFloat*, TSharedPtr<const FRichCurve>> CopiedCurves;
	auto CopyCurve = [&CopiedCurves](UCurveFloat* CurveIn) -> TSharedPtr<const FRichCurve>
	{
		if (!CurveIn)
		{
			return nullptr;
		}

		if (const TSharedPtr<const FRichCurve>* Existing = CopiedCurves.Find(CurveIn))
		{
			return *Existing;
		}

		TSharedPtr<const FRichCurve> CopiedCurve = MakeShared<FRichCurve>(CurveIn->FloatCurve);
		CopiedCurves.Add(CurveIn, CopiedCurve);
		
		return CopiedCurve;
	};

	auto UpdateOpTextureEntry = [this, &CopyCurve](const UTexturePatchEntry& Source, FTexturePatchModifierOp::FTextureEntry& Dest)
	{
		Dest.BlendMode = Source.BlendMode;
		Dest.SoftnessParameter = Source.SoftnessParameter;
		Dest.AlphaMode = Source.AlphaMode;
		Dest.SelfMaskTolerance = Source.SelfMaskTolerance;

		if (Source.bUseValueCurve)
		{
			Dest.ValueCurve = CopyCurve(Source.ValueCurve);
			Dest.ValueCurveOffset = Source.ValueCurveOffset;
			Dest.ValueCurveScale = Source.ValueCurveScale;
		}

		Dest.FalloffMode = Source.FalloffMode;
		Dest.FalloffDistance = Source.FalloffDistance;
		Dest.CornerRadius = Source.CornerRadius;
		if (Source.FalloffMode == MeshPartition::ETexturePatchFalloffMode::CustomCurve)
		{
			Dest.FalloffCurve = CopyCurve(Source.FalloffCurve);
		}
	};

	if (HeightChannel && HeightChannelField)
	{
		Op->HeightChannel.Emplace(HeightChannelField.ToSharedRef(), FilteredDisplacementMap.ToSharedRef());
		UpdateOpTextureEntry(*HeightChannel, *Op->HeightChannel);
		Op->HeightChannel->ZeroInEncoding = HeightChannel->ZeroInEncoding;
		Op->HeightChannel->EncodingScale = HeightChannel->EncodingScale;
		Op->HeightChannel->HeightScaleWeightChannel = HeightChannel->HeightScaleWeightChannel.GetName();
	}
	for (int32 Index = 0, Num = FMath::Min(WeightChannels.Num(), WeightChannelFields.Num()); Index < Num; ++Index)
	{
		const TObjectPtr<MeshPartition::UTexturePatchWeightEntry>& WeightChannelEntry = WeightChannels[Index];
		const TSharedPtr<const FChannelFieldData>& WeightChannelField = WeightChannelFields[Index];

		if (!WeightChannelEntry || !WeightChannelField)
		{
			continue;
		}
		FTexturePatchModifierOp::FWeightEntry& Entry = Op->WeightChannels.Emplace_GetRef(WeightChannelFields[Index].ToSharedRef());
		UpdateOpTextureEntry(*WeightChannelEntry, Entry);
		Entry.WeightChannelName = WeightChannelEntry->WeightChannelName;
		Entry.bApplyHeightMinMaxBlend = WeightChannelEntry->bApplyHeightMinMaxBlend;
		Entry.HeightMinMaxBlendDistance = WeightChannelEntry->HeightMinMaxBlendDistance;
		Entry.ZeroInEncoding = WeightChannelEntry->ZeroInEncoding;
		Entry.EncodingScale = WeightChannelEntry->EncodingScale;
	}

	return Op;
}

void UTexturePatchModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UTexturePatchModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	if (!HasScalarField())
	{
		// no dependencies, because it does nothing currently
		return;
	}

	// used to compute LocalBounds
	Dependencies += UnscaledPatchCoverage;
	Dependencies += VerticalPatchExtentUp;
	Dependencies += VerticalPatchExtentDown;
	Dependencies += bApplyComponentZScale;

	auto GatherDependenciesForGetAdjustedTextureValue = [&Dependencies](UTexturePatchEntry& Entry, const FChannelFieldData& FieldData)
		{
			// depend on texture source data, and declare the texture as a package dependency
			Dependencies.AddPackageDependency(Entry.TextureAsset);
			Dependencies += FieldData.GetScalarFieldHash();		// more technically correct... we use the scalar field in the apply(), not the texture itself
			Dependencies += Entry.TextureChannelIndex;

			// double-check the scalar field is in sync with the texture asset
			ensure(FieldData.GetScalarFieldHash() == (Entry.TextureAsset ? Entry.TextureAsset->Source.GetId() : FGuid()));

			// use the value curve if it is present
			if (Entry.bUseValueCurve && Entry.ValueCurve)
			{
				Dependencies += Entry.ValueCurve;
				Dependencies += Entry.ValueCurveOffset;
				Dependencies += Entry.ValueCurveScale;
			}
		};

	auto GatherDependenciesForGetAlphaValue = [&Dependencies](UTexturePatchEntry& Entry, const FChannelFieldData& FieldData)
		{
			Dependencies += Entry.AlphaMode;
			switch (Entry.AlphaMode)
			{
			case MeshPartition::ETexturePatchAlphaMode::AlwaysOne:
				break;
			case MeshPartition::ETexturePatchAlphaMode::SelfMask:
				Dependencies += Entry.SelfMaskTolerance;
				break;
			case MeshPartition::ETexturePatchAlphaMode::ThisAlphaChannel:
				// double-check the alpha field is in sync with the texture asset
				ensure(FieldData.GetAlphaFieldHash() == (Entry.TextureAsset ? Entry.TextureAsset->Source.GetId() : FGuid()));
				if (FieldData.GetAlphaField())
				{
					Dependencies += FieldData.GetAlphaFieldHash();
					Dependencies += Entry.AlphaChannelIndex;
				}
				break;
			case MeshPartition::ETexturePatchAlphaMode::OtherAlphaChannel:
				Dependencies.AddPackageDependency(Entry.AlphaTextureAsset);
				// double-check the alpha field is in sync with the texture asset
				ensure(FieldData.GetAlphaFieldHash() == (Entry.AlphaTextureAsset ? Entry.AlphaTextureAsset->Source.GetId() : FGuid()));
				if (FieldData.GetAlphaField())
				{
					Dependencies += FieldData.GetAlphaFieldHash();
					Dependencies += Entry.AlphaChannelIndex;
				}
				break;
			}
		};

	auto GatherDependenciesForGetFalloffAlpha = [&Dependencies](MeshPartition::UTexturePatchEntry& Entry)
		{
			MeshPartition::ETexturePatchFalloffMode FalloffModeToUse =
				Entry.FalloffMode == MeshPartition::ETexturePatchFalloffMode::CustomCurve && !Entry.FalloffCurve ?
				MeshPartition::ETexturePatchFalloffMode::Smooth : Entry.FalloffMode;
			Dependencies += FalloffModeToUse;
			switch (FalloffModeToUse)
			{
			case MeshPartition::ETexturePatchFalloffMode::Linear:
				break;
			case MeshPartition::ETexturePatchFalloffMode::Smooth:
				break;
			case MeshPartition::ETexturePatchFalloffMode::CustomCurve:
				Dependencies += Entry.FalloffCurve;
				break;

			}
			Dependencies += Entry.CornerRadius;
			Dependencies += Entry.FalloffDistance;
		};

	auto GatherDependenciesForBlendValues = [&Dependencies](MeshPartition::UTexturePatchEntry& Entry)
		{
			Dependencies += Entry.BlendMode;
			switch (Entry.BlendMode)
			{
			case MeshPartition::ETexturePatchBlendMode::Additive:
				break;
			case MeshPartition::ETexturePatchBlendMode::AlphaBlend:
				break;
			case MeshPartition::ETexturePatchBlendMode::Min:
			case MeshPartition::ETexturePatchBlendMode::Max:
				Dependencies += Entry.SoftnessParameter;
				break;
			}
		};

	// dependencies for height apply
	if (HeightChannel && HeightChannelField)
	{
		MeshPartition::UTexturePatchHeightEntry& Height = *HeightChannel;

		GatherDependenciesForGetAdjustedTextureValue(Height, *HeightChannelField);
		Dependencies += Height.ZeroInEncoding;
		Dependencies += Height.EncodingScale;
		Dependencies += Height.HeightScaleWeightChannel;

		GatherDependenciesForGetAlphaValue(Height, *HeightChannelField);
		GatherDependenciesForGetFalloffAlpha(Height);
		GatherDependenciesForBlendValues(Height);
	}

	// dependencies for weight apply
	for (int32 Index = 0, Num = FMath::Min(WeightChannels.Num(), WeightChannelFields.Num()); Index < Num; ++Index)
	{
		const TObjectPtr<MeshPartition::UTexturePatchWeightEntry>& WeightChannel = WeightChannels[Index];
		const TSharedPtr<const FChannelFieldData>& WeightChannelField = WeightChannelFields[Index];
		if (WeightChannel && !WeightChannel->WeightChannelName.IsNone() && WeightChannelField)
		{
			UTexturePatchWeightEntry& Weight = *WeightChannel;

			Dependencies += Weight.WeightChannelName;
			Dependencies += Weight.ZeroInEncoding;
			Dependencies += Weight.EncodingScale;

			GatherDependenciesForGetAdjustedTextureValue(Weight, *WeightChannelField);
			GatherDependenciesForGetAlphaValue(Weight, *WeightChannelField);
			GatherDependenciesForGetFalloffAlpha(Weight);
			if (Weight.bApplyHeightMinMaxBlend)
			{
				Dependencies += Weight.HeightMinMaxBlendDistance;
			}
			GatherDependenciesForBlendValues(Weight);
		}
	}

	// dependencies for adaptive tessellation 
	Dependencies += TessellationMode;
	Dependencies += TessellationErrorMode;
	Dependencies += TessellationError;
	Dependencies += MinimumEdgeLength;
	Dependencies += MaximumEdgeLength;
	Dependencies += TessellationFeatureSensitivity;
	Dependencies += bTessellationMeshRegularization;
	Dependencies += DetailAdjustmentChannelName;
}

FGuid UTexturePatchModifier::GetCodeVersionKey() const
{
	return FTexturePatchModifierOp::GetCodeVersionKey();
}

Tasks::FTask UTexturePatchModifier::GetAsyncPrepareResourcesTask() const
{
	TArray<UE::Tasks::FTask, TInlineAllocator<8>> Prerequisites;
	if (HeightChannelField && FilteredDisplacementMap)
	{
		Prerequisites.Emplace(HeightChannelField->GetAsyncInitTask());
		Prerequisites.Emplace(FilteredDisplacementMap->GetAsyncInitTask());
	}
	for (const TSharedPtr<const FChannelFieldData>& WeightChannelField : WeightChannelFields)
	{
		if (WeightChannelField)
		{
			Prerequisites.Emplace(WeightChannelField->GetAsyncInitTask());
		}
	}
	return UE::Tasks::Launch(TEXT("AsyncJoin"), [](){}, Prerequisites);
}

void UTexturePatchModifier::DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const
{
	const FColor RectangleColor = FColor::Red;
	const FColor LocalBoundsColor = FColor::Yellow;
	constexpr float RectangleThickness = 3;
	constexpr float BoundsThickness = 1; 

	constexpr float DepthBias = 1;
	constexpr bool bScreenSpace = true;

	const FTransform PatchToWorld = GetComponentTransform();

	if (bDrawPatchRectangle)
	{
		DrawRectangle(PDI, PatchToWorld.GetTranslation(), PatchToWorld.GetUnitAxis(EAxis::X), PatchToWorld.GetUnitAxis(EAxis::Y),
			RectangleColor, GetUnscaledCoverage().X * PatchToWorld.GetScale3D().X, GetUnscaledCoverage().Y * PatchToWorld.GetScale3D().Y, SDPG_Foreground,
			RectangleThickness, DepthBias, bScreenSpace);
	}

	if (bDrawAffectedBox)
	{
		FVector2d Coverage = GetUnscaledCoverage();
		FBox LocalBounds(
			FVector3d(-Coverage / 2, -VerticalPatchExtentDown),
			FVector3d(Coverage / 2, VerticalPatchExtentUp));
		DrawWireBox(PDI, PatchToWorld.ToMatrixWithScale(), LocalBounds,
			LocalBoundsColor, SDPG_World, BoundsThickness, DepthBias, bScreenSpace);
	}

	// Note: when patches are deleted, selection seems to be updated only after giving another call to the
	// visualization drawing, so do not ensure if patch was null.
}


// Triggered by user button click
void UTexturePatchModifier::UpdateFromTexture()
{
	UpdateAllTextureData();
	// Trigger megamesh update
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void UTexturePatchModifier::MatchWeightFalloffsToHeight()
{
	if (!HeightChannel)
	{
		return;
	}
	for (MeshPartition::UTexturePatchEntry* WeightChannelEntry : WeightChannels)
	{
		if (WeightChannelEntry)
		{
			WeightChannelEntry->FalloffMode = HeightChannel->FalloffMode;
			WeightChannelEntry->FalloffDistance = HeightChannel->FalloffDistance;
			WeightChannelEntry->CornerRadius = HeightChannel->CornerRadius;
			WeightChannelEntry->FalloffCurve = HeightChannel->FalloffCurve;
		}
	}
	// Trigger megamesh update
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void UTexturePatchModifier::MatchWeightAlphaBlendToHeight()
{
	if (!HeightChannel)
	{
		return;
	}
	for (MeshPartition::UTexturePatchEntry* WeightChannelEntry : WeightChannels)
	{
		if (WeightChannelEntry)
		{
			WeightChannelEntry->AlphaMode = HeightChannel->AlphaMode;
			WeightChannelEntry->AlphaTextureAsset = HeightChannel->AlphaTextureAsset;
			WeightChannelEntry->AlphaChannelIndex = HeightChannel->AlphaChannelIndex;
			WeightChannelEntry->SelfMaskTolerance = HeightChannel->SelfMaskTolerance;
			WeightChannelEntry->BlendMode = HeightChannel->BlendMode;
			WeightChannelEntry->SoftnessParameter = HeightChannel->SoftnessParameter;
		}
	}
	// Trigger megamesh update
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

void UTexturePatchModifier::MatchWeightCurveToHeight()
{
	if (!HeightChannel)
	{
		return;
	}
	for (MeshPartition::UTexturePatchEntry* WeightChannelEntry : WeightChannels)
	{
		if (WeightChannelEntry)
		{
			WeightChannelEntry->bUseValueCurve = HeightChannel->bUseValueCurve;
			WeightChannelEntry->ValueCurve = HeightChannel->ValueCurve;
			WeightChannelEntry->ValueCurveOffset = HeightChannel->ValueCurveOffset;
			WeightChannelEntry->ValueCurveScale = HeightChannel->ValueCurveScale;
		}
	}
	// Trigger megamesh update
	OnChanged(ComputeBounds(), EChangeType::StateChange);
}

bool UTexturePatchModifier::HasScalarField() const
{
	if (HeightChannel && HeightChannelField)
	{
		return true;
	}

	for (int32 Index = 0, Num = FMath::Min(WeightChannels.Num(), WeightChannelFields.Num()); Index < Num; ++Index)
	{
		if (WeightChannels[Index] && WeightChannelFields[Index])
		{
			return true;
		}
	}

	return false;
}

void UTexturePatchModifier::UpdateAllTextureData()
{
	UpdateHeightChannel();

	WeightChannelFields.SetNum(WeightChannels.Num());
	for (int32 Index = 0, Num = WeightChannels.Num(); Index < Num; ++Index)
	{
		UpdateWeightChannel(Index);
	}
}

void UTexturePatchModifier::UpdateHeightChannel()
{
	HeightChannelField.Reset();

	if (HeightChannel)
	{
		FReadTextureAdapter ReadTextureAdapter(*HeightChannel);
		if (ReadTextureAdapter.HasValidData())
		{
			HeightChannelField = MakeShared<FChannelFieldData>(MoveTemp(ReadTextureAdapter));

			UpdateDisplacementMap();
		}
	}
}

void UTexturePatchModifier::UpdateWeightChannel(int32 Index)
{
	if (ensure(Index < WeightChannels.Num()))
	{
		const TObjectPtr<UTexturePatchWeightEntry>& WeightChannel = WeightChannels[Index];

		if (Index >= WeightChannelFields.Num())
		{
			WeightChannelFields.SetNum(Index + 1);
		}

		TSharedPtr<const FChannelFieldData>& WeightChannelField = WeightChannelFields[Index];

		WeightChannelField.Reset();

		if (WeightChannel)
		{
			FReadTextureAdapter ReadTextureAdapter(*WeightChannel);
			if (ReadTextureAdapter.HasValidData())
			{
				WeightChannelField = MakeShared<FChannelFieldData>(MoveTemp(ReadTextureAdapter));
			}
		}
	}
}

void UTexturePatchModifier::UpdateChannelData(const UTexturePatchEntry* Channel)
{
	if (Channel)
	{
		if (Channel == HeightChannel)
		{
			UpdateHeightChannel();
		}
		else
		{
			for (int32 Index = 0, Num = WeightChannels.Num(); Index < Num; ++Index)
			{
				if (WeightChannels[Index] == Channel)
				{
					UpdateWeightChannel(Index);
					break;
				}
			}
		}
	}
}

void UTexturePatchModifier::UpdateDisplacementMap()
{
	if (HeightChannelField.IsValid())
	{
		const float FilterWidth = TessellationMode == ETexturePatchTessellationMode::NonAdaptive
							? 0.f
							: CalculateSampleRate(TessellationErrorMode, TessellationError, UnscaledPatchCoverage);

		FilteredDisplacementMap = HeightChannelField->MakeDisplacementMap(FilterWidth, UnscaledPatchCoverage);
	}
}

} // namespace UE::MeshPartition