// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/MeshPartitionWeightUtilityModifier.h"
#include "DynamicMesh/MeshNormals.h"
#include "VectorUtil.h"


namespace UE::MeshPartition
{
namespace MegaMeshWeightUtilityModifierLocals
{
	class FBackgroundOp : public MeshPartition::IModifierBackgroundOp
	{
	public:
		FBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

		FBox GlobalBounds;
		FVector WorldLocation = FVector::ZeroVector;
		FTransform ComponentTransform;
	
		FName WeightChannelName;
		float Radius;
		float Falloff;
		float InnerValue;
		float OuterValue;
		bool  bCosineWeighted;
	
		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;
	
		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("6f7c7bac-9256-4b16-9a3a-ffa2261c426f"));
			return VersionKey;
		}

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override { return false; }
	};
} // end namespace MegaMeshWeightUtilityModifierLocals

UWeightUtilityModifier::UWeightUtilityModifier()
{
	static const FName Name= TEXT("WeightUtility");
	MeshPartition::UModifierComponent::SetType(Name);
}

TArray<FBox> UWeightUtilityModifier::ComputeBounds() const
{
	const FVector PatchLocation = GetComponentLocation();
	return { FBox(PatchLocation - FVector(Radius + Falloff, Radius + Falloff, MaxZDistance / 2.f), PatchLocation + FVector(Radius + Falloff, Radius + Falloff, MaxZDistance / 2.f)) };
}

void MegaMeshWeightUtilityModifierLocals::FBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);

	
	if (!WeightChannelName.IsNone() && !OutInstanceInfos.IsEmpty())
	{
		OutInstanceInfos[0].ReadViewComponents  |= EMeshViewComponents::DynamicSubmesh;
		OutInstanceInfos[0].ReadViewComponents  |= EMeshViewComponents::VertexAttributeWeight;
		OutInstanceInfos[0].WriteViewComponents |= EMeshViewComponents::VertexAttributeWeight;
		OutInstanceInfos[0].UsedChannels.Emplace(WeightChannelName);
	}
}

TSharedPtr<const MeshPartition::IModifierBackgroundOp> UWeightUtilityModifier::CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const
{
	using namespace MegaMeshWeightUtilityModifierLocals;

	TSharedPtr<FBackgroundOp> Op = MakeShared<FBackgroundOp>(GetFName());
	Op->GlobalBounds = ComputeCombinedBounds();
	Op->WorldLocation = GetComponentLocation();
	Op->ComponentTransform = GetComponentTransform();
	Op->WeightChannelName = WeightChannelName;
	Op->Radius = Radius;
	Op->Falloff = Falloff;
	Op->InnerValue = InnerValue;
	Op->OuterValue = OuterValue;
	Op->bCosineWeighted = bCosineWeighted;

	return Op;
}

FGuid UWeightUtilityModifier::GetCodeVersionKey() const
{
	return MegaMeshWeightUtilityModifierLocals::FBackgroundOp::GetCodeVersionKey();
}

void MegaMeshWeightUtilityModifierLocals::FBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UWeightUtilityModifier::ApplyModifications);

	const FVector MeshSpaceLocation = InTransform.InverseTransformPosition(WorldLocation);

	if (WeightChannelName.IsNone())
	{
		return;
	}

	const FDynamicMesh3& Mesh = InMeshView.GetSubmesh();
	Geometry::FMeshNormals MeshNormals(&Mesh);
	if (bCosineWeighted)
	{
		MeshNormals.ComputeVertexNormals();
	}

	for (int VertexIndex = 0; VertexIndex < InMeshView.VertexCount(); ++ VertexIndex)
	{
		FVector3d MeshVertex = InMeshView.GetVertexPos(VertexIndex);
		const double Distance = FVector2D::Distance(FVector2D(MeshVertex), FVector2D(MeshSpaceLocation));

		const float FalloffWeight = FMath::SmoothStep<float>(Radius, Radius + Falloff, Distance);
		
		float Weight = (1.f - FalloffWeight) * InnerValue + FalloffWeight * OuterValue;
		
		if (bCosineWeighted)
		{
			const FVector3d MeshNormal = MeshNormals[VertexIndex];
			const FVector3d MeshNormalWorld = Geometry::VectorUtil::TransformNormal(InTransform, MeshNormal);
			const FVector3d MeshNormalPatch = Geometry::VectorUtil::InverseTransformNormal(ComponentTransform, MeshNormalWorld);

			const double Cosine = Geometry::Normalized(MeshNormalPatch)[2];

			Weight *= FMath::Max(0., Cosine);
		}

		InMeshView.SetVertexAttributeWeight(WeightChannelName, VertexIndex, Weight);
	}
}

void UWeightUtilityModifier::GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWeightUtilityModifier::GatherDependencies);

	Super::GatherDependencies(Dependencies);

	Dependencies += Radius;
	Dependencies += Falloff;
	Dependencies += MaxZDistance;
	Dependencies += WeightChannelName;
}
} // namespace UE::MeshPartition