// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionRemeshOp.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshConstraintsUtil.h"
#include "QueueRemesher.h"
#include "SubRegionRemesher.h"
#include "ProjectionTargets.h"

namespace UE::MeshPartition
{
	FMegaMeshRemeshBackgroundOpBase::FMegaMeshRemeshBackgroundOpBase(const FName& InOperationName)
		: MeshPartition::IModifierBackgroundOp(InOperationName)
	{
	}

	void FMegaMeshRemeshBackgroundOpBase::RemeshROI(FDynamicMesh3& Mesh, const FTransform3d& MegaMeshTransform, const TSet<int32>& TriangleROI) const
	{
		using namespace Geometry;

		if (bComputeNormalSeams)
		{
			Geometry::FMeshNormals FaceNormals(&Mesh);
			FaceNormals.ComputeTriangleNormals();
			const TArray<FVector3d>& Normals = FaceNormals.GetNormals();
			Mesh.Attributes()->PrimaryNormals()->CreateFromPredicate([&Normals, this](int VID, int TA, int TB)
				{
					return Normals[TA].Dot(Normals[TB]) > NormalSeamDotProductThreshold;
				}, 0);
		}

		// Set up our constraints. In particular, we need to constrain the edges of our boundary, which
		//  should not be flipped even if boundary mode is set to "free".
		Geometry::FMeshConstraints Constraints;
		bool bAllowSeamSmoothing = false;
		EEdgeRefineFlags BoundaryConstraintType = Geometry::EEdgeRefineFlags::NoFlip;
		FVertexConstraint BoundaryVertexConstraint = FVertexConstraint::Unconstrained();
		switch (BoundaryMode)
		{
		case EMegaMeshRemeshModifierBoundaryMode::Free:
			break;
		case EMegaMeshRemeshModifierBoundaryMode::SplitOnly:
			BoundaryConstraintType = EEdgeRefineFlags::SplitsOnly;
			BoundaryVertexConstraint = FVertexConstraint::FullyConstrained();
			break;
		case EMegaMeshRemeshModifierBoundaryMode::FullyConstrained:
			BoundaryConstraintType = EEdgeRefineFlags::FullyConstrained;
			BoundaryVertexConstraint = FVertexConstraint::FullyConstrained();
			break;
		}

		// TODO: Could see if doing any of this in parallel is much faster.
		TSet<int32> ProcessedEids;
		int32 NumSeamEdges = 0;
		int32 NumBoundaryEdges = 0;
		for (int32 Tid : TriangleROI)
		{
			FIndex3i TriEids = Mesh.GetTriEdges(Tid);
			for (int i = 0; i < 3; ++i)
			{
				int32 Eid = TriEids[i];
				bool bAlreadyProcessed = false;
				ProcessedEids.Add(Eid, &bAlreadyProcessed);
				if (bAlreadyProcessed)
				{
					continue;
				}

				if (Mesh.IsBoundaryEdge(Eid))
				{
					++NumBoundaryEdges;
				}

				if (Mesh.HasAttributes() && Mesh.Attributes()->IsSeamEdge(Eid))
				{
					++NumSeamEdges;
				}

				// Get regular edge constraints based on whether this is a seam of some kind
				FVertexConstraint VtxConstraintA = FVertexConstraint::Unconstrained();
				FVertexConstraint VtxConstraintB = FVertexConstraint::Unconstrained();
				FEdgeConstraint EdgeConstraint(EEdgeRefineFlags::NoConstraint);
				bool bHaveConstraintUpdate = Geometry::FMeshConstraintsUtil::ConstrainEdgeBoundariesAndSeams(
					Eid,
					Mesh,
					BoundaryConstraintType,
					EEdgeRefineFlags::NoFlip, // Group boundary constraint
					EEdgeRefineFlags::NoFlip, // Material boundary constraint
					EEdgeRefineFlags::SplitsOnly, // Seam constraint
					bAllowSeamSmoothing,
					EdgeConstraint, VtxConstraintA, VtxConstraintB);

				// If we're not a mesh boundary but are on the boundary of our region, we still need to apply our
				//  boundary constraint.
				FDynamicMesh3::FEdge Edge = Mesh.GetEdge(Eid);
				int32 OtherTid = Edge.Tri.A == Tid ? Edge.Tri.B : Edge.Tri.A;
				if (OtherTid != IndexConstants::InvalidID && !TriangleROI.Contains(OtherTid))
				{
					bHaveConstraintUpdate = true;

					VtxConstraintA.CombineConstraint(BoundaryVertexConstraint);
					VtxConstraintB.CombineConstraint(BoundaryVertexConstraint);

					// Our boundary constraint is always at least as high as the seam constraints, so we don't need
					//  to do any constraint combination here.
					EdgeConstraint = FEdgeConstraint(BoundaryConstraintType);
				}

				if (bHaveConstraintUpdate)
				{
					Constraints.SetOrUpdateEdgeConstraint(Eid, EdgeConstraint);

					VtxConstraintA.CombineConstraint(Constraints.GetVertexConstraint(Edge.Vert.A));
					Constraints.SetOrUpdateVertexConstraint(Edge.Vert.A, VtxConstraintA);

					VtxConstraintB.CombineConstraint(Constraints.GetVertexConstraint(Edge.Vert.B));
					Constraints.SetOrUpdateVertexConstraint(Edge.Vert.B, VtxConstraintB);
				}
			}
		}

		// TODO: Add additional options for a sub region remesher besides just the full pass one. I.e. we
		//  would need to write a FSubRegionQueueRemesher and/or FSubRegionNormalFlowRemesher and add the options
		//  to use them.

		TUniquePtr<Geometry::FRestrictedSubRegionRemesher> RemesherPtr = MakeUnique< Geometry::FRestrictedSubRegionRemesher>(&Mesh, TriangleROI);

		Geometry::FRemesher& Remesher = *RemesherPtr;

		Remesher.bEnableSplits = true;
		Remesher.bEnableFlips = true;
		Remesher.bEnableCollapses = true;
		Remesher.SetTargetEdgeLength(TargetEdgeLength);

		FDynamicMesh3 OriginalMesh;
		FDynamicMeshAABBTree3 OriginalMeshSpatial;
		Geometry::FMeshProjectionTarget ProjectionTarget;

		if (bProjectToInputMesh)
		{
			Remesher.ProjectionMode = Geometry::FRemesher::ETargetProjectionMode::AfterRefinement;

			OriginalMesh.Copy(Mesh, true, true, true, true);
			OriginalMeshSpatial = FDynamicMeshAABBTree3(&OriginalMesh, true);
			ProjectionTarget = Geometry::FMeshProjectionTarget(&OriginalMesh, &OriginalMeshSpatial);

			Remesher.SetProjectionTarget(&ProjectionTarget);
		}
		else
		{
			Remesher.ProjectionMode = Geometry::FRemesher::ETargetProjectionMode::NoProjection;
		}


		Remesher.bEnableSmoothing = (SmoothingStrength > 0);
		Remesher.SmoothSpeedT = SmoothingStrength;
		// Set the correct smoothing type (modeled on RemeshMeshOp.cpp)
		Remesher.SmoothType = FRemesher::ESmoothTypes::Uniform;
		switch (SmoothingType)
		{
		case ERemeshSmoothingType::Uniform:
			Remesher.SmoothType = FRemesher::ESmoothTypes::Uniform;
			Remesher.FlipMetric = FRemesher::EFlipMetric::OptimalValence;
			break;
		case ERemeshSmoothingType::Cotangent:
			Remesher.SmoothType = FRemesher::ESmoothTypes::Cotan;
			Remesher.FlipMetric = FRemesher::EFlipMetric::MinEdgeLength;
			break;
		case ERemeshSmoothingType::MeanValue:
			Remesher.SmoothType = FRemesher::ESmoothTypes::MeanValue;
			Remesher.FlipMetric = FRemesher::EFlipMetric::MinEdgeLength;
			break;
		default:
			ensure(false);
		}

		Remesher.bPreventTinyTriangles = true;
		Remesher.bPreventNormalFlips = true;
		Remesher.DEBUG_CHECK_LEVEL = 0;

		Remesher.SetExternalConstraints(MoveTemp(Constraints));


		if (bUseDensityWeightChannel)
		{
			if (const FDynamicMeshAttributeSet* const Attribs = Mesh.Attributes())
			{
				for (int32 Idx = 0; Idx < Attribs->NumWeightLayers(); ++Idx)
				{
					const FDynamicMeshWeightAttribute* const DensityWeightLayer = Attribs->GetWeightLayer(Idx);

					if (DensityWeightLayer->GetName() == DensityWeightChannelName)
					{
						// Capturing DensityWeightLayer and this is okay since they are by value and we are doing the Remesh immediately
						Remesher.CustomEdgeLengthScaleF = [DensityWeightLayer, this](const FDynamicMesh3& Mesh, int VertexA, int VertexB) -> double
						{						
							float WeightValueA;
							DensityWeightLayer->GetValue(VertexA, &WeightValueA);
							WeightValueA = FMath::Clamp(WeightValueA, 0.0f, 1.0f);

							float WeightValueB;
							DensityWeightLayer->GetValue(VertexB, &WeightValueB);
							WeightValueB = FMath::Clamp(WeightValueB, 0.0f, 1.0f);

							const double AvgWeight = 0.5 * (WeightValueA + WeightValueB);

							return FMath::Pow(0.5, -RelativeDensity * AvgWeight);
						};

						break;
					}
				}
			}
		}


		for (int Index = 0; Index < RemeshIterations; ++Index)
		{
			RemesherPtr->UpdateROI();
			RemesherPtr->BasicRemeshPass();
		}
	}


	FMegaMeshRemeshBackgroundOp::FMegaMeshRemeshBackgroundOp(const FName& InOperationName)
		: FMegaMeshRemeshBackgroundOpBase(InOperationName)
	{
	}

	void FMegaMeshRemeshBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
	{
		FInstanceInfo Desc;
		Desc.InstanceID = 0;
		Desc.Bounds = GlobalBounds;
		Desc.ReadViewComponents = EMeshViewComponents::DynamicSubmesh;
		Desc.WriteViewComponents = EMeshViewComponents::DynamicSubmesh;

		if (Desc.Bounds.Intersect(InBounds))
		{
			OutInstanceInfos.Add(Desc);
		}
	}

	void FMegaMeshRemeshBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& MegaMeshTransform, const FInstanceInfo& InInstanceDesc) const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::URemeshModifier::ApplyModifications);

		using namespace Geometry;

		// Note: Currently, we only remesh the edges that are entirely in our region of influence. A drawback of this is
		//  that in the case of a very simple megamesh with huge triangles, we will fail to remesh those triangles unless we
		//  contain them entirely. The obvious solution to this is to have an option of embedding new edges along the boundary
		//  (using a mesh boolean with an appropriate box), but those big triangles would need to be included in our bounds to
		//  begin with, so we would need a large margin.
		// Fortunately this use case is unlikely to be a particularly important one for us. The current approach will likely 
		//  be good enough for meshes with even a fairly coarse triangulation.

		FDynamicMesh3& Submesh = InMeshView.GetSubmeshMutable();

		TSet<int32> TidROI;

		// Caching of whether verts are in our coverage region.
		TMap<int32, bool> VertInCoverage;
		auto IsVertInCoverage = [this, &VertInCoverage, &Submesh, &MegaMeshTransform](int32 Vid)
		{
			bool* Found = VertInCoverage.Find(Vid);
			if (Found)
			{
				return *Found;
			}
			FVector3d LocalPosition = ModifierToWorld.InverseTransformPosition(
				MegaMeshTransform.TransformPosition(Submesh.GetVertex(Vid)));
			bool bInCoverage = LocalCoverage.Contains(LocalPosition);
			VertInCoverage.Add(Vid, bInCoverage);
			return bInCoverage;
		};
		const FDynamicMeshWeightAttribute* DensityWeightLayer = nullptr;
		if (bUseDensityWeightChannel)
		{
			if (const FDynamicMeshAttributeSet* const Attribs = Submesh.Attributes())
			{
				for (int32 Idx = 0; Idx < Attribs->NumWeightLayers(); ++Idx)
				{
					if (Attribs->GetWeightLayer(Idx)->GetName() == DensityWeightChannelName)
					{
						DensityWeightLayer = Attribs->GetWeightLayer(Idx);
					}
				}
			}
		}
		// helper lambda to be called only if the weight layer is in use
		auto VertAtBelowThreshold = [this, DensityWeightLayer](int32 VID)
		{
			float Value;
			DensityWeightLayer->GetValue(VID, &Value);
			return (Value <= MinWeightThreshold);
		};

		for (int32 Tid : Submesh.TriangleIndicesItr())
		{
			FIndex3i TriVids = Submesh.GetTriangle(Tid);
			if (!IsVertInCoverage(TriVids.A) || !IsVertInCoverage(TriVids.B) || !IsVertInCoverage(TriVids.C))
			{
				// One of the verts is not inside coverage
				continue;
			}
			// if we have a weight threshold and all weights are at/below threshold, skip the tri
			if (bUseWeightThreshold && DensityWeightLayer &&
				VertAtBelowThreshold(TriVids.A) &&
				VertAtBelowThreshold(TriVids.B) &&
				VertAtBelowThreshold(TriVids.C))
			{
				continue;
			}

			TidROI.Add(Tid);
		}

		// Do ROI contractions if needed. This is not relevant in FullyConstrained mode, where our changes
		//  are not dependent on triangles outside of our region of interest.
		if (BoundaryMode != EMegaMeshRemeshModifierBoundaryMode::FullyConstrained)
		{
			TSet<int32> TidsToRemove;

			// This disallows edits in cases where there's a neighboring triangle in the base
			//  mesh, so that we don't end up with a crack when we add a vertex in our view.
			if (bDisallowUnsafeBoundaryEdits)
			{
				TSet<int32> InternalBoundaryEdges;
				InMeshView.GetSubmeshInternalBoundaryEdges(InternalBoundaryEdges);
				for (int32 Eid : InternalBoundaryEdges)
				{
					TidsToRemove.Add(Submesh.GetEdgeT(Eid).A);
				}
			}

			// This disallows edits that cause splits of triangles that are in our view, but not
			//  fully contained in our coverage (if the coverage is not axis aligned)
			if (bDisallowSafeEditsOutsideCoverage)
			{
				for (int32 Tid : TidROI)
				{
					FIndex3i TriEids = Submesh.GetTriEdges(Tid);
					for (int i = 0; i < 3; ++i)
					{
						FIndex2i EdgeT = Submesh.GetEdgeT(TriEids[i]);
						int32 OtherTid = EdgeT.A == Tid ? EdgeT.B : EdgeT.A;
						if (OtherTid != IndexConstants::InvalidID && !TidROI.Contains(OtherTid))
						{
							TidsToRemove.Add(Tid);
							break;
						}
					}
				}
			}

			if (!TidsToRemove.IsEmpty())
			{
				TidROI = TidROI.Difference(TidsToRemove);
			}
		}


		RemeshROI(Submesh, MegaMeshTransform, TidROI);

	}

}
