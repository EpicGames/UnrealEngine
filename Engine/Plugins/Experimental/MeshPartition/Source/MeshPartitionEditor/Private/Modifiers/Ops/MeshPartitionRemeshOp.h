// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionChannel.h"
#include "CleaningOps/RemeshMeshOp.h"
#include "Modifiers/MeshPartitionRemeshModifierTypes.h"

namespace UE::MeshPartition
{
	class FMegaMeshRemeshBackgroundOpBase : public MeshPartition::IModifierBackgroundOp
	{
	public:

		FMegaMeshRemeshBackgroundOpBase(const FName& InOperationName);
		~FMegaMeshRemeshBackgroundOpBase() = default;

		FBox GlobalBounds;
		Geometry::FAxisAlignedBox3d LocalCoverage;
		FTransform ModifierToWorld;
		bool bComputeNormalSeams;
		double NormalSeamDotProductThreshold;
		EMegaMeshRemeshModifierBoundaryMode BoundaryMode;
		bool bDisallowUnsafeBoundaryEdits;
		bool bDisallowSafeEditsOutsideCoverage;
		float TargetEdgeLength;
		int32 RemeshIterations;
		float SmoothingStrength;
		ERemeshSmoothingType SmoothingType;
		bool bProjectToInputMesh;

		bool bUseDensityWeightChannel = false;
		MeshPartition::FChannelName DensityWeightChannelName;
		float RelativeDensity;
		bool bUseWeightThreshold = false;
		float MinWeightThreshold = 0.f;

	protected:

		void RemeshROI(FDynamicMesh3& Mesh, const FTransform3d& MegaMeshTransform, const TSet<int32>& TriangleROI) const;
	};


	class FMegaMeshRemeshBackgroundOp : public FMegaMeshRemeshBackgroundOpBase
	{
	public:
		FMegaMeshRemeshBackgroundOp(const FName& InOperationName);
		~FMegaMeshRemeshBackgroundOp() = default;

		// Generate a new random guid before submitting any code changes to the op
		static FGuid GetCodeVersionKey()
		{
			static FGuid VersionKey(TEXT("20d6f04f-c4e8-41e4-8479-d621a86ee652"));
			return VersionKey;
		}

	private:

		virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
		virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
			const FInstanceInfo& InInstanceDesc) const override;

		// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
		// and poisoning the cache/generating lots of unused intermediate data.
		virtual bool DisableDDCWrite() const override 
		{ 
			return false; 
		}
	};

} // namespace UE::MeshPartition
