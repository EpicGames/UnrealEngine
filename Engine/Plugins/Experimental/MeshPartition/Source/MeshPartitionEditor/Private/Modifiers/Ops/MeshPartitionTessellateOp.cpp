// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionTessellateOp.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "QueueRemesher.h"
#include "MeshConstraintsUtil.h"
#include "Operations/SelectiveTessellate.h"
#include "Selections/GeometrySelectionUtil.h"

namespace UE::MeshPartition
{
FMegaMeshTessellateBackgroundOpBase::FMegaMeshTessellateBackgroundOpBase(const FName& InOperationName)
	: MeshPartition::IModifierBackgroundOp(InOperationName)
{
}

void FMegaMeshTessellateBackgroundOpBase::TessellateROI(FDynamicMesh3& Mesh, const FTransform3d& MegaMeshTransform, const TSet<int32>& TriangleROI) const
{
	using namespace Geometry;

	FDynamicMesh3 InputMesh;
	InputMesh.Copy(Mesh);

	if (!ensure(TargetEdgeLength > 0.0))
	{
		return;
	}

	const FDynamicMeshWeightAttribute* DensityWeightLayer = nullptr;
	if (bUseDensityWeightChannel)
	{
		if (const FDynamicMeshAttributeSet* const Attribs = InputMesh.Attributes())
		{
			for (int32 Idx = 0; Idx < Attribs->NumWeightLayers(); ++Idx)
			{
				if (Attribs->GetWeightLayer(Idx)->GetName() == DensityWeightChannelName)
				{
					DensityWeightLayer = Attribs->GetWeightLayer(Idx);
				}
			}
		}
	}

	TFunction<int32(int32)> TessellationForTriangleFunc;
	if (bUseTargetEdgeLength)
	{
		TessellationForTriangleFunc = [&InputMesh, &DensityWeightLayer, this](int32 TriangleIndex) -> int32
		{
			auto TessellationForEdgeFunc = [&InputMesh, &DensityWeightLayer, this](int32 EdgeIndex) -> int32
			{
				if (InputMesh.IsEdge(EdgeIndex))
				{
					double EdgeLengthScale = 1.0;

					if (bUseDensityWeightChannel && DensityWeightLayer)
					{
						const FIndex2i EdgeVertices = InputMesh.GetEdgeV(EdgeIndex);

						float WeightValueA;
						DensityWeightLayer->GetValue(EdgeVertices[0], &WeightValueA);
						WeightValueA = FMath::Clamp(WeightValueA, 0.0f, 1.0f);

						float WeightValueB;
						DensityWeightLayer->GetValue(EdgeVertices[1], &WeightValueB);
						WeightValueB = FMath::Clamp(WeightValueB, 0.0f, 1.0f);

						const double AvgWeight = 0.5 * (WeightValueA + WeightValueB);

						EdgeLengthScale = FMath::Pow(0.5, -RelativeDensity * AvgWeight);
					}

					FVector3d EdgeEndA, EdgeEndB;
					InputMesh.GetEdgeV(EdgeIndex, EdgeEndA, EdgeEndB);
					const double CurrEdgeLength = EdgeLengthScale * FVector3d::Dist(EdgeEndA, EdgeEndB);
					const int32 EdgeLevel = FMath::Clamp(FMath::RoundToInt32(CurrEdgeLength / TargetEdgeLength) - 1, 0, MaxTessellationLevel);
					return EdgeLevel;
				}
				return 0;
			};

			if (InputMesh.IsTriangle(TriangleIndex))
			{
				const FIndex3i Edges = InputMesh.GetTriEdges(TriangleIndex);
				return FMath::Max3(TessellationForEdgeFunc(Edges[0]), TessellationForEdgeFunc(Edges[1]), TessellationForEdgeFunc(Edges[2]));
			}
			return 0;
		};
	}
	else
	{
		TessellationForTriangleFunc = [this](int32)
		{
			return TessellationLevel;
		};
	}


	switch (TessellationMethod)
	{
	case EMegaMeshRemeshModifierTessellateMethod::UniformRings:
	{
		TArray<int> TriTessLevels;
		TriTessLevels.Init(0, InputMesh.MaxTriangleID());
		for (const int32 InsideTriangleIndex : TriangleROI)
		{
			TriTessLevels[InsideTriangleIndex] = TessellationForTriangleFunc(InsideTriangleIndex);
		}

		TArray<int> EdgeTessLevels;
		EdgeTessLevels.Init(0, InputMesh.MaxEdgeID());
		for (int32 TriID : InputMesh.TriangleIndicesItr())
		{
			const FIndex3i Edges = InputMesh.GetTriEdges(TriID);
			EdgeTessLevels[Edges[0]] = FMath::Max(EdgeTessLevels[Edges[0]], TriTessLevels[TriID]);
			EdgeTessLevels[Edges[1]] = FMath::Max(EdgeTessLevels[Edges[1]], TriTessLevels[TriID]);
			EdgeTessLevels[Edges[2]] = FMath::Max(EdgeTessLevels[Edges[2]], TriTessLevels[TriID]);
		}
		FSelectiveTessellate Tessellate(&InputMesh, &Mesh);
		TUniquePtr<FTessellationPattern> Pattern = FSelectiveTessellate::CreateConcentricRingsTessellationPattern(&InputMesh, EdgeTessLevels, TriTessLevels);
		Tessellate.Pattern = Pattern.Get();
		Tessellate.Compute();
	}
	break;
	case EMegaMeshRemeshModifierTessellateMethod::AdaptiveRings:
	{
		TArray<int> TriTessLevels;
		TaperPerTriangleValues(InputMesh, TriangleROI, TessellationForTriangleFunc, TriTessLevels);

		TArray<int> EdgeTessLevels;
		EdgeTessLevels.Init(0, InputMesh.MaxEdgeID());
		for (int32 TriID : InputMesh.TriangleIndicesItr())
		{
			const FIndex3i Edges = InputMesh.GetTriEdges(TriID);
			EdgeTessLevels[Edges[0]] = FMath::Max(EdgeTessLevels[Edges[0]], TriTessLevels[TriID]);
			EdgeTessLevels[Edges[1]] = FMath::Max(EdgeTessLevels[Edges[1]], TriTessLevels[TriID]);
			EdgeTessLevels[Edges[2]] = FMath::Max(EdgeTessLevels[Edges[2]], TriTessLevels[TriID]);
		}

		FSelectiveTessellate Tessellate(&InputMesh, &Mesh);
		TUniquePtr<FTessellationPattern> Pattern = FSelectiveTessellate::CreateConcentricRingsTessellationPattern(&InputMesh, EdgeTessLevels, TriTessLevels);
		Tessellate.Pattern = Pattern.Get();
		Tessellate.Compute();
	}
	break;
	case EMegaMeshRemeshModifierTessellateMethod::AdaptiveRegular:
	{
		TArray<int> TriTessLevels;
		TaperPerTriangleValues(Mesh, TriangleROI, TessellationForTriangleFunc, TriTessLevels);

		FSelectiveTessellate Tessellate(&InputMesh, &Mesh);
		TUniquePtr<FTessellationPattern> Pattern = FSelectiveTessellate::CreateRedGreenTessellationPattern(&InputMesh, TriTessLevels);
		Tessellate.Pattern = Pattern.Get();
		Tessellate.Compute();
	}
	break;
	};

	if (bVertexSmoothing || bEdgeFlips)
	{
		PostProcess(Mesh);
	}
}

void FMegaMeshTessellateBackgroundOpBase::PostProcess(FDynamicMesh3& Mesh) const
{
	using namespace Geometry;

	TUniquePtr<FDynamicMesh3> OriginalMesh;
	TUniquePtr<FDynamicMeshAABBTree3> OriginalMeshSpatial;

	if (bResampleUVs)
	{
		OriginalMesh = MakeUnique<FDynamicMesh3>();
		OriginalMesh->Copy(Mesh);
		OriginalMeshSpatial = MakeUnique<FDynamicMeshAABBTree3>(OriginalMesh.Get(), true);
	}

	FQueueRemesher QueueRemesher(&Mesh);
	QueueRemesher.MaxFastSplitIterations = 0;
	QueueRemesher.MaxRemeshIterations = PostProcessingIterations;

	FRemesher Remesher(&Mesh);

	constexpr bool bUseQueueRemesher = false;
	FRemesher* UseRemesher = bUseQueueRemesher ? &QueueRemesher : &Remesher;

	// Disable collapses and splits
	UseRemesher->bEnableCollapses = false;
	UseRemesher->bEnableSplits = false;
	UseRemesher->bPreventTinyTriangles = false;

	// Flips
	UseRemesher->bEnableFlips = bEdgeFlips;
	UseRemesher->FlipMetric = FRemesher::EFlipMetric::OptimalValence;
	UseRemesher->bPreventNormalFlips = true;

	// Smoothing
	UseRemesher->bEnableSmoothing = bVertexSmoothing;
	UseRemesher->SmoothSpeedT = SmoothingStrength;
	UseRemesher->SmoothType = FRemesher::ESmoothTypes::Uniform;
	UseRemesher->bEnableParallelSmooth = true;
	// From header comments: If smoothing is done in-place, we don't need an extra buffer, but also there will some randomness introduced in results. Probably worse.
	UseRemesher->bEnableSmoothInPlace = false;

	FMeshConstraints Constraints;
	FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, Mesh, EEdgeRefineFlags::FullyConstrained, EEdgeRefineFlags::FullyConstrained, EEdgeRefineFlags::FullyConstrained, false, false, false, true);
	UseRemesher->SetExternalConstraints(MoveTemp(Constraints));

	if (bUseQueueRemesher)
	{
		QueueRemesher.FastestRemesh();
	}
	else
	{
		for (int32 Iteration = 0; Iteration < PostProcessingIterations; ++Iteration)
		{
			Remesher.BasicRemeshPass();
		}
	}

	if (bResampleUVs)
	{
		if (Mesh.HasAttributes() && Mesh.Attributes()->NumUVLayers() > 0)
		{

			TArray<FVector2f> SampledUVs;
			SampledUVs.SetNumUninitialized(Mesh.MaxVertexID());

			ParallelFor(Mesh.MaxVertexID(), [this, &Mesh, &OriginalMesh, &OriginalMeshSpatial, &SampledUVs](int32 VertexID)
				{
					const FVector3d VertexPos = Mesh.GetVertex(VertexID);

					TArray<int> ElementsForVID;
					Mesh.Attributes()->PrimaryUV()->GetVertexElements(VertexID, ElementsForVID);

					if (ElementsForVID.Num() == 1)
					{
						double DistSqr;
						const int TriangleID = OriginalMeshSpatial->FindNearestTriangle(VertexPos, DistSqr);

						if (TriangleID != FDynamicMesh3::InvalidID)
						{
							FVector3d Vertex1, Vertex2, Vertex3;
							OriginalMesh->GetTriVertices(TriangleID, Vertex1, Vertex2, Vertex3);

							FVector3d BaryCoords = VectorUtil::BarycentricCoords(VertexPos, Vertex1, Vertex2, Vertex3);
							FVector2d FoundUVs;
							OriginalMesh->Attributes()->PrimaryUV()->GetTriBaryInterpolate(TriangleID, &BaryCoords[0], &FoundUVs[0]);

							SampledUVs[VertexID] = (FVector2f)FoundUVs;
						}
					}
				});

			for (int32 VertexID : Mesh.VertexIndicesItr())
			{
				TArray<int> ElementsForVID;
				Mesh.Attributes()->PrimaryUV()->GetVertexElements(VertexID, ElementsForVID);

				if (ElementsForVID.Num() == 1)
				{
					Mesh.Attributes()->PrimaryUV()->SetElement(ElementsForVID[0], SampledUVs[VertexID]);
				}
			}
		}
	}
}


FMegaMeshTessellateBackgroundOp::FMegaMeshTessellateBackgroundOp(const FName& InOperationName)
	: FMegaMeshTessellateBackgroundOpBase(InOperationName)
{}

void FMegaMeshTessellateBackgroundOp::GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const
{
	AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);

	if (!OutInstanceInfos.IsEmpty())
	{
		OutInstanceInfos[0].ReadViewComponents = EMeshViewComponents::DynamicSubmesh;
		OutInstanceInfos[0].WriteViewComponents = EMeshViewComponents::DynamicSubmesh;
	}
}

void FMegaMeshTessellateBackgroundOp::ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::URemeshModifier::ApplyModifications);

	using namespace Geometry;

	FDynamicMesh3& SubMesh = InMeshView.GetSubmeshMutable();

	const FDynamicMeshWeightAttribute* DensityWeightLayer = nullptr;
	if (bUseDensityWeightChannel)
	{
		if (const FDynamicMeshAttributeSet* const Attribs = SubMesh.Attributes())
		{
			for (int32 Idx = 0; Idx < Attribs->NumWeightLayers(); ++Idx)
			{
				if (Attribs->GetWeightLayer(Idx)->GetName() == DensityWeightChannelName)
				{
					DensityWeightLayer = Attribs->GetWeightLayer(Idx);
				}
			}
		}
	}

	TSet<int32> InsideTriangles;
	for (int32 VertexID : SubMesh.VertexIndicesItr())
	{
		const FVector3d LocalPosition = ModifierTransform.InverseTransformPosition(InTransform.TransformPosition(SubMesh.GetVertex(VertexID)));
		if (LocalBounds.IsInside(LocalPosition))
		{
			// don't add vertex if weight is at/below threshold
			if (bUseWeightThreshold && DensityWeightLayer)
			{
				float Value;
				DensityWeightLayer->GetValue(VertexID, &Value);
				if (Value <= MinWeightThreshold)
				{
					continue;
				}
			}
			for (const int32 TriIndex : SubMesh.VtxTrianglesItr(VertexID))
			{
				InsideTriangles.Add(TriIndex);
			}
		}
	}

	TessellateROI(SubMesh, InTransform, InsideTriangles);
}

} // namespace UE::MeshPartition

