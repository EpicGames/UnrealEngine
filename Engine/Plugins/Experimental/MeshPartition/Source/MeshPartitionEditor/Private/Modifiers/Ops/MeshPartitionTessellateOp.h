// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionChannel.h"
#include "Modifiers/MeshPartitionRemeshModifierTypes.h"


namespace UE::MeshPartition
{
class FMegaMeshTessellateBackgroundOpBase : public MeshPartition::IModifierBackgroundOp
{
public:
	FMegaMeshTessellateBackgroundOpBase(const FName& InOperationName);
	~FMegaMeshTessellateBackgroundOpBase() = default;

	FBox LocalBounds;
	FBox GlobalBounds;

	bool bUseTargetEdgeLength;
	int32 TessellationLevel;
	float TargetEdgeLength;
	int32 MaxTessellationLevel;
		
	bool bUseDensityWeightChannel = false;
	MeshPartition::FChannelName DensityWeightChannelName;
	float RelativeDensity;
	bool bUseWeightThreshold = false;
	float MinWeightThreshold = 0.f;

	bool bVertexSmoothing;
	double SmoothingStrength;
	int32 PostProcessingIterations;
	bool bResampleUVs;
	bool bEdgeFlips;

	EMegaMeshRemeshModifierTessellateMethod TessellationMethod;
	FTransform ModifierTransform;

protected:

	void TessellateROI(FDynamicMesh3& Mesh, const FTransform3d& MegaMeshTransform, const TSet<int32>& TriangleROI) const;
	void PostProcess(FDynamicMesh3& Mesh) const;
};


class FMegaMeshTessellateBackgroundOp : public FMegaMeshTessellateBackgroundOpBase
{
public:
	FMegaMeshTessellateBackgroundOp(const FName& InOperationName);
	~FMegaMeshTessellateBackgroundOp() = default;

	// Generate a new random guid before submitting any code changes to the op
	static FGuid GetCodeVersionKey()
	{
		static FGuid VersionKey(TEXT("6d90ac8e-ffb2-4d02-a952-f618ac512f60"));
		return VersionKey;
	}

private:

	virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;
	virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
		const FInstanceInfo& InInstanceDesc) const override;

	// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
	// and poisoning the cache/generating lots of unused intermediate data.
	virtual bool DisableDDCWrite() const override
	{
		return false;
	}
};
}
