// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Spatial/SampledScalarField2.h"

namespace UE::MeshPartition
{
class FTexturePatchModifierOp : public MeshPartition::IModifierBackgroundOp
{
public:
	FTexturePatchModifierOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

	// Falloff-related values per texture, to use in the loop further below
	struct FFalloffData
	{
		double ClampedRadius = 0;
		double ClampedFalloff = 0;
		FVector2d CornerCenter = FVector2d::Zero();
		MeshPartition::ETexturePatchFalloffMode FalloffModeToUse = MeshPartition::ETexturePatchFalloffMode::Smooth;
	};
	struct FTextureEntry
	{
		FTextureEntry(TSharedRef<const FChannelFieldData> InFieldData)
			: FieldData(InFieldData)
		{
		}

		MeshPartition::ETexturePatchBlendMode BlendMode;
		double SoftnessParameter;

		TSharedRef<const FChannelFieldData> FieldData;

		double ZeroInEncoding;
		double EncodingScale;

		MeshPartition::ETexturePatchAlphaMode AlphaMode;
		double SelfMaskTolerance;

		TSharedPtr<const FRichCurve> ValueCurve;
		double ValueCurveOffset;
		double ValueCurveScale;

		MeshPartition::ETexturePatchFalloffMode FalloffMode;
		float FalloffDistance;
		float CornerRadius;
		TSharedPtr<const FRichCurve> FalloffCurve;
	};
	struct FHeightEntry : public FTextureEntry
	{
		FHeightEntry(TSharedRef<const FChannelFieldData> InFieldData, TSharedRef<Utils::TAsyncTransform<Geometry::FDisplacementMap>> InDisplacementMap)
			: FTextureEntry(InFieldData)
			, DisplacementMap(InDisplacementMap)
		{
		}

		TSharedRef<Utils::TAsyncTransform<Geometry::FDisplacementMap>> DisplacementMap;
		FName HeightScaleWeightChannel;
	};
	struct FWeightEntry : public FTextureEntry
	{
		FWeightEntry(TSharedRef<const FChannelFieldData> InFieldData)
			: FTextureEntry(InFieldData)
		{
		}

		FName WeightChannelName;
		bool bApplyHeightMinMaxBlend;
		double HeightMinMaxBlendDistance;
	};

	TOptional<FHeightEntry> HeightChannel;
	TArray<FWeightEntry> WeightChannels;
		
	TArray<FBox> GlobalBounds;
	TArray<FTransform> PatchTransforms;
	FVector2D UnscaledPatchCoverage;
	double VerticalPatchExtentUp;
	double VerticalPatchExtentDown;

	MeshPartition::ETexturePatchTessellationMode TessellationMode;
	MeshPartition::ETexturePatchTessellationErrorMode TessellationErrorMode;
	float TessellationError;
	float MinimumEdgeLength;
	float MaximumEdgeLength;
	float FeatureSensitivity;
	bool bMeshRegularization;
	FName DetailAdjustmentChannelName;

	static double GetFalloffAlpha(const FVector2d LocalPatchExtent, const FVector2d Local2DPosition, const FTextureEntry& Entry, const FFalloffData& FalloffData);

	static double ApplyCurve(const FTextureEntry& InTextureEntry, double Value);

	// Gets alpha at sampling coordinates, using appropriate alpha mode
	static float GetAlphaValue(const FTextureEntry& InTextureEntry, const FVector2f& InSamplingCoordinates, float Value);

	static void PrepFalloffData(const FVector2d& LocalPatchExtent, const FTextureEntry& TextureEntry, FFalloffData& Data);

	virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override;

	virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
		const FInstanceInfo& InInstanceDesc) const override;

	// Generate a new random guid before submitting any code changes to the op
	static FGuid GetCodeVersionKey()
	{
		static FGuid VersionKey(TEXT("faa3e01d-6595-47e4-aa6d-e047edf48973"));
		return VersionKey;
	}

	// Set to true whenever iterating on code changes to prevent any builds including this modifier being picked up by ddc
	// and poisoning the cache/generating lots of unused intermediate data.
	virtual bool DisableDDCWrite() const override { return false; }

private:

	bool ShouldApplyAdaptiveDisplacement() const 
	{
		return TessellationMode != MeshPartition::ETexturePatchTessellationMode::NonAdaptive && HeightChannel.IsSet();
	}

	void ApplyAdaptiveDisplacement(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
		const FInstanceInfo& InInstanceDesc) const;

	void ApplyDisplacement(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform,
		const FInstanceInfo& InInstanceDesc) const;
};

}