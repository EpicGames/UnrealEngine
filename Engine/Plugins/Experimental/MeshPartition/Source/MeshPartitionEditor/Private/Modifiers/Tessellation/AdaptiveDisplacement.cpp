// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Tessellation/AdaptiveDisplacement.h"
#include "Modifiers/Tessellation/HalfEdgeMesh.h"
#include "Modifiers/Tessellation/MeshPostProcessing.h"
#include "MeshPartitionEditorModule.h" // for log

#include "TriangleUtil.h"
#include "LerpVert.h"
#include "Async/ParallelFor.h"
#include "Logging/LogMacros.h"

#include "ImageCore.h"
#include "ImageCoreUtils.h"

#include "LerpVert.h"
#include "Tessellation/Affine.h"
#include "Image/DisplacementMap.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshNormals.h"
#include "Util/IndexUtil.h"
#include "VectorUtil.h"

#include "AdaptiveTessellator.h"

#include "AdaptiveTessellatorMesh.h"
#include "AdaptiveTessellatorDisplace.h"
#include "Tessellation/AdaptiveTessellator.h"
#include "Tessellation/ParallelAdaptiveRefinement.h"
#include "Tessellation/DynamicMeshPolicy.h"
#include "Tessellation/FeatureAdaptive.h"
#include "Tessellation/AdaptiveMetrics.h"

#include "IndexTypes.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "HAL/IConsoleManager.h"

#include "MeshConstraintsUtil.h"
#include "MeshPartitionMeshData.h"

namespace UE::MeshPartition {

using namespace Nanite;

static TAutoConsoleVariable<int32> CVarExactDisplacementBounds(
	TEXT("MegaMesh.AdaptiveTessellation.DisplacementBoundsRefinements"),
	1,
	TEXT("number of refinements for displacement level bounds")
);

static TAutoConsoleVariable<int32> CVarParallelTessellationBatchsize(
	TEXT("MegaMesh.AdaptiveTessellation.Batchsize"),
	1024,
	TEXT("Batch size for parallel execution in parallel tessellation. <= 0: single threaded")
);

static TAutoConsoleVariable<int32> CVarTessellationMaxRefinement(
	TEXT("MegaMesh.AdaptiveTessellation.MaxRefinementLevel"),
	10,
	TEXT("maximum number of refinements")
);


template <typename T>
inline UE::Math::TVector2<T> CalculateErrorBounds(
	const UE::Math::TVector<T> Barycentrics[3],
	const UE::Math::TVector<T> VertexNormals[3],
	const FVector2f VertexUVs[3],
	const UE::Math::TVector<T> Displacements[3],
	const UE::Geometry::FDisplacementMap& DisplacementMap,
	const T Center,
	const FVector2f Magnitude, 
	const int32 DisplacementBoundsRefinements)
{
	FVector2f UVBounds[2];
	FVector3f Barycentrics3f[3] = { FVector3f(Barycentrics[0]), FVector3f(Barycentrics[1]), FVector3f(Barycentrics[2]) };
	UE::Geometry::TessellationUtil::CalculateUVBounds(Barycentrics3f, VertexUVs, UVBounds);

	if (UVBounds[0][0] > 1.f || UVBounds[0][1] > 1.f || UVBounds[1][0] < 0.f || UVBounds[1][1] < 0.f)
	{
		return UE::Math::TVector2<T>(T(0.));
	}

	return UE::Geometry::TessellationUtil::CalculateErrorBounds(Barycentrics, VertexNormals, UVBounds, Displacements, DisplacementMap, Center, Magnitude, DisplacementBoundsRefinements);
}

struct FDynamicMeshDisplacementPolicyPerVertex
{
	using FIndex3i = Geometry::FIndex3i;

	FDynamicMeshDisplacementPolicyPerVertex(
		const Geometry::FDynamicMesh3& InMesh,
		const Geometry::FDisplacementMap& InDisplacementMap,
		const double InSampleRate,
		const double InTargetError,
		const float InCenter,
		const float InMagnitude,
		const float InMinimumEdgeLength,
		const float InMaximumEdgeLength,
		const float InFeatureSensitivity,
		const Geometry::TDynamicMeshVertexAttribute<float, 2>* InSamplingCoords,
		const Geometry::TDynamicMeshVertexAttribute<float, 1>* InSampleRateWeight,
		const Geometry::TDynamicMeshVertexAttribute<float, 1>* InHeightScale )
		: Mesh(InMesh)
		, DisplacementMap(InDisplacementMap)
		, SampleRate(InSampleRate)
		, TargetError(InTargetError)
		, Center(InCenter)
		, Magnitude(InMagnitude)
		, MinimumEdgeLength(InMinimumEdgeLength)
		, MaximumEdgeLength(InMaximumEdgeLength)
		, FeatureSensitivity(InFeatureSensitivity)
		, SamplingCoords(InSamplingCoords)
		, SampleRateWeightLayer(InSampleRateWeight)
		, HeightScaleLayer(InHeightScale)
		, MaxRefinementLevel(CVarTessellationMaxRefinement.GetValueOnAnyThread())
		, DisplacementBoundsRefinements(CVarExactDisplacementBounds.GetValueOnAnyThread())
		, LongestEdgeRatioThreshold(0.95f)
	{
		check(Mesh.HasVertexNormals());
	}

	inline FVector2f GetVertexUV(int VertexIndex) const
	{
		FVector2f UV;
		SamplingCoords->GetValue(VertexIndex, UV);
		return UV;
	}

	inline FVector3d GetVertexDisplacement(int32 VertexIndex, const int TriIndex) const
	{
		FVector2f UV = GetVertexUV(VertexIndex);

		if (UV[0] < 0.f || UV[0] > 1.f || UV[1] < 0.f || UV[1] > 1.f)
			return FVector3d(0.f);

		float Scale = 1.f;
		if (HeightScaleLayer)
		{
			HeightScaleLayer->GetValue<float*const>(VertexIndex, &Scale);
		}

		return FVector3d(Mesh.GetVertexNormal(VertexIndex) * (DisplacementMap.Sample(UV) - Center) * Scale * Magnitude);
	}

	inline FVector3d GetDisplacement(const FVector3d Barycentrics, const int TriIndex) const
	{
		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		FVector3f TriNormals[3] = {	Mesh.GetVertexNormal(Triangle[0]),
			Mesh.GetVertexNormal(Triangle[1]),
			Mesh.GetVertexNormal(Triangle[2]) };

		FVector3f Normal = TriNormals[0] * Barycentrics.X +
						   TriNormals[1] * Barycentrics.Y +
						   TriNormals[2] * Barycentrics.Z;
		Normal.Normalize();

		FVector2f UVs[3] = { GetVertexUV(Triangle[0]),
			GetVertexUV(Triangle[1]),
			GetVertexUV(Triangle[2]) };

		const FVector2f UV = UVs[0] * Barycentrics.X +
							 UVs[1] * Barycentrics.Y +
							 UVs[2] * Barycentrics.Z;

		if (UV[0] < 0.f || UV[0] > 1.f || UV[1] < 0.f || UV[1] > 1.f)
			return FVector3d(0.);

		float Scale = 1.f;
		if (HeightScaleLayer)
		{
			FVector3f VertexWeights;
			HeightScaleLayer->GetValue<float*const>(Triangle[0], &VertexWeights[0]);
			HeightScaleLayer->GetValue<float*const>(Triangle[1], &VertexWeights[1]);
			HeightScaleLayer->GetValue<float*const>(Triangle[2], &VertexWeights[2]);
			Scale = VertexWeights | FVector3f(Barycentrics);
		}

		return FVector3d(Normal) * (DisplacementMap.Sample(UV) - Center) * Magnitude * Scale;
	}

	inline FVector2d GetErrorBounds( const FVector3d* const Barycentrics,
									 const FVector3d Displacement0,
									 const FVector3d Displacement1,
									 const FVector3d Displacement2,
									 const int TriIndex ) const
	{
		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		FVector3f VertexNormals[3] = { 
			Mesh.GetVertexNormal(Triangle[0]),
			Mesh.GetVertexNormal(Triangle[1]),
			Mesh.GetVertexNormal(Triangle[2]) };

		FVector3d VertexNormals64[3] = { FVector3d(VertexNormals[0]), FVector3d(VertexNormals[1]), FVector3d(VertexNormals[2]) };

		FVector2f UVs[3] = { 
			GetVertexUV(Triangle[0]),
			GetVertexUV(Triangle[1]),
			GetVertexUV(Triangle[2]) };

		const FVector3d Displacements[] = { Displacement0, Displacement1, Displacement2 };

		FVector2f MagnitudeScaleBounds(Magnitude);
		if (HeightScaleLayer)
		{
			float VertexWeights[3];
			HeightScaleLayer->GetValue<float*const>(Triangle[0], &VertexWeights[0]);
			HeightScaleLayer->GetValue<float*const>(Triangle[1], &VertexWeights[1]);
			HeightScaleLayer->GetValue<float*const>(Triangle[2], &VertexWeights[2]);
			
			MagnitudeScaleBounds *= FVector2f(FMath::Min3(VertexWeights[0], VertexWeights[1], VertexWeights[2]), 
			                                  FMath::Max3(VertexWeights[0], VertexWeights[1], VertexWeights[2]));
		}

		return CalculateErrorBounds<double>(Barycentrics, VertexNormals64, UVs, Displacements, DisplacementMap, Center, MagnitudeScaleBounds, DisplacementBoundsRefinements);
	}

	inline int32 GetNumSamples(const FVector3d* const Barycentrics, const FIndex3i& Triangle, const int TriIndex) const
	{
		FVector2f UVs[3] = { 
			GetVertexUV(Triangle[0]),
			GetVertexUV(Triangle[1]),
			GetVertexUV(Triangle[2]) };

		return UE::Geometry::TessellationUtil::GetNumSamples( Barycentrics, UVs, FVector2f(DisplacementMap.GetSizeX(), DisplacementMap.GetSizeY()));
	}

	inline bool ShouldRefine(const int32 TriIndex, const TArray<FVector3d>& Displacements, FVector3d& SplitVertexBary, const int32 Level) const
	{
		if (Level >= MaxRefinementLevel) 
		{
			return false;
		}

		FVector3d P0, P1, P2;
		Mesh.GetTriVertices(TriIndex, P0, P1, P2);

		const FVector3d Edge01 = P1 - P0;
		const FVector3d Edge12 = P2 - P1;
		const FVector3d Edge20 = P0 - P2;

		const FVector3d EdgeLengths( Edge01.Length(), Edge12.Length(), Edge20.Length() );

		SplitVertexBary = FVector3d( 1./3., 1./3., 1./3. );

		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		float ScaledSampleRate  = SampleRate;
		float ScaledTargetError = TargetError; 
		if (SampleRateWeightLayer)
		{
			float VertexWeights[3];
			SampleRateWeightLayer->GetValue<float*const>(Triangle[0], &VertexWeights[0]);
			SampleRateWeightLayer->GetValue<float*const>(Triangle[1], &VertexWeights[1]);
			SampleRateWeightLayer->GetValue<float*const>(Triangle[2], &VertexWeights[2]);
			
			for (int Idx = 0; Idx < 3; ++Idx)
			{
				VertexWeights[Idx] = 1./ FMath::Pow(2.0, VertexWeights[Idx]);
			}

			const float TriSampleRateWeight = FMath::Clamp(1.f/3.f * (VertexWeights[0] + VertexWeights[1] + VertexWeights[2]), 0.1f, 1000.f);
			ScaledSampleRate  *= TriSampleRateWeight;
			ScaledTargetError *= TriSampleRateWeight * TriSampleRateWeight;  
		}

		FVector2f UVs[3] = { GetVertexUV(Triangle[0]),
							 GetVertexUV(Triangle[1]),
							 GetVertexUV(Triangle[2]) };

		// check for padding region
		const FVector2f MinUV(UVs[0].ComponentMin(UVs[1]).ComponentMin(UVs[2]));
		const FVector2f MaxUV(UVs[0].ComponentMax(UVs[1]).ComponentMax(UVs[2]));
		
		if (MaxUV[0] < 0.f || MaxUV[1] < 0.f || MinUV[0] > 1.f || MinUV[1] > 1.f)
		{
			return false;
		}

		if( EdgeLengths[0] < MinimumEdgeLength &&
			EdgeLengths[1] < MinimumEdgeLength &&
			EdgeLengths[2] < MinimumEdgeLength )
		{
			return false;
		}

		if( EdgeLengths[0] > MaximumEdgeLength ||
			EdgeLengths[1] > MaximumEdgeLength ||
			EdgeLengths[2] > MaximumEdgeLength )
		{
			return true;
		}

		const FVector3d Displacement0 = Displacements[Triangle.A];
		const FVector3d Displacement1 = Displacements[Triangle.B];
		const FVector3d Displacement2 = Displacements[Triangle.C];

		if (FeatureSensitivity > 0.f)
		{
			Geometry::TFeatureSensitivityRefinementCheck<double> RefinementCheck(FeatureSensitivity, ScaledSampleRate, P0 + Displacement0, P1 + Displacement1, P2 + Displacement2);

			if (RefinementCheck.NeedsCheck())
			{
				for (int32 EdgeIdx = 0; EdgeIdx < 3; ++EdgeIdx)
				{
					const Geometry::FDynamicMesh3::FEdge Edge = Mesh.GetEdge(Mesh.GetTriEdge(TriIndex, EdgeIdx));

					if (Edge.Tri[1] != IndexConstants::InvalidID)
					{
						const int32 AdjTriIdx = (Edge.Tri[0] == TriIndex) ? Edge.Tri[1] : Edge.Tri[0];

						const FIndex3i AdjTri = Mesh.GetTriangle(AdjTriIdx);
						const int32 FarLocalIndex = IndexUtil::FindTriOtherIndex(Edge.Vert[0], Edge.Vert[1], AdjTri);
						const int32 FarVID = AdjTri[FarLocalIndex];

						check(AdjTri[(FarLocalIndex+1)%3] == Edge.Vert[0] || AdjTri[(FarLocalIndex+1)%3] == Edge.Vert[1]);
						check(AdjTri[(FarLocalIndex+2)%3] == Edge.Vert[0] || AdjTri[(FarLocalIndex+2)%3] == Edge.Vert[1]);

						const FVector3d P = Mesh.GetVertex(FarVID) + Displacements[FarVID];

						if (RefinementCheck.CheckCriterion(EdgeIdx, P))
						{
							return true;
						}
					}
				}
			}
		}

		if( EdgeLengths[0] < ScaledSampleRate &&
			EdgeLengths[1] < ScaledSampleRate &&
			EdgeLengths[2] < ScaledSampleRate )
		{
			return false;
		}
		
		// avoid skinny triangles
		const int32 LongestEdgeIdx = FMath::Max3Index( EdgeLengths[0], EdgeLengths[1], EdgeLengths[2] );
		const double LongestEdge = EdgeLengths[LongestEdgeIdx];
		const double RemainingEdgeLen = EdgeLengths[(LongestEdgeIdx+1)%3] + EdgeLengths[(LongestEdgeIdx+2)%3];
			
		if (LongestEdge > LongestEdgeRatioThreshold * RemainingEdgeLen)
		{
			// try edge split
			if (!Mesh.IsBoundaryEdge(Mesh.GetTriEdge(TriIndex, LongestEdgeIdx)))
			{
				SplitVertexBary[LongestEdgeIdx]       = 0.5f;
				SplitVertexBary[(LongestEdgeIdx+1)%3] = 0.5f;
				SplitVertexBary[(LongestEdgeIdx+2)%3] = 0.0f;
				
				return true;
			}

			return false;
		}
	
		FVector3d Barycentrics[3] =
		{
			{ 1.0f, 0.0f, 0.0f },
			{ 0.0f, 1.0f, 0.0f },
			{ 0.0f, 0.0f, 1.0f },
		};

		const FVector2d ErrorBounds = GetErrorBounds(
			Barycentrics,
			Displacement0,
			Displacement1,
			Displacement2,
			TriIndex);

		if (ErrorBounds[1] > ScaledTargetError)
		{
			return true;
		}

		return false;
	}

private:
	const Geometry::FDynamicMesh3&    Mesh;
	const Geometry::FDisplacementMap& DisplacementMap;
	const double                      SampleRate;
	const double                      TargetError;
	const float                       Center;
	const float                       Magnitude;
	const float                       MinimumEdgeLength;
	const float                       MaximumEdgeLength;
	const float                       FeatureSensitivity;
	
	const Geometry::TDynamicMeshVertexAttribute<float,2>* SamplingCoords        { nullptr };
	const Geometry::TDynamicMeshVertexAttribute<float,1>* SampleRateWeightLayer { nullptr };
	const Geometry::TDynamicMeshVertexAttribute<float,1>* HeightScaleLayer      { nullptr };
	

	const int32                    MaxRefinementLevel;
	const int32                    DisplacementBoundsRefinements;
	const float                    LongestEdgeRatioThreshold;
};

// apply adaptive tessellation on FDynamicMesh3 directly, avoid UV and normal attribute layers
void TessellateAdaptiveNative(Geometry::FDynamicMesh3& Mesh,
	Geometry::FMeshConstraints& MeshConstraints,
	const FTransform3d& MeshToWorld,
	const FTransform3d& PatchToWorld,
	const FVector2D& UnscaledPatchCoverage,
	const Geometry::FDisplacementMap& DisplacementMap,
	const float Center,
	const float Magnitude,
	const float SampleRate,
	const float MinimumEdgeLength,
	const float MaximumEdgeLength,
	const float FeatureSensitivity,
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* SampleRateWeightLayer,
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* HeightScaleLayer )
{
 	UE_LOGF(LogMegaMeshEditor, Verbose, "Running adaptive tessellation on FDynamicMesh3");

	Geometry::FDynamicMeshAttributeSet* AttributeSet = Mesh.Attributes();
	check(Mesh.Attributes());

	if (!Mesh.HasVertexNormals())
	{
		Geometry::FMeshNormals MeshNormals(&Mesh);
		MeshNormals.ComputeVertexNormals();
		MeshNormals.CopyToVertexNormals(&Mesh, false);
	}

	Geometry::TDynamicMeshVertexAttribute<float, 2>* SamplingCoords = new Geometry::TDynamicMeshVertexAttribute<float, 2>(&Mesh);
	const FName LayerName = "AdaptiveTess.SamplingCoords";
	AttributeSet->AttachAttribute(LayerName, SamplingCoords);

	for (int32 VID : Mesh.VertexIndicesItr())
	{
		check(Mesh.IsVertex(VID)); // VertexIndicesItr only walks over valid vertices
		const FVector3d PatchLocalVertPosition = PatchToWorld.InverseTransformPosition(MeshToWorld.TransformPosition(Mesh.GetVertex(VID)));
		const FVector2f UV = FVector2f(PatchLocalVertPosition.X / UnscaledPatchCoverage.X + 0.5, PatchLocalVertPosition.Y / UnscaledPatchCoverage.Y + 0.5);
		SamplingCoords->SetValue(VID, UV);
	}

	const size_t VerticesBefore = Mesh.VertexCount();

	Geometry::FDynamicMesh3Policy DynamicMesh3Policy(Mesh, MeshConstraints);
	FDynamicMeshDisplacementPolicyPerVertex DynamicMeshDisplacementPolicy(Mesh, DisplacementMap, SampleRate, SampleRate * SampleRate, 
		Center, Magnitude, MinimumEdgeLength, MaximumEdgeLength, FeatureSensitivity, SamplingCoords, SampleRateWeightLayer, HeightScaleLayer);
	using TessellatorT = Geometry::TAdaptiveTessellator<Geometry::FDynamicMesh3Policy, FDynamicMeshDisplacementPolicyPerVertex>;

	Geometry::FAdaptiveTessellatorOptions Options;
	Options.TargetError      = SampleRate * SampleRate;
	Options.SampleRate       = SampleRate;
	Options.bCrackFree       = false;
	Options.bFinalDisplace   = false;
	Options.MaxTriangles     = 5'000'000u;
	Options.RefinementMethod = Geometry::EAdaptiveTessellationRefinementMethod::Custom;

	TessellatorT Tessellator(DynamicMesh3Policy, DynamicMeshDisplacementPolicy, Options);

	UE_LOGF(LogMegaMeshEditor, Verbose, "Adaptive tessellation increased number of vertices from %zu to %d (#triangles: %d)",
		VerticesBefore, Mesh.VertexCount(), Mesh.TriangleCount() );

	AttributeSet->RemoveAttribute(LayerName);
}

// Used for parallel tessellation
class FHalfEdgeRefinementPolicy
{
public:
	using FIndex3i = Geometry::FIndex3i;
	using FIndex2i = Geometry::FIndex2i;

	FHalfEdgeRefinementPolicy(Geometry::FHalfEdgeMesh& InMesh,
		const Geometry::FDisplacementMap& InDisplacementMap,
		const float InCenter,
		const float InMagnitude,
		const float InSampleRate,
		const float InTargetError,
		const float InMinimumEdgeLength,
		const float InMaximumEdgeLength,
		const float InFeatureSensitivity,
		const Geometry::TDynamicMeshVertexAttribute<float, 1>* InSampleRateWeight,
		const Geometry::TDynamicMeshVertexAttribute<float, 1>* InHeightScale)
		: Mesh(InMesh)
		, DisplacementMap(InDisplacementMap)
		, Center(InCenter)
		, Magnitude(InMagnitude)
		, SampleRate(InSampleRate)
		, TargetError(InTargetError)
		, MinimumEdgeLength(InMinimumEdgeLength)
		, MaximumEdgeLength(InMaximumEdgeLength)
		, MaxRefinementLevel(CVarTessellationMaxRefinement.GetValueOnAnyThread())
		, DisplacementBoundsRefinements(CVarExactDisplacementBounds.GetValueOnAnyThread())
		, FeatureSensitivity(InFeatureSensitivity)
		, SampleRateWeightLayer(InSampleRateWeight)
		, HeightScaleLayer(InHeightScale)
	{
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT("FParallelAdaptiveRefinement.InitDisplacements");
			Displacements.Init( FVector3f( std::numeric_limits<float>::signaling_NaN(),
										   std::numeric_limits<float>::signaling_NaN(),
										   std::numeric_limits<float>::signaling_NaN() ), Mesh.MaxVertexID() );

			for( int32 TriIndex = 0; TriIndex < Mesh.MaxTriID(); ++TriIndex )
			{
				if (!Mesh.IsValidTri(TriIndex))
				{
					continue;
				}
				for( int k = 0; k < 3; k++ )
				{
					const uint32 VID = Mesh.GetVertexIndex(TriIndex, k);
					check(Mesh.IsValidVertex(VID));
					Displacements[VID] = GetVertexDisplacement(VID, TriIndex);
				}
			}
		}
	}

	void AllocateVertices(int32 VerticesToAdd)
	{
		Displacements.AddDefaulted(VerticesToAdd);
	}

	void VertexAdded(int32 VertexIndex, FIndex2i Tris)
	{
		Displacements[VertexIndex] = GetVertexDisplacement(VertexIndex, Tris[0]); // todo, could do averaging if material indices are different
	}

	[[nodiscard]] FORCEINLINE float EvaluateDisplacement(const FVector2f UV) const
	{
		if (UV[0] < 0.f || UV[0] > 1.f || UV[1] < 0.f || UV[1] > 1.f)
			return 0.f;

		return (DisplacementMap.Sample(UV) - Center) * Magnitude;
	}

	[[nodiscard]] FORCEINLINE FVector3f GetVertexDisplacement(const int32 VertexIndex, const int32 TriIndex) const
	{
		// we don't need to interpolate here
		const FVector2f UV = Mesh.GetUV(VertexIndex);

		float Scale = 1.f;
		if (HeightScaleLayer)
		{
			const Geometry::FHalfEdgeMesh::FVertex Vertex = Mesh.GetVertex(VertexIndex);
			const Geometry::FIndex3i BaseTri = Mesh.GetBaseTriangle(Mesh.GetBaseTriangleIndex(TriIndex));

			float VertexWeights[3];
			HeightScaleLayer->GetValue<float*const>(BaseTri[0], &VertexWeights[0]);
			HeightScaleLayer->GetValue<float*const>(BaseTri[1], &VertexWeights[1]);
			HeightScaleLayer->GetValue<float*const>(BaseTri[2], &VertexWeights[2]);
			Scale = VertexWeights[0] * Vertex.Barycentric[0] + VertexWeights[1] * Vertex.Barycentric[1] + VertexWeights[2] * Vertex.Barycentric[2];
		}

		return Mesh.GetNormal(VertexIndex) * EvaluateDisplacement(UV) * Scale;
	}

	[[nodiscard]] FORCEINLINE FVector3f GetDisplacement(const FVector3f Barycentrics, const int TriIndex) const
	{
		FVector2f UV;
		FVector3f Normal;
		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		UV  = Mesh.GetUV(Triangle.A) * Barycentrics.X;
		UV += Mesh.GetUV(Triangle.B) * Barycentrics.Y;
		UV += Mesh.GetUV(Triangle.C) * Barycentrics.Z;

		Normal  = Mesh.GetNormal(Triangle.A) * Barycentrics.X;
		Normal += Mesh.GetNormal(Triangle.B) * Barycentrics.Y;
		Normal += Mesh.GetNormal(Triangle.C) * Barycentrics.Z;
		Normal.Normalize();

		float Scale = 1.f;
		if (HeightScaleLayer)
		{
			const Geometry::FIndex3i BaseTri = Mesh.GetBaseTriangle(Mesh.GetBaseTriangleIndex(TriIndex));

			float VertexWeights[3];
			HeightScaleLayer->GetValue<float*const>(BaseTri[0], &VertexWeights[0]);
			HeightScaleLayer->GetValue<float*const>(BaseTri[1], &VertexWeights[1]);
			HeightScaleLayer->GetValue<float*const>(BaseTri[2], &VertexWeights[2]);

			const FVector3f Barycentric[3] = { 
				Mesh.GetVertex(Triangle.A).Barycentric,
				Mesh.GetVertex(Triangle.B).Barycentric,
				Mesh.GetVertex(Triangle.C).Barycentric };

			const FVector3f CombinedBary( 
				Barycentrics | FVector3f(Barycentric[0][0], Barycentric[1][0], Barycentric[2][0]),
				Barycentrics | FVector3f(Barycentric[0][1], Barycentric[1][1], Barycentric[2][1]),
				Barycentrics | FVector3f(Barycentric[0][2], Barycentric[1][2], Barycentric[2][2]) );

			Scale = VertexWeights[0] * CombinedBary[0] + 
			        VertexWeights[1] * CombinedBary[1] + 
					VertexWeights[2] * CombinedBary[2];
		}

		return Normal * EvaluateDisplacement(UV) * Scale;
	}

	[[nodiscard]] FORCEINLINE FVector2f GetErrorBounds(const FVector3f* const Barycentrics, const FVector3f Displacement0, const FVector3f Displacement1, const FVector3f Displacement2, const int TriIndex, const FVector2f MagnitudeScaleBounds ) const
	{
		const FIndex3i Triangle = Mesh.GetTriangle(TriIndex);

		const FVector2f UVs[] = { Mesh.GetUV(Triangle.A), Mesh.GetUV(Triangle.B), Mesh.GetUV(Triangle.C) };
		const FVector3f Normals[] = { Mesh.GetNormal(Triangle.A), Mesh.GetNormal(Triangle.B), Mesh.GetNormal(Triangle.C) };
		const FVector3f Disps[] = { Displacement0, Displacement1, Displacement2 };

		return CalculateErrorBounds(Barycentrics, Normals, UVs, Disps, DisplacementMap, Center, MagnitudeScaleBounds, DisplacementBoundsRefinements);
	}

	[[nodiscard]] FORCEINLINE int32 GetNumSamples(const FVector3f* const Barycentrics, const FIndex3i& Triangle, const int TriIndex) const
	{
		const FVector2f VertexUVs[] = { Mesh.GetUV(Triangle.A), Mesh.GetUV(Triangle.B), Mesh.GetUV(Triangle.C) };
		return Nanite::GetNumSamples(Barycentrics, VertexUVs, FVector2f(DisplacementMap.GetSizeX(), DisplacementMap.GetSizeY()));
	}

	[[nodiscard]] inline bool AllowRefine(const int32 TriIndex, const int32 Level) const
	{
		if (Level >= MaxRefinementLevel) 
		{
			return false;
		}

		if (!Mesh.AllowEdgeSplit(TriIndex, 0) ||
			!Mesh.AllowEdgeSplit(TriIndex, 1) ||
			!Mesh.AllowEdgeSplit(TriIndex, 2))
		{
			return false;
		}

		return true;
	}

	[[nodiscard]] inline bool ShouldRefine(const int32 TriIndex, FVector3f& SplitVertexBary, const int32 Level) const
	{
		if (!AllowRefine(TriIndex, Level))
		{
			return false;
		}

		float ScaledSampleRate  = SampleRate;
		float ScaledTargetError = TargetError; 
		if (SampleRateWeightLayer)
		{
			const int32 BaseTriangleIndex         = Mesh.GetBaseTriangleIndex(TriIndex);
			const Geometry::FIndex3i BaseTriangle = Mesh.GetBaseTriangle(BaseTriangleIndex);

			float VertexWeights[3];
			SampleRateWeightLayer->GetValue<float*const>(BaseTriangle[0], &VertexWeights[0]);
			SampleRateWeightLayer->GetValue<float*const>(BaseTriangle[1], &VertexWeights[1]);
			SampleRateWeightLayer->GetValue<float*const>(BaseTriangle[2], &VertexWeights[2]);
			
			for (int Idx = 0; Idx < 3; ++Idx)
			{
				VertexWeights[Idx] = 1./ FMath::Pow(2.0, VertexWeights[Idx]);
			}

			const float TriSampleRateWeight = FMath::Clamp(1.f/3.f * (VertexWeights[0] + VertexWeights[1] + VertexWeights[2]), 0.1f, 1000.f);
			ScaledSampleRate  *= TriSampleRateWeight;
			ScaledTargetError *= TriSampleRateWeight * TriSampleRateWeight;  
		}

		FIndex3i Triangle = Mesh.GetTriangle(TriIndex);
		using VecType = FVector3f;

		// check for padding region
		const FVector2f UVs[] = { Mesh.GetUV(Triangle.A), Mesh.GetUV(Triangle.B), Mesh.GetUV(Triangle.C) };
		const FVector2f MinUV(UVs[0].ComponentMin(UVs[1]).ComponentMin(UVs[2]));
		const FVector2f MaxUV(UVs[0].ComponentMax(UVs[1]).ComponentMax(UVs[2]));
		
		if (MaxUV[0] < 0.f || MaxUV[1] < 0.f || MinUV[0] > 1.f || MinUV[1] > 1.f)
		{
			return false;
		}

		const auto& Displacement0 = Displacements[ Triangle.A ];
		const auto& Displacement1 = Displacements[ Triangle.B ];
		const auto& Displacement2 = Displacements[ Triangle.C ];

		FVector3f P0 = Mesh.GetVertexPosition(Triangle.A);
		FVector3f P1 = Mesh.GetVertexPosition(Triangle.B);
		FVector3f P2 = Mesh.GetVertexPosition(Triangle.C);

		const VecType Edge01 = P1 - P0;
		const VecType Edge12 = P2 - P1;
		const VecType Edge20 = P0 - P2;

		const VecType EdgeLengths( Edge01.Length(), Edge12.Length(), Edge20.Length() );

		if( EdgeLengths[0] < MinimumEdgeLength &&
			EdgeLengths[1] < MinimumEdgeLength &&
			EdgeLengths[2] < MinimumEdgeLength )
		{
			return false;
		}
		
		if( EdgeLengths[0] > MaximumEdgeLength ||
			EdgeLengths[1] > MaximumEdgeLength ||
			EdgeLengths[2] > MaximumEdgeLength )
		{
			return true;
		}

		if (FeatureSensitivity > 0.f)
		{
			Geometry::TFeatureSensitivityRefinementCheck<float> RefinementCheck(FeatureSensitivity, ScaledSampleRate, P0 + Displacement0, P1 + Displacement1, P2 + Displacement2);

			if (RefinementCheck.NeedsCheck())
			{
				for (int32 EdgeIdx = 0; EdgeIdx < 3; ++EdgeIdx)
				{
					const int32 AdjHalfEdge = Mesh.GetAdjHalfEdge(TriIndex, EdgeIdx);
					if (AdjHalfEdge != IndexConstants::InvalidID)
					{
						const int32 AdjTriIdx = AdjHalfEdge / 3;
						const int32 AdjEdgeIdx = AdjHalfEdge % 3;

						const FIndex3i AdjTri = Mesh.GetTriangle(AdjTriIdx);
						const int32 VID = AdjTri[(AdjEdgeIdx + 2) % 3];

						const FVector3f P = Mesh.GetVertexPosition(VID) + Displacements[VID];

						if (RefinementCheck.CheckCriterion(EdgeIdx, P))
						{
							return true;
						}
					}
				}
			}
		}

		if( EdgeLengths[0] < ScaledSampleRate &&
			EdgeLengths[1] < ScaledSampleRate &&
			EdgeLengths[2] < ScaledSampleRate )
		{
			return false;
		}

		VecType Barycentrics[3] =
		{
			{ 1.0f, 0.0f, 0.0f },
			{ 0.0f, 1.0f, 0.0f },
			{ 0.0f, 0.0f, 1.0f },
		};

		FVector2f MagnitudeScaleBounds(Magnitude);
		if (HeightScaleLayer)
		{
			const int32 BaseTriangleIndex = Mesh.GetBaseTriangleIndex(TriIndex);
			const Geometry::FIndex3i BaseTriangle = Mesh.GetBaseTriangle(BaseTriangleIndex);

			float VertexWeights[3];
			HeightScaleLayer->GetValue<float*const>(BaseTriangle[0], &VertexWeights[0]);
			HeightScaleLayer->GetValue<float*const>(BaseTriangle[1], &VertexWeights[1]);
			HeightScaleLayer->GetValue<float*const>(BaseTriangle[2], &VertexWeights[2]);
			
			MagnitudeScaleBounds *= FVector2f(FMath::Min3(VertexWeights[0], VertexWeights[1], VertexWeights[2]), 
			                                  FMath::Max3(VertexWeights[0], VertexWeights[1], VertexWeights[2]));
		}

		const FVector2f ErrorBounds = GetErrorBounds(
			Barycentrics,
			Displacement0,
			Displacement1,
			Displacement2,
			TriIndex,
			MagnitudeScaleBounds);

		if (ErrorBounds[1] > ScaledTargetError)
		{
			SplitVertexBary = VecType( 1.f/3.f, 1.f/3.f, 1.f/3.f );
			return true;
		}

		return false;
	}

private:
	const Geometry::FHalfEdgeMesh&  Mesh;
	const Geometry::FDisplacementMap&             DisplacementMap;
	TArray<FVector3f>                   Displacements;
	const float                         Center;
	const float                         Magnitude;
	const float                         SampleRate;
	const float                         TargetError;
	const float                         MinimumEdgeLength;
	const float                         MaximumEdgeLength;
	const int32                         MaxRefinementLevel;
	const int32                         DisplacementBoundsRefinements;
	const float 						FeatureSensitivity;

	const Geometry::TDynamicMeshVertexAttribute<float, 1>* SampleRateWeightLayer { nullptr };
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* HeightScaleLayer      { nullptr };
};

void TessellateAdaptiveParallel(Geometry::FDynamicMesh3& Mesh,
	const FTransform3d& MeshToWorld,
	const FTransform3d& PatchToWorld,
	const FVector2D& UnscaledPatchCoverage,
	const Geometry::FDisplacementMap& DisplacementMap,
	const float Center,
	const float Magnitude,
	const float SampleRate,
	const float MinimumEdgeLength,
	const float MaximumEdgeLength,
	const float FeatureSensitivity,
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* const SampleRateWeightLayer,
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* const HeightScaleLayer)
{
	Geometry::FHalfEdgeMesh HalfEdgeMesh;

	const int32 VerticesBefore = Mesh.VertexCount();

	HalfEdgeMesh.ReserveTriangles(Mesh.TriangleCount());
	HalfEdgeMesh.ReserveVertices(Mesh.VertexCount());

	for (int32 VID : Mesh.VertexIndicesItr())
	{
		const FVector3d PatchLocalVertPosition = PatchToWorld.InverseTransformPosition(MeshToWorld.TransformPosition(Mesh.GetVertex(VID)));

		const FVector2f SamplingUV(PatchLocalVertPosition.X / UnscaledPatchCoverage.X + 0.5, 
			                       PatchLocalVertPosition.Y / UnscaledPatchCoverage.Y + 0.5);

		const auto Frame = Mesh.GetVertexFrame(VID);
		const FVector3f Normal(Frame.Z());

		HalfEdgeMesh.AddVertex(FVector3f(Mesh.GetVertex(VID)), SamplingUV, Normal);
	}

	for (int32 TID : Mesh.TriangleIndicesItr())
	{
		HalfEdgeMesh.AddTriangle(Mesh.GetTriangleRef(TID));
	}

	double BuildTopologyTime = 0.;
	{
		FScopedDurationTimer Timer(BuildTopologyTime);
		HalfEdgeMesh.BuildTopology();
	}

	const int32 Batchsize = CVarParallelTessellationBatchsize.GetValueOnAnyThread();

	FHalfEdgeRefinementPolicy RefinementPolicy(HalfEdgeMesh, DisplacementMap, Center, Magnitude, 
		SampleRate, SampleRate * SampleRate, 
		MinimumEdgeLength, 
		MaximumEdgeLength, 
		FeatureSensitivity, 
		SampleRateWeightLayer,
		HeightScaleLayer);

	double TessellationTime = 0.;
	{
		FScopedDurationTimer Timer(TessellationTime);

		using TessellatorType = Geometry::TParallelAdaptiveRefinement<Geometry::FHalfEdgeMesh, FHalfEdgeRefinementPolicy>;

		typename TessellatorType::FOptions Options;
		TessellatorType Tessellator(HalfEdgeMesh, RefinementPolicy, Options);
	}

	Geometry::FDynamicMesh3 RefinedMesh;
	double ConvertToDynamicMeshTime = 0.;
	{
		FScopedDurationTimer Timer(ConvertToDynamicMeshTime);
		Geometry::ConvertToDynamicMesh3(HalfEdgeMesh, Mesh, RefinedMesh);
	}

	double ConvertToCustomMesh = 0.;
	#if 0
	{
		FScopedDurationTimer Timer(ConvertToCustomMesh);
		MeshPartition::FMeshData CustomMesh;
		Geometry::ConvertToMeshPartitionMesh(HalfEdgeMesh, Mesh, CustomMesh);
	}
	#endif

	UE_LOGF(LogMegaMeshEditor, VeryVerbose, "ParallelRefinement Topology %gs Tessellation %gs Conversion to DynamicMesh %gs CustomMesh %gs", BuildTopologyTime, TessellationTime, ConvertToDynamicMeshTime, ConvertToCustomMesh);

	UE_LOGF(LogMegaMeshEditor, Verbose, "Parallel Adaptive tessellation increased number of vertices from %d to %d (#triangles: %d)",
		VerticesBefore, RefinedMesh.VertexCount(), RefinedMesh.TriangleCount() );


	Mesh.Clear();
	Mesh = MoveTemp(RefinedMesh);
}

namespace MegaMeshAdaptiveDisplacementLocals
{

int32 CountBoundaryEdges(const Geometry::FDynamicMesh3& Mesh)
{
	int32 Count(0);
	for (const int32 BoundaryEdge: Mesh.BoundaryEdgeIndicesItr())
	{
		++Count;
	}
	return Count;
}

} // namespace MegaMeshAdaptiveDisplacementLocals

// Adaptively tessellate the mesh according to specified displacement map.
// Sampling coordinates will be determine from transforming from mesh to patch local space and UnscaledPatchCoverage
// Preserves per-vector UVs
// Will not actually displace
void TessellateAdaptive(Geometry::FDynamicMesh3& Mesh,
	const FTransform3d& MeshToWorld,
	const FTransform3d& PatchToWorld,
	const FVector2D& UnscaledPatchCoverage,
	const Geometry::FDisplacementMap& DisplacementMap,
	const float Center,
	const float Magnitude,
	bool bParallel,
	const float SampleRate,
	const float MinimumEdgeLength,
	const float MaximumEdgeLength,
	const float FeatureSensitivity,
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* const SampleRateWeightLayer,
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* const HeightScaleLayer)
{
	const int32 OrigBoundaryEdges = MegaMeshAdaptiveDisplacementLocals::CountBoundaryEdges(Mesh);

	double Elapsed = 0;
	{
		FScopedDurationTimer Timer(Elapsed);

		if (bParallel)
		{
			TessellateAdaptiveParallel(Mesh, MeshToWorld, PatchToWorld, UnscaledPatchCoverage, DisplacementMap, Center, Magnitude, SampleRate, 
				MinimumEdgeLength, MaximumEdgeLength, FeatureSensitivity, SampleRateWeightLayer, HeightScaleLayer);
		}
		else
		{
			Geometry::FMeshConstraints MeshConstraints;
	
			// don't allow boundary to be split
			const Geometry::EEdgeRefineFlags MeshBoundaryConstraint = Geometry::EEdgeRefineFlags::FullyConstrained;
			const Geometry::EEdgeRefineFlags GroupBoundaryConstraint = Geometry::EEdgeRefineFlags::NoFlip;
			const Geometry::EEdgeRefineFlags MaterialBoundaryConstraint = Geometry::EEdgeRefineFlags::NoFlip;

			Geometry::FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(MeshConstraints, Mesh,
				MeshBoundaryConstraint, GroupBoundaryConstraint, MaterialBoundaryConstraint, true /* seam splits */, false /* seam smooth, not needed */ );

			for ( auto EdgeID : Mesh.EdgeIndicesItr())
			{
				const Geometry::FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EdgeID);
				if (Edge.Tri[1] == IndexConstants::InvalidID)
				{
					check(!MeshConstraints.GetEdgeConstraint(EdgeID).CanSplit());
				}
			}

			TessellateAdaptiveNative(Mesh, MeshConstraints, MeshToWorld, PatchToWorld, UnscaledPatchCoverage, 
				DisplacementMap, Center, Magnitude, SampleRate, MinimumEdgeLength, MaximumEdgeLength, FeatureSensitivity, SampleRateWeightLayer, HeightScaleLayer);
		}
	}

	const int32 NewBoundaryEdges = MegaMeshAdaptiveDisplacementLocals::CountBoundaryEdges(Mesh);

	if (NewBoundaryEdges != OrigBoundaryEdges)
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Boundary should stay unchanged, but number boundary edges has changed from %d to %d.", OrigBoundaryEdges, NewBoundaryEdges);
	}
		
	UE_LOGF(LogMegaMeshEditor, Verbose, "Adaptive tessellation time: %gs", Elapsed);
}

} // namespace UE::MeshPartition
