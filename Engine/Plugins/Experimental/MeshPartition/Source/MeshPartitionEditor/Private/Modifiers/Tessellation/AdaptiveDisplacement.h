// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "Math/MathFwd.h"
#include "DynamicMesh/DynamicVertexAttribute.h"

namespace UE {

namespace Geometry 
{ 
class FDynamicMesh3; 
class FDisplacementMap; 
}

namespace MeshPartition {

// Adaptively tessellate the mesh according to specified displacement map, not applying the displacement.
// Sampling coordinates will be determined from transformed points from mesh to patch space, applying UnscaledPatchCoverage to [-0.5, 0.5]^2
// range shifted to [0,1]^2.
//
void TessellateAdaptive(
	Geometry::FDynamicMesh3& Mesh,
	const FTransform3d& MeshToWorld,
	const FTransform3d& PatchToWorld,
	const FVector2D& UnscaledPatchCoverage,
	const UE::Geometry::FDisplacementMap& DisplacementMap,
	const float Center, //< texture value corresponding to zero offset
	const float Magnitude, //< texture value scaling 
	const bool bParallel, //< parallel adaptive refinement
	const float SampleRate, //< one-dimensional length of smallest features to sample
	const float MinimumEdgeLength, //< edges smaller than this value won't be refined
	const float MaximumEdgeLength, //< edges larger than this value will always be refined
	const float FeatureSensitivity, //< curvature-sensitivity in [0,1] 
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* const SampleRateWeightLayer, //< adjust sample rate by 1/2^x, where x comes from this layer
	const Geometry::TDynamicMeshVertexAttribute<float, 1>* const HeightScaleLayer); //< vertex dependent texture value magnitude scale

} // namespace MeshPartition
} // namespace UE