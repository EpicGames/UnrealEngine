// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Tessellation/HalfEdgeMesh.h"
#include "MeshPartitionMeshData.h"

#include "DynamicMesh/DynamicMeshBulkEdit.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Containers/Array.h"

namespace UE {
namespace Geometry {

	
static TAutoConsoleVariable<int32> CVarTessellationBulkInitialize(
	TEXT("MegaMesh.AdaptiveTessellation.BulkInitialize"),
	1,
	TEXT("0: disabled, 1: enabled, 2: enabled but skip edgelist build")
);

void FHalfEdgeMesh::BuildTopology()
{
	Triangles = BaseTriangles;

	const int32 TriangleCount = BaseTriangles.Num();

	BaseTriangleIndices.SetNum(TriangleCount);
	HalfEdges.SetNum(TriangleCount * 3);
	
	for (int32 T(0); T < TriangleCount; ++T)
	{
		BaseTriangleIndices[T] = T;
	}

	struct FSortEdge
	{
		int32 HalfEdgeIndex;
		int32 VtxA, VtxB;

		FORCEINLINE bool operator<(const FSortEdge& other) const
		{
			return VtxA < other.VtxA || (VtxA == other.VtxA && VtxB < other.VtxB);
		}
	};

	TArray<FSortEdge> SortedHalfEdges;
	SortedHalfEdges.Reserve(HalfEdges.Num());

	const int32 NumHalfEdges = HalfEdgeCount();
	for (int32 i = 0; i < NumHalfEdges; ++i)
	{
		const int32 V0 = Triangles[i/3][i%3];
		const int32 V1 = Triangles[i/3][Cycle3(i%3)];
		SortedHalfEdges.Emplace( i, V0 < V1 ? V0 : V1, V0 < V1 ? V1 : V0 );
	}
	SortedHalfEdges.Sort();

	// linear walk to count number of edges
	int32 NumEdges = 0;
	for (int32 i(0); i < NumHalfEdges; ++i)
	{
		NumEdges++;
		if (i<NumHalfEdges-1 && 
			SortedHalfEdges[i].VtxA == SortedHalfEdges[i+1].VtxA && 
			SortedHalfEdges[i].VtxB == SortedHalfEdges[i+1].VtxB)
		{
			++i;
		}
	}
	Edges.SetNum(NumEdges);

	// linear walk to assign edges and halfedges
	int32 EdgeIdx = 0;
	for (int32 i=0; i<NumHalfEdges; ++i)
	{
		if (i<NumHalfEdges-1 && 
			SortedHalfEdges[i].VtxA == SortedHalfEdges[i+1].VtxA && 
			SortedHalfEdges[i].VtxB == SortedHalfEdges[i+1].VtxB)
		{
			const int32 HalfEdgeIndex0 = SortedHalfEdges[i  ].HalfEdgeIndex;
			const int32 HalfEdgeIndex1 = SortedHalfEdges[i+1].HalfEdgeIndex;

			FHalfEdge& HalfEdge0 = HalfEdges[HalfEdgeIndex0];
			FHalfEdge& HalfEdge1 = HalfEdges[HalfEdgeIndex1];

			HalfEdge0.AdjHalfEdge = HalfEdgeIndex1;
			HalfEdge1.AdjHalfEdge = HalfEdgeIndex0;

			HalfEdge0.Edge = EdgeIdx;
			HalfEdge1.Edge = EdgeIdx;

			Edges[EdgeIdx].HalfEdge[0] = HalfEdgeIndex0;
			Edges[EdgeIdx].HalfEdge[1] = HalfEdgeIndex1;

			++i;
		}
		else
		{
			FHalfEdge& HalfEdge0 = HalfEdges[SortedHalfEdges[i].HalfEdgeIndex];

			HalfEdge0.AdjHalfEdge = IndexConstants::InvalidID;
			HalfEdge0.Edge = EdgeIdx;

			Edges[EdgeIdx].HalfEdge[0] = SortedHalfEdges[i].HalfEdgeIndex;
			Edges[EdgeIdx].HalfEdge[1] = IndexConstants::InvalidID;
		}

		Edges[EdgeIdx].BaseEdge = EdgeIdx;
		EdgeIdx++;
	}

	BaseEdges = Edges;
}

void ConvertToDynamicMesh3(const FHalfEdgeMesh& HalfEdgeMesh, const FDynamicMesh3& InputMesh, FDynamicMesh3& RefinedMesh)
{
	TArray<int32>     MeshVtxIdCompact;  //< map original vertices indices to compact/dense indices
	TArray<int32>     MeshSourceVtxId;   //< map compact/dense to original vertices indices

	TArray<int32>     MeshTriIdCompact;  //< map original triangle indices to compact/dense indices
	TArray<int32>     MeshSourceTriId;   //< map compact/dense to original triangle indices

	MeshVtxIdCompact.Init(IndexConstants::InvalidID, InputMesh.MaxVertexID());
	MeshTriIdCompact.Init(IndexConstants::InvalidID, InputMesh.MaxTriangleID());

	const int32 VertexCount = InputMesh.VertexCount();

	MeshSourceVtxId.Reserve(VertexCount);
	int32 CompactVID(0);
	for (int32 VID : InputMesh.VertexIndicesItr())
	{
		MeshSourceVtxId.Add(VID);
		MeshVtxIdCompact[VID] = CompactVID++;
	}
	check(CompactVID == InputMesh.VertexCount());
	
	MeshSourceTriId.Reserve(InputMesh.TriangleCount());
	int32 CompactTID(0);
	for (int32 TID : InputMesh.TriangleIndicesItr())
	{
		MeshSourceTriId.Add(TID);
		MeshTriIdCompact[TID] = CompactTID++;
	}
	check(CompactTID == InputMesh.TriangleCount());

	const int32 BulkMode = CVarTessellationBulkInitialize.GetValueOnAnyThread();

	if (BulkMode > 0)
	{
		FDynamicMeshBulkEdit BulkEdit(RefinedMesh, BulkMode != 2);
		BulkEdit.Initialize(HalfEdgeMesh.MaxVertexID(), HalfEdgeMesh.MaxTriID(), HalfEdgeMesh.EdgeCount());
		
		for (int32 VID(0); VID < HalfEdgeMesh.MaxVertexID(); ++VID)
		{
			BulkEdit.SetVertex(VID, FVector3d(HalfEdgeMesh.GetVertexPosition(VID)));
		}

		for (int32 TID(0); TID < HalfEdgeMesh.MaxTriID(); ++TID)
		{
			BulkEdit.SetTriVertexIndices(TID, HalfEdgeMesh.GetTriangle(TID));

			FIndex3i LocTriEdges;
			for (int32 Idx=0; Idx<3; ++Idx) {
				LocTriEdges[Idx] = HalfEdgeMesh.GetEdgeID(TID, Idx);
			}

			BulkEdit.SetTriEdgeIndices(TID, LocTriEdges);
		}

		for (int32 EID(0); EID < HalfEdgeMesh.EdgeCount(); ++EID)
		{
			FDynamicMesh3::FEdge Edge;
			FIndex2i EdgeVertexIndices;

			HalfEdgeMesh.GetEdge(EID, EdgeVertexIndices, Edge.Tri);

			if (EdgeVertexIndices[0] < EdgeVertexIndices[1])
			{
				Edge.Vert = EdgeVertexIndices;
			}
			else
			{
				Edge.Vert = FIndex2i(EdgeVertexIndices[1], EdgeVertexIndices[0]);
			}

			BulkEdit.SetEdge(EID, Edge);
		}
		BulkEdit.Finalize();
	}
	else
	{
		for (int32 VID(0); VID < HalfEdgeMesh.MaxVertexID(); ++VID)
		{
			// RefinedMesh.AppendVertex(FVector3d(HalfEdgeMesh.GetVertexPosition(VID)));

			// this is equivalent to the above
			// keep it here as validation that barycentric coordinates are correct
			const FHalfEdgeMesh::FVertex Vertex = HalfEdgeMesh.GetVertex(VID);

			FVector3d WeightedVertex(0.);

			if (VID < VertexCount)
			{
				WeightedVertex = InputMesh.GetVertex(MeshSourceVtxId[VID]);
			}
			else
			{
				const FIndex3i SourceTri = InputMesh.GetTriangle(MeshSourceTriId[Vertex.BaseHalfEdge / 3]);

				WeightedVertex += InputMesh.GetVertex(MeshSourceVtxId[SourceTri[0]]) * Vertex.Barycentric[0];
				WeightedVertex += InputMesh.GetVertex(MeshSourceVtxId[SourceTri[1]]) * Vertex.Barycentric[1];
				WeightedVertex += InputMesh.GetVertex(MeshSourceVtxId[SourceTri[2]]) * Vertex.Barycentric[2];
			}

			RefinedMesh.AppendVertex(FVector3d(WeightedVertex));
		}
		for (int32 TID(0); TID < HalfEdgeMesh.MaxTriID(); ++TID)
		{
			RefinedMesh.AppendTriangle(HalfEdgeMesh.GetTriangle(TID));
		}
	}

	if (!InputMesh.HasAttributes())
	{
		return;
	}

	const int NumUVLayers     = InputMesh.Attributes()->NumUVLayers();
	const int NumWeightLayers = InputMesh.Attributes()->NumWeightLayers();

	// creates one UV and one Normal layer by default
	RefinedMesh.EnableAttributes();

	// transfer all UV layers
	RefinedMesh.Attributes()->SetNumUVLayers(NumUVLayers);
	for (int32 UVLayer(0); UVLayer < NumUVLayers; ++UVLayer)
	{
		const FDynamicMeshUVOverlay* SourceUVOverlay = InputMesh.Attributes()->GetUVLayer(UVLayer);
		if (SourceUVOverlay->ElementCount() == 0)
		{
			continue;
		}

		FDynamicMeshUVOverlay* TargetUVOverlay = RefinedMesh.Attributes()->GetUVLayer(UVLayer);

		const bool bSourceOverlayHasSeamEdges = SourceUVOverlay->HasInteriorSeamEdges();
		
		TArray<int32> UVVtxIdCompact;  //< map original vertices indices to compact indices
		TArray<int32> UVSourceVtxId;   //< map original vertices indices to compact indices

		UVVtxIdCompact.Init(IndexConstants::InvalidID, SourceUVOverlay->MaxElementID());
		UVSourceVtxId.Reserve(SourceUVOverlay->ElementCount());

		int32 CompactEID(0);
	
		// first we inherit all the elements at the root vertices
		for (int32 EID : SourceUVOverlay->ElementIndicesItr())
		{
			UVSourceVtxId.Add(EID);	
			UVVtxIdCompact[EID] = CompactEID++;
			FVector2f UV;
			SourceUVOverlay->GetElement(EID, UV);
			TargetUVOverlay->AppendElement(UV);
		}

		TargetUVOverlay->InitializeTriangles(HalfEdgeMesh.MaxTriID());

		TArray<FIndex2i> VertexUVIndices;
		VertexUVIndices.Init(FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID), HalfEdgeMesh.MaxVertexID());

		// duplicate topology
		for (int32 TID(0); TID < HalfEdgeMesh.MaxTriID(); ++TID)
		{
			const FIndex3i Triangle = HalfEdgeMesh.GetTriangle(TID);
			const int32 BaseTriangleIndex = HalfEdgeMesh.GetBaseTriangleIndex(TID);

			check(BaseTriangleIndex >= 0 && BaseTriangleIndex < InputMesh.TriangleCount());

			const FIndex3i SourceTri = InputMesh.GetTriangle(MeshSourceTriId[BaseTriangleIndex]);

			FIndex3i TargetUVTri(IndexConstants::InvalidID, IndexConstants::InvalidID, IndexConstants::InvalidID);
			
			if (!SourceUVOverlay->IsSetTriangle(MeshSourceTriId[BaseTriangleIndex]))
			{
				// no source triangle -> do nothing.
				continue;
			}

			const FIndex3i SourceUVTri = SourceUVOverlay->GetTriangle(MeshSourceTriId[BaseTriangleIndex]);

			for (int32 j=0; j<3; ++j)
			{
				const int32 VID = Triangle[j];
				
				if (VID < VertexCount)
				{
					// this maps to the original mesh. find the corresponding corner in the original topology
					// and look up the attribute index at that corner
					const int32 OrigVID = MeshSourceVtxId[VID];

					bool Found = false;
					for (int32 k=0; k<3; ++k)
					{
						if (SourceTri[k] == OrigVID)
						{
							Found = true;
							TargetUVTri[j] = UVVtxIdCompact[SourceUVTri[k]];
							break;
						}
					}
					check(Found);
					continue;
				}

				const FHalfEdgeMesh::FVertex& Vertex = HalfEdgeMesh.GetVertex(VID);

				bool bIsSeamEdgeVertex = false;
				if (bSourceOverlayHasSeamEdges && Vertex.BaseAdjHalfEdge != IndexConstants::InvalidID)
				{
					// get the two triangles in UV space
					const FIndex3i Tri0 = SourceUVOverlay->GetTriangle(MeshSourceTriId[Vertex.BaseHalfEdge/3]);
					const FIndex3i Tri1 = SourceUVOverlay->GetTriangle(MeshSourceTriId[Vertex.BaseAdjHalfEdge/3]);
					
					if (Tri0[(Vertex.BaseHalfEdge+0)%3] != Tri1[(Vertex.BaseAdjHalfEdge+1)%3] ||
						Tri0[(Vertex.BaseHalfEdge+1)%3] != Tri1[(Vertex.BaseAdjHalfEdge+0)%3])
					{
						bIsSeamEdgeVertex = true;
					}
				}

				int32 Side = 0;
				int32 HalfEdge = 0;

				if (bIsSeamEdgeVertex)
				{
				
					if (BaseTriangleIndex == Vertex.BaseHalfEdge / 3)
					{
						Side = 0;
						HalfEdge = Vertex.BaseHalfEdge;
					}
					else
					{
						check(BaseTriangleIndex == Vertex.BaseAdjHalfEdge / 3);
						Side = 1;
						HalfEdge = Vertex.BaseAdjHalfEdge;
					}
				}
				else
				{
					// on edge but not a seam vertex, or interior vertex of triangle. in either case,
					// just apply plain barycentric interpolation
					Side = 0;
					HalfEdge = Vertex.BaseHalfEdge;
				}

				if (VertexUVIndices[VID][Side] == IndexConstants::InvalidID)
				{
					const FIndex3i BaseTri = SourceUVOverlay->GetTriangle(MeshSourceTriId[HalfEdge / 3]);

					FVector2f Weighted(0.f);

					FVector2f UV;
					SourceUVOverlay->GetElement(BaseTri[0], UV);
					Weighted += UV * Vertex.Barycentric[0];

					SourceUVOverlay->GetElement(BaseTri[1], UV);
					Weighted += UV * Vertex.Barycentric[1];

					SourceUVOverlay->GetElement(BaseTri[2], UV);
					Weighted += UV * Vertex.Barycentric[2];

					VertexUVIndices[VID][Side] = TargetUVOverlay->AppendElement(Weighted);
				}

				TargetUVTri[j] = VertexUVIndices[VID][Side];
			}
						
			TargetUVOverlay->SetTriangle(TID, TargetUVTri);
		}
	}

	// Transfer all weight layers
	RefinedMesh.Attributes()->SetNumWeightLayers(NumWeightLayers);
	for (int32 WeightLayerIdx(0); WeightLayerIdx < NumWeightLayers; ++WeightLayerIdx)
	{
		const Geometry::FDynamicMeshWeightAttribute* SourceWeightLayer = InputMesh.Attributes()->GetWeightLayer(WeightLayerIdx);
		Geometry::FDynamicMeshWeightAttribute* TargetWeightLayer = RefinedMesh.Attributes()->GetWeightLayer(WeightLayerIdx);

		TargetWeightLayer->SetName(SourceWeightLayer->GetName());
		TargetWeightLayer->Initialize();

		for (int32 VID(0); VID < HalfEdgeMesh.MaxVertexID(); ++VID)
		{
			float WeightedValue = 0.f;
			if (VID < VertexCount)
			{
				SourceWeightLayer->GetValue<float* const>(MeshSourceVtxId[VID], &WeightedValue);
			}
			else
			{
				const FHalfEdgeMesh::FVertex& Vertex = HalfEdgeMesh.GetVertex(VID);
				const FIndex3i BaseTri = InputMesh.GetTriangle(MeshSourceTriId[Vertex.BaseHalfEdge / 3]);

				float VertexValue;
				SourceWeightLayer->GetValue<float* const>(BaseTri[0], &VertexValue);
				WeightedValue += Vertex.Barycentric[0] * VertexValue;

				SourceWeightLayer->GetValue<float* const>(BaseTri[1], &VertexValue);
				WeightedValue += Vertex.Barycentric[1] * VertexValue;

				SourceWeightLayer->GetValue<float* const>(BaseTri[2], &VertexValue);
				WeightedValue += Vertex.Barycentric[2] * VertexValue;
			}
			TargetWeightLayer->SetScalarValue(VID, WeightedValue);
		}
	}

	// Transfer material ids
	if (InputMesh.Attributes()->HasMaterialID())
	{
		RefinedMesh.Attributes()->EnableMaterialID();

		const FDynamicMeshMaterialAttribute* SourceMaterialID = InputMesh.Attributes()->GetMaterialID();
		FDynamicMeshMaterialAttribute* TargetMaterialID = RefinedMesh.Attributes()->GetMaterialID();

		for (int32 TID(0); TID < HalfEdgeMesh.MaxTriID(); ++TID)
		{
			const int32 BaseTriangleIndex = HalfEdgeMesh.GetBaseTriangleIndex(TID);
			if (ensure(BaseTriangleIndex >= 0 && BaseTriangleIndex < InputMesh.TriangleCount()))
			{
				const int32 InputTriangleIndex = MeshSourceTriId[BaseTriangleIndex];
				TargetMaterialID->SetNewValue(TID, SourceMaterialID->GetValue(InputTriangleIndex));
			}
		}
	}

	const int32 NumPolygroups = InputMesh.Attributes()->NumPolygroupLayers();
	RefinedMesh.Attributes()->SetNumPolygroupLayers(NumPolygroups);

	for (int32 PolygroupIdx(0); PolygroupIdx < NumPolygroups; ++PolygroupIdx)
	{		
		const FDynamicMeshPolygroupAttribute* SourcePolygroupLayer = InputMesh.Attributes()->GetPolygroupLayer(PolygroupIdx);
		FDynamicMeshPolygroupAttribute* TargetPolygroupLayer = RefinedMesh.Attributes()->GetPolygroupLayer(PolygroupIdx);
		TargetPolygroupLayer->SetName(SourcePolygroupLayer->GetName());

		for (int32 TID(0); TID < HalfEdgeMesh.MaxTriID(); ++TID)
		{
			const int32 BaseTriangleIndex = HalfEdgeMesh.GetBaseTriangleIndex(TID);
			if (ensure(BaseTriangleIndex >= 0 && BaseTriangleIndex < InputMesh.TriangleCount()))
			{
				const int32 InputTriangleIndex = MeshSourceTriId[BaseTriangleIndex];
				TargetPolygroupLayer->SetNewValue(TID, SourcePolygroupLayer->GetValue(InputTriangleIndex));
			}
		}
	}
}

void ConvertToMeshPartitionMesh(const FHalfEdgeMesh& HalfEdgeMesh, const FDynamicMesh3& InputMesh, UE::MeshPartition::FMeshData& OutputMesh)
{
	TArray<int32>     MeshVtxIdCompact;  //< map original vertices indices to compact/dense indices
	TArray<int32>     MeshSourceVtxId;   //< map compact/dense to original vertices indices

	TArray<int32>     MeshTriIdCompact;  //< map original triangle indices to compact/dense indices
	TArray<int32>     MeshSourceTriId;   //< map compact/dense to original triangle indices

	MeshVtxIdCompact.Init(IndexConstants::InvalidID, InputMesh.MaxVertexID());
	MeshTriIdCompact.Init(IndexConstants::InvalidID, InputMesh.MaxTriangleID());

	const int32 VertexCount = InputMesh.VertexCount();

	MeshSourceVtxId.Reserve(VertexCount);
	int32 CompactVID(0);
	for (int32 VID : InputMesh.VertexIndicesItr())
	{
		MeshSourceVtxId.Add(VID);
		MeshVtxIdCompact[VID] = CompactVID++;
	}
	check(CompactVID == InputMesh.VertexCount());
	
	MeshSourceTriId.Reserve(InputMesh.TriangleCount());
	int32 CompactTID(0);
	for (int32 TID : InputMesh.TriangleIndicesItr())
	{
		MeshSourceTriId.Add(TID);
		MeshTriIdCompact[TID] = CompactTID++;
	}
	check(CompactTID == InputMesh.TriangleCount());

	OutputMesh.Clear();
	OutputMesh.ReserveAdditionalVertices(InputMesh.VertexCount());
	OutputMesh.ReserveAdditionalTriangles(InputMesh.TriangleCount());
	
	for (int32 VID(0); VID < HalfEdgeMesh.MaxVertexID(); ++VID)
	{
		OutputMesh.AppendVertex(FVector3d(HalfEdgeMesh.GetVertexPosition(VID)));
	}
	for (int32 TID(0); TID < HalfEdgeMesh.MaxTriID(); ++TID)
	{
		OutputMesh.AppendTriangle(HalfEdgeMesh.GetTriangle(TID));
	}

	if (!InputMesh.HasAttributes())
	{
		return;
	}

	const int NumUVLayers     = InputMesh.Attributes()->NumUVLayers();
	const int NumWeightLayers = InputMesh.Attributes()->NumWeightLayers();


	// transfer all UV layers
	for (int32 UVLayer(0); UVLayer < NumUVLayers && UVLayer < 1; ++UVLayer)
	{
		const FDynamicMeshUVOverlay* SourceUVOverlay = InputMesh.Attributes()->GetUVLayer(UVLayer);
		if (SourceUVOverlay->ElementCount() == 0)
		{
			continue;
		}

		const bool bSourceOverlayHasSeamEdges = SourceUVOverlay->HasInteriorSeamEdges();
		
		TArray<int32> UVVtxIdCompact;  //< map original vertices indices to compact indices
		TArray<int32> UVSourceVtxId;   //< map original vertices indices to compact indices

		UVVtxIdCompact.Init(IndexConstants::InvalidID, SourceUVOverlay->MaxElementID());
		UVSourceVtxId.Reserve(SourceUVOverlay->ElementCount());

		int32 CompactEID(0);
		int32 DestVID = 0;

		// first we inherit all the elements at the root vertices
		for (int32 EID : SourceUVOverlay->ElementIndicesItr())
		{
			UVSourceVtxId.Add(EID);	
			UVVtxIdCompact[EID] = CompactEID++;
			FVector2f UV;
			SourceUVOverlay->GetElement(EID, UV);

			OutputMesh.SetVertexUV(DestVID++, UV, UVLayer);
		}

		TArray<FIndex2i> VertexUVIndices;
		VertexUVIndices.Init(FIndex2i(IndexConstants::InvalidID, IndexConstants::InvalidID), HalfEdgeMesh.MaxVertexID());

		// duplicate topology
		for (int32 TID(0); TID < HalfEdgeMesh.MaxTriID(); ++TID)
		{
			const FIndex3i Triangle = HalfEdgeMesh.GetTriangle(TID);
			const int32 BaseTriangleIndex = HalfEdgeMesh.GetBaseTriangleIndex(TID);

			check(BaseTriangleIndex >= 0 && BaseTriangleIndex < InputMesh.TriangleCount());

			const FIndex3i SourceTri = InputMesh.GetTriangle(MeshSourceTriId[BaseTriangleIndex]);

			FIndex3i TargetUVTri(IndexConstants::InvalidID, IndexConstants::InvalidID, IndexConstants::InvalidID);
			
			if (!SourceUVOverlay->IsSetTriangle(MeshSourceTriId[BaseTriangleIndex]))
			{
				continue;
			}

			const FIndex3i SourceUVTri = SourceUVOverlay->GetTriangle(MeshSourceTriId[BaseTriangleIndex]);

			for (int32 j=0; j<3; ++j)
			{
				const int32 VID = Triangle[j];
				
				if (VID < VertexCount)
				{
					// this maps to the original mesh. find the corresponding corner in the original topology
					// and look up the attribute index at that corner
					const int32 OrigVID = MeshSourceVtxId[VID];

					bool Found = false;
					for (int32 k=0; k<3; ++k)
					{
						if (SourceTri[k] == OrigVID)
						{
							Found = true;
							TargetUVTri[j] = UVVtxIdCompact[SourceUVTri[k]];
							break;
						}
					}
					check(Found);
					continue;
				}

				const FHalfEdgeMesh::FVertex& Vertex = HalfEdgeMesh.GetVertex(VID);

				bool bIsSeamEdgeVertex = false;
				if (bSourceOverlayHasSeamEdges && Vertex.BaseAdjHalfEdge != IndexConstants::InvalidID)
				{
					// get the two triangles in UV space
					const FIndex3i Tri0 = SourceUVOverlay->GetTriangle(MeshSourceTriId[Vertex.BaseHalfEdge/3]);
					const FIndex3i Tri1 = SourceUVOverlay->GetTriangle(MeshSourceTriId[Vertex.BaseAdjHalfEdge/3]);
					
					if (Tri0[(Vertex.BaseHalfEdge+0)%3] != Tri1[(Vertex.BaseAdjHalfEdge+1)%3] ||
						Tri0[(Vertex.BaseHalfEdge+1)%3] != Tri1[(Vertex.BaseAdjHalfEdge+0)%3])
					{
						bIsSeamEdgeVertex = true;
					}
				}

				int32 Side = 0;
				int32 HalfEdge = 0;

				if (bIsSeamEdgeVertex)
				{
				
					if (BaseTriangleIndex == Vertex.BaseHalfEdge / 3)
					{
						Side = 0;
						HalfEdge = Vertex.BaseHalfEdge;
					}
					else
					{
						check(BaseTriangleIndex == Vertex.BaseAdjHalfEdge / 3);
						Side = 1;
						HalfEdge = Vertex.BaseAdjHalfEdge;
					}
				}
				else
				{
					// on edge but not a seam vertex, or interior vertex of triangle. in either case,
					// just apply plain barycentric interpolation
					Side = 0;
					HalfEdge = Vertex.BaseHalfEdge;
				}

				if (VertexUVIndices[VID][Side] == IndexConstants::InvalidID)
				{
					const FIndex3i BaseTri = SourceUVOverlay->GetTriangle(MeshSourceTriId[HalfEdge / 3]);

					FVector2f Weighted(0.f);

					FVector2f UV;
					SourceUVOverlay->GetElement(BaseTri[0], UV);
					Weighted += UV * Vertex.Barycentric[0];

					SourceUVOverlay->GetElement(BaseTri[1], UV);
					Weighted += UV * Vertex.Barycentric[1];

					SourceUVOverlay->GetElement(BaseTri[2], UV);
					Weighted += UV * Vertex.Barycentric[2];

					OutputMesh.SetVertexUV(DestVID, Weighted, UVLayer);
					VertexUVIndices[VID][Side] = DestVID++;
				}
				TargetUVTri[j] = VertexUVIndices[VID][Side];
			}
		}
	}

	// Transfer all weight layers
	for (int32 WeightLayerIdx(0); WeightLayerIdx < NumWeightLayers; ++WeightLayerIdx)
	{
		const Geometry::FDynamicMeshWeightAttribute* SourceWeightLayer = InputMesh.Attributes()->GetWeightLayer(WeightLayerIdx);
	
		OutputMesh.InitializeWeightLayer(SourceWeightLayer->GetName());
		TArray<int32> Indices;
		TArray<float> Weights;

		Indices.Reserve(HalfEdgeMesh.MaxVertexID());
		Weights.Reserve(HalfEdgeMesh.MaxVertexID());

		for (int32 VID(0); VID < HalfEdgeMesh.MaxVertexID(); ++VID)
		{
			float WeightedValue = 0.f;
			if (VID < VertexCount)
			{
				SourceWeightLayer->GetValue<float* const>(MeshSourceVtxId[VID], &WeightedValue);
			}
			else
			{
				const FHalfEdgeMesh::FVertex& Vertex = HalfEdgeMesh.GetVertex(VID);
				const FIndex3i BaseTri = InputMesh.GetTriangle(MeshSourceTriId[Vertex.BaseHalfEdge / 3]);

				float VertexValue;
				SourceWeightLayer->GetValue<float* const>(BaseTri[0], &VertexValue);
				WeightedValue += Vertex.Barycentric[0] * VertexValue;

				SourceWeightLayer->GetValue<float* const>(BaseTri[1], &VertexValue);
				WeightedValue += Vertex.Barycentric[1] * VertexValue;

				SourceWeightLayer->GetValue<float* const>(BaseTri[2], &VertexValue);
				WeightedValue += Vertex.Barycentric[2] * VertexValue;
			}
			// OutputMesh.TargetWeightLayer->SetScalarValue(VID, WeightedValue);
			
			Weights.Add(WeightedValue);
			Indices.Add(VID);
		}
		OutputMesh.SetWeightLayerValues(SourceWeightLayer->GetName(), Indices, Weights);
	}
}

} // namespace Geometry
} // namespace UE