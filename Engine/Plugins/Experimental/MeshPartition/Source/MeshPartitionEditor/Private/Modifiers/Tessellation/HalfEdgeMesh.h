// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/AssertionMacros.h"
#include "IndexTypes.h"
#include "VectorUtil.h"
#include "Templates/UnrealTemplate.h"

#include <atomic>

namespace UE::MeshPartition
{
	class FMeshData;
}

namespace UE {
namespace Geometry {

class FDynamicMesh3;

// Simplistic mesh class with half-edges and edges that provides the interface needed by TParallelAdaptiveRefinement
class FHalfEdgeMesh
{
	FORCEINLINE static uint32 Cycle3(uint32 Value)
	{
		uint32 ValueMod3 = Value % 3;
		uint32 Value1Mod3 = (1 << ValueMod3) & 3;
		return Value - ValueMod3 + Value1Mod3;
	}

  public:
	using FIndex3i = UE::Geometry::FIndex3i;

	using RealType = float;
	using VecType  = FVector3f;

	struct FHalfEdge
	{
		int32 AdjHalfEdge; //< neighboring half-edge, winding opposite direction
		int32 Edge;        //< shared edge index
	};

	struct FEdge
	{
		FIndex2i HalfEdge; //< opposing halfedges. on boundary second halfedge is IndexConstants::InvalidID
		int32    BaseEdge { IndexConstants::InvalidID };
	};

	struct FVertex
	{
		FVector3f Position;
		FVector2f UV;           // sampling coords
		FVector3f Normal;       // displacement direction
		FVector3f Barycentric;  // barycentric coordinates with respect to base triangle

		// for edges, tracks the base halfedge.
		// for interior vertices, only used to track the base triangle, with respect to which barycentric coordinates are valid
		int32     BaseHalfEdge    { IndexConstants::InvalidID };

		// neighbor halfedge of the base mesh. InvalidID for interior vertices or boundary vertices
		int32     BaseAdjHalfEdge { IndexConstants::InvalidID };

		// swaps the two halfedges and the meaning the barycentric coordinates respectively
		void Flip()
		{
			check(BaseAdjHalfEdge != IndexConstants::InvalidID);

			const FVector3f OldBary = Barycentric;
			Barycentric[(BaseAdjHalfEdge+0)%3] = OldBary[(BaseHalfEdge+1)%3];
			Barycentric[(BaseAdjHalfEdge+1)%3] = OldBary[(BaseHalfEdge+0)%3];
			Barycentric[(BaseAdjHalfEdge+2)%3] = 0.f;

			ensure(OldBary[(BaseHalfEdge+2)%3] == 0.f);
					
			Swap(BaseHalfEdge, BaseAdjHalfEdge);
		}
	};

	// should be called after all triangles have been added
	void BuildTopology();

	inline void AddVertex(const FVector3f& Position, const FVector2f& UV, const FVector3f& Normal)
	{
		Vertices.Emplace( Position, UV, Normal, FVector3f(1.f, 0.f, 0.f), IndexConstants::InvalidID, IndexConstants::InvalidID );
	}

	inline void AddTriangle(const FIndex3i& Triangle)
	{
		BaseTriangles.Add(Triangle);
	}

	void ReserveTriangles(const int32 TriangleCount)
	{
		BaseTriangles.Reserve(TriangleCount);
		Triangles.Reserve(TriangleCount);
		HalfEdges.Reserve(3*TriangleCount);
	}

	void ReserveVertices(const int32 VertexCount)
	{
		Vertices.Reserve(VertexCount);
		Edges.Reserve(3*VertexCount);
	}

	[[nodiscard]] FORCEINLINE int32 GetVertexIndex(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		check(TriEdgeIndex < 3);
		check(TriIndex < MaxTriID());

		return Triangles[TriIndex][TriEdgeIndex];
	}

	[[nodiscard]] FORCEINLINE FIndex3i GetTriangle(const int32 TriIndex) const
	{
		check(TriIndex < MaxTriID());
		check(TriIndex >= 0);

		check(Triangles[TriIndex][0] <= std::numeric_limits<int32>::max());
		check(Triangles[TriIndex][1] <= std::numeric_limits<int32>::max());
		check(Triangles[TriIndex][2] <= std::numeric_limits<int32>::max());

		return Triangles[TriIndex];
	}

	FORCEINLINE void SetTriangle(const int32 TriIndex, const FIndex3i& Triangle, const int32 ParentTriIndex)
	{
		check(Triangle[0] <= std::numeric_limits<int32>::max());
		check(Triangle[1] <= std::numeric_limits<int32>::max());
		check(Triangle[2] <= std::numeric_limits<int32>::max());

		Triangles[TriIndex] = Triangle;

		if (ParentTriIndex != TriIndex)
		{
			check(ParentTriIndex < BaseTriangleIndices.Num());
			BaseTriangleIndices[TriIndex] = BaseTriangleIndices[ParentTriIndex];
		}
	}

	[[nodiscard]] FORCEINLINE FIndex3i GetBaseTriangle(const int32 BaseTriangleIndex) const
	{
		check(BaseTriangleIndex < BaseTriangles.Num());
		check(BaseTriangleIndex >= 0);

		return BaseTriangles[BaseTriangleIndex];
	}

	[[nodiscard]] FORCEINLINE FIndex2i GetHalfEdgeVertexIndices(const int32 HalfEdgeIndex) const
	{
		return { Triangles[HalfEdgeIndex/3][HalfEdgeIndex%3],
				 Triangles[HalfEdgeIndex/3][Cycle3(HalfEdgeIndex%3)] };
	}

	[[nodiscard]] FORCEINLINE FIndex2i GetEdgeVertexIndices(const int32 EdgeIndex) const
	{
		const FEdge& Edge = Edges[EdgeIndex];
		return GetHalfEdgeVertexIndices(Edge.HalfEdge[0]);
	}

	FORCEINLINE void GetEdge(const int32 EdgeIndex, FIndex2i& OutVerts, FIndex2i& OutTris) const
	{
		const FEdge& Edge = Edges[EdgeIndex];

		OutVerts = GetEdgeVertexIndices(EdgeIndex);
		check(Edge.HalfEdge[0] != IndexConstants::InvalidID);
		OutTris = FIndex2i(Edge.HalfEdge[0] / 3,
						   Edge.HalfEdge[1] == IndexConstants::InvalidID ? IndexConstants::InvalidID : Edge.HalfEdge[1] / 3);
	}

	[[nodiscard]] FORCEINLINE int32 GetEdgeID(const int32 TriIndex, const int32 k) const
	{
		return HalfEdges[3 * TriIndex + k].Edge;
	}

	[[nodiscard]] FORCEINLINE const FVector3f& GetVertexPosition(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		return Vertices[GetVertexIndex(TriIndex, TriEdgeIndex)].Position;
	}

	[[nodiscard]] FORCEINLINE const FVector3f& GetVertexPosition(const int32 VertexIndex) const
	{
		return Vertices[VertexIndex].Position;
	}

	[[nodiscard]] FORCEINLINE const FVector2f& GetUV(const int32 VertexIndex) const
	{
		return Vertices[VertexIndex].UV;
	}

	[[nodiscard]] FORCEINLINE const FVector3f& GetNormal(const int32 VertexIndex) const
	{
		return Vertices[VertexIndex].Normal;
	}

	[[nodiscard]] FORCEINLINE const FVertex& GetVertex(const int32 VertexIndex) const
	{
		return Vertices[VertexIndex];
	}

	FORCEINLINE void SetVertexPosition(const int32 VertexIndex, const FVector3f Position)
	{
		Vertices[VertexIndex].Position = Position;
	}

	FORCEINLINE void GetTriVertices(const int32 TriIndex, FVector3f& v0, FVector3f& v1, FVector3f& v2) const
	{
		v0 = Vertices[GetVertexIndex(TriIndex, 0)].Position;
		v1 = Vertices[GetVertexIndex(TriIndex, 1)].Position;
		v2 = Vertices[GetVertexIndex(TriIndex, 2)].Position;
	}

	[[nodiscard]] FORCEINLINE int32 MaxVertexID() const
	{
		return Vertices.Num();
	}

	[[nodiscard]] FORCEINLINE bool IsValidVertex(const int32 VertexID) const
	{
		return true;
	}

	[[nodiscard]] FORCEINLINE int32 MaxTriID() const
	{
		return Triangles.Num();
	}

	[[nodiscard]] FORCEINLINE bool IsValidTri(const int32 TriIndex) const
	{
		return true;
	}

	// Return the triangle index on the other side of the half edge given by TriIndex, TriEdgeIndex or IndexConstants::InvalidID on boundary
	[[nodiscard]] FORCEINLINE int32 GetAdjTriangle(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		const int32 AdjEdge = HalfEdges[TriIndex * 3 + TriEdgeIndex].AdjHalfEdge;

		if (AdjEdge < 0)
			return IndexConstants::InvalidID;

		return AdjEdge / 3;
	}

	// Return (adjacent triangle index or IndexConstants::InvalidID on boundary, adjacent local edge index)
	[[nodiscard]] FORCEINLINE TPair<int32,int32> GetAdjEdge(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		const int32 AdjEdge = HalfEdges[TriIndex * 3 + TriEdgeIndex].AdjHalfEdge;

		if (AdjEdge < 0)
		{
			return { IndexConstants::InvalidID, IndexConstants::InvalidID };
		}

		return { AdjEdge / 3, AdjEdge % 3 };
	}

	// Return the triangle index on the other side of the half edge given by TriIndex, TriEdgeIndex or IndexConstants::InvalidID on boundary
	[[nodiscard]] FORCEINLINE int32 GetAdjHalfEdge(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		return HalfEdges[TriIndex * 3 + TriEdgeIndex].AdjHalfEdge;
	}

	[[nodiscard]] FORCEINLINE bool EdgeManifoldCheck(const int32 TriIndex, const int32 TriEdgeIndex) const
	{
		const int32 HalfEdgeIndex = TriIndex * 3 + TriEdgeIndex;
		const int32 AdjHalfEdgeIndex = HalfEdges[HalfEdgeIndex].AdjHalfEdge;

		if (AdjHalfEdgeIndex < 0)
			return true;

		const FIndex2i Verts0 = GetHalfEdgeVertexIndices(HalfEdgeIndex);
		const FIndex2i Verts1 = GetHalfEdgeVertexIndices(AdjHalfEdgeIndex);

		// Manifoldness check
		if( Verts0[0] != Verts1[1] || Verts0[1] != Verts1[0] )
		{
			return false;
		}

		return true;
	}

	[[nodiscard]] FORCEINLINE bool AllowEdgeFlip(const int32 TriIndex, const int32 TriEdgeIndex, const int32 AdjTriIndex ) const
	{
		return true;
	}

	[[nodiscard]] FORCEINLINE bool AllowEdgeSplit(const int32 TriIndex, const int32 TriEdgeIndex ) const
	{
		const int32 AdjEdge = HalfEdges[TriIndex * 3 + TriEdgeIndex].AdjHalfEdge;
		return (AdjEdge != IndexConstants::InvalidID);
	}

	[[nodiscard]] FORCEINLINE FVector3f GetTriangleNormal(const int32 TriIndex) const
	{
		const FIndex3i Triangle = GetTriangle(TriIndex);
		const FVector3f& p0 = Vertices[Triangle.A].Position;
		const FVector3f& p1 = Vertices[Triangle.B].Position;
		const FVector3f& p2 = Vertices[Triangle.C].Position;

		const FVector3f Edge01 = p1 - p0;
		const FVector3f Edge12 = p2 - p1;
		const FVector3f Edge20 = p0 - p2;

		return (Edge01 ^ Edge20).GetSafeNormal();
	}

	FORCEINLINE int32 AddEdges( int32 EdgesToAdd )
	{
		return Edges.AddDefaulted( EdgesToAdd );
	}

	FORCEINLINE int32 AddVertices( int32 VerticesToAdd )
	{
		const int32 Num = Vertices.AddDefaulted( VerticesToAdd );
		return Num;
	}

	FORCEINLINE int32 AddTriangles( const int32 TrianglesToAdd )
	{
		const int32 TriCount = Triangles.Num();

		const int32 NewHalfEdges = 3 * TrianglesToAdd;
		HalfEdges.AddDefaulted(NewHalfEdges);
		Triangles.AddDefaulted(TrianglesToAdd);
		BaseTriangleIndices.AddDefaulted(TrianglesToAdd);
		check(BaseTriangleIndices.Num() == Triangles.Num());

		return TriCount;
	}

	// Interpolate vertex \param TargetIdx along given edge with provided weight (with respect to first half-edge).
	inline void InterpolateVertex(const float Weight, const int32 TargetIdx, const int32 EdgeIndex)
	{
		const FEdge& Edge = Edges[EdgeIndex];

		const FIndex2i VIDs = GetHalfEdgeVertexIndices(Edge.HalfEdge[0]);

		const int32 VID0 = VIDs[0];
		const int32 VID1 = VIDs[1];

		FVertex& NewVertex = Vertices[TargetIdx];
		NewVertex.Position = Vertices[VID0].Position * (1.f - Weight) + Vertices[VID1].Position * Weight;
		NewVertex.UV       = Vertices[VID0].UV       * (1.f - Weight) + Vertices[VID1].UV       * Weight;
		NewVertex.Normal   = Vertices[VID0].Normal   * (1.f - Weight) + Vertices[VID1].Normal   * Weight;

		if (Vertices[VID0].BaseHalfEdge == IndexConstants::InvalidID &&
			Vertices[VID1].BaseHalfEdge == IndexConstants::InvalidID)
		{
			// Interpolating two base vertices
			const FEdge& BaseEdge = BaseEdges[Edge.BaseEdge];

			NewVertex.BaseHalfEdge    = BaseEdge.HalfEdge[0];
			NewVertex.BaseAdjHalfEdge = BaseEdge.HalfEdge[1];

			const int32 LocalEdgeIndex = BaseEdge.HalfEdge[0] % 3;
			NewVertex.Barycentric[LocalEdgeIndex] = (1.f - Weight);
			NewVertex.Barycentric[(LocalEdgeIndex+1)%3] = Weight;
			NewVertex.Barycentric[(LocalEdgeIndex+2)%3] = 0.f;
		}
		else if (Vertices[VID0].BaseHalfEdge != IndexConstants::InvalidID &&
				 Vertices[VID1].BaseHalfEdge != IndexConstants::InvalidID)
		{
			// two edge vertices, either along or across edge
			FVertex V0 = Vertices[VID0];
			FVertex V1 = Vertices[VID1];

			bool Flip0 = false, Flip1 = false;
			if (V0.BaseHalfEdge / 3 == V1.BaseHalfEdge / 3)
			{
				// no flips
			}
			else if (V1.BaseAdjHalfEdge != IndexConstants::InvalidID && V0.BaseHalfEdge / 3 == V1.BaseAdjHalfEdge / 3)
			{
				Flip1 = true;
			}
			else if (V0.BaseAdjHalfEdge != IndexConstants::InvalidID && V0.BaseAdjHalfEdge / 3 == V1.BaseHalfEdge / 3)
			{
				Flip0 = true;
			}
			else
			{
				check(V1.BaseAdjHalfEdge != IndexConstants::InvalidID);
				check(V0.BaseAdjHalfEdge / 3 == V1.BaseAdjHalfEdge / 3);
				Flip0 = Flip1 = true;
			}

			if (Flip0) { V0.Flip(); }
			if (Flip1) { V1.Flip(); }

			check(V0.BaseHalfEdge / 3 == V1.BaseHalfEdge / 3);

			NewVertex.Barycentric = V0.Barycentric * (1.f - Weight) + V1.Barycentric * Weight;
			NewVertex.BaseHalfEdge = V0.BaseHalfEdge;
			NewVertex.BaseAdjHalfEdge = (V0.BaseHalfEdge == V1.BaseHalfEdge && V1.BaseAdjHalfEdge != IndexConstants::InvalidID) ? V0.BaseAdjHalfEdge : IndexConstants::InvalidID;
		}
		else
		{
			// one base and one edge vertex
			float WeightA = 0.f;

			int32 VA, VB;
			if (Vertices[VID0].BaseHalfEdge == IndexConstants::InvalidID)
			{
				check(Vertices[VID1].BaseHalfEdge != IndexConstants::InvalidID);
				VA = VID0;
				VB = VID1;
				WeightA = (1.f - Weight);
			}
			else
			{
				check(Vertices[VID0].BaseHalfEdge != IndexConstants::InvalidID);
				check(Vertices[VID1].BaseHalfEdge == IndexConstants::InvalidID);

				VA = VID1;
				VB = VID0;
				WeightA = Weight;
			}

			check(Vertices[VB].BaseHalfEdge < BaseTriangles.Num() * 3);

			// VA is base vertex, VB is not
			const FIndex3i BaseTriangle = BaseTriangles[Vertices[VB].BaseHalfEdge / 3];

			NewVertex.BaseHalfEdge    = Vertices[VB].BaseHalfEdge;
			NewVertex.BaseAdjHalfEdge = Vertices[VB].BaseAdjHalfEdge;

			int32 LocalEdgeIndex = IndexConstants::InvalidID;
			int32 BaseHalfEdge = Vertices[VB].BaseHalfEdge;
			if (BaseTriangle[BaseHalfEdge%3] == VA)
			{
				// along the same edge
				LocalEdgeIndex = BaseHalfEdge%3;
			}
			else if (BaseTriangle[(BaseHalfEdge+1)%3] == VA )
			{
				// along the same edge
				LocalEdgeIndex = (BaseHalfEdge+1)%3;
			}
			else if (BaseTriangle[(BaseHalfEdge+2)%3] == VA)
			{
				// not along the same edge -> interior
				NewVertex.BaseAdjHalfEdge = IndexConstants::InvalidID;
				LocalEdgeIndex = (BaseHalfEdge+2)%3;
			}

			if (LocalEdgeIndex >= 0)
			{
				NewVertex.Barycentric = Vertices[VB].Barycentric * (1.f - WeightA);
				NewVertex.Barycentric[LocalEdgeIndex] += WeightA;
			}
			else
			{
				check(Vertices[VB].BaseAdjHalfEdge != IndexConstants::InvalidID);

				FVertex VC = Vertices[VB];
				VC.Flip();

				NewVertex.BaseHalfEdge = VC.BaseHalfEdge;
				NewVertex.BaseAdjHalfEdge = IndexConstants::InvalidID;
				const FIndex3i BaseAdjTriangle = BaseTriangles[VC.BaseHalfEdge / 3];

				check(BaseTriangle[(BaseHalfEdge+0)%3] == BaseAdjTriangle[(VC.BaseHalfEdge+1)%3] && 
				      BaseTriangle[(BaseHalfEdge+1)%3] == BaseAdjTriangle[(VC.BaseHalfEdge+0)%3]);
				
				check(BaseAdjTriangle[(VC.BaseHalfEdge+2)%3] == VA);

				NewVertex.Barycentric = VC.Barycentric * (1.f - WeightA);
				NewVertex.Barycentric[(VC.BaseHalfEdge+2)%3] += WeightA;
			}
		}

		check(NewVertex.BaseHalfEdge < BaseTriangles.Num() * 3);
		check(NewVertex.BaseAdjHalfEdge == IndexConstants::InvalidID || NewVertex.BaseAdjHalfEdge < BaseTriangles.Num() * 3);

		static constexpr bool bValidate = false;

		if (bValidate)
		{
			// recompute position
			FVector3f p(0.f);

			const FIndex3i BaseTriangle = BaseTriangles[NewVertex.BaseHalfEdge / 3];

			p += Vertices[BaseTriangle.A].Position * NewVertex.Barycentric[0];
			p += Vertices[BaseTriangle.B].Position * NewVertex.Barycentric[1];
			p += Vertices[BaseTriangle.C].Position * NewVertex.Barycentric[2];

			check(VectorUtil::EpsilonEqual(p, NewVertex.Position, 1.e-5f));
		}
	}

	[[nodiscard]] FORCEINLINE int32 HalfEdgeCount() const
	{
		return Triangles.Num() * 3;
	}

	[[nodiscard]] FORCEINLINE int32 EdgeCount() const
	{
		return Edges.Num();
	}

	[[nodiscard]] FORCEINLINE int32 GetEdgeIndex(int32 TriIndex, int LocalEdgeIndex) const
	{
		return HalfEdges[TriIndex * 3 + LocalEdgeIndex].Edge;
	}

	FORCEINLINE void LinkEdge(int32 HalfEdgeIndex0, int32 HalfEdgeIndex1)
	{
		HalfEdges[ HalfEdgeIndex0 ].AdjHalfEdge = HalfEdgeIndex1;
		if( HalfEdgeIndex1 >= 0 )
		{
			HalfEdges[ HalfEdgeIndex1 ].AdjHalfEdge = HalfEdgeIndex0;
		}
	}

	FORCEINLINE void LinkEdge(int32 HalfEdgeIndex0, int32 HalfEdgeIndex1, int32 EdgeIdx, int32 ParentEdgeIdx)
	{
		check(HalfEdgeIndex0 >= 0);
		check(EdgeIdx >= 0);

		LinkEdge(HalfEdgeIndex0, HalfEdgeIndex1);
		HalfEdges[HalfEdgeIndex0].Edge = EdgeIdx;
		if (HalfEdgeIndex1 >= 0)
		{
			HalfEdges[HalfEdgeIndex1].Edge = EdgeIdx;
		}

		Edges[EdgeIdx].HalfEdge = FIndex2i(HalfEdgeIndex0, HalfEdgeIndex1);

		if (ParentEdgeIdx == IndexConstants::InvalidID)
		{
			Edges[EdgeIdx].BaseEdge = IndexConstants::InvalidID;
		}
		else if (ParentEdgeIdx != EdgeIdx)
		{
			Edges[EdgeIdx].BaseEdge = Edges[ParentEdgeIdx].BaseEdge;
		}
	}

	void Unlink(int32 HalfEdgeIndex)
	{
		const int32 AdjHalfEdge = HalfEdges[HalfEdgeIndex].AdjHalfEdge;
		if (AdjHalfEdge != IndexConstants::InvalidID)
		{
			HalfEdges[AdjHalfEdge].AdjHalfEdge = IndexConstants::InvalidID;
		}

		HalfEdges[HalfEdgeIndex].AdjHalfEdge = IndexConstants::InvalidID;
	}

	int32 GetBaseTriangleIndex(const int32 TriIndex) const
	{
		return BaseTriangleIndices[TriIndex];
	}

	void CheckConsistency() const;

	void Dump(std::ostream& out) const;

private:

	TArray<FIndex3i>	Triangles;           // triangle as triplets
	TArray<FVertex>     Vertices;
	TArray<FHalfEdge>   HalfEdges;
	TArray<FEdge>       Edges;

	TArray<int32>       BaseTriangleIndices; // triangle to base triangle
	TArray<FIndex3i>    BaseTriangles;
	TArray<FEdge>       BaseEdges;
};

inline void FHalfEdgeMesh::CheckConsistency() const
{
	check(HalfEdges.Num() == Triangles.Num() * 3);

	for (int32 HalfEdgeIdx=0; HalfEdgeIdx < HalfEdges.Num(); ++HalfEdgeIdx)
	{
		const FHalfEdge& HalfEdge = HalfEdges[HalfEdgeIdx];
		if (HalfEdge.AdjHalfEdge != IndexConstants::InvalidID )
		{
			check(HalfEdges[HalfEdge.AdjHalfEdge].AdjHalfEdge == HalfEdgeIdx);
		}

		check(Triangles[HalfEdgeIdx/3][HalfEdgeIdx%3] < Vertices.Num());

		const FEdge& Edge = Edges[HalfEdge.Edge];
		check(Edge.HalfEdge[0] == HalfEdgeIdx || Edge.HalfEdge[0] == HalfEdge.AdjHalfEdge);
		check(Edge.HalfEdge[1] == IndexConstants::InvalidID || Edge.HalfEdge[1] == HalfEdgeIdx || Edge.HalfEdge[1] == HalfEdge.AdjHalfEdge);
		check(Edge.HalfEdge[1] != Edge.HalfEdge[0]);
	}

	for (const FVertex& Vertex : Vertices)
	{
		check(Vertex.BaseHalfEdge / 3 < BaseTriangles.Num());
	}
}

inline void FHalfEdgeMesh::Dump(std::ostream& out) const
{
	for (int32 HalfEdgeIdx=0; HalfEdgeIdx < HalfEdges.Num(); ++HalfEdgeIdx)
	{
		const FHalfEdge& H = HalfEdges[HalfEdgeIdx];
		out << "HalfEdge[ " << HalfEdgeIdx << "] = (v0 = " << Triangles[HalfEdgeIdx/3][HalfEdgeIdx%3] << ", adj " << H.AdjHalfEdge << ", edge " << H.Edge << ")" << std::endl;
	}

	for (int32 EdgeIdx=0; EdgeIdx < Edges.Num(); ++EdgeIdx)
	{
		const FEdge& E = Edges[EdgeIdx];
		out << "Edge[" << EdgeIdx << "] = (" << E.HalfEdge.A << ", " << E.HalfEdge.B << ")" << std::endl;
	}
}

void ConvertToHalfEdgeMesh(const FDynamicMesh3& Mesh, FHalfEdgeMesh& HalfEdgeMesh);

void ConvertToDynamicMesh3(const FHalfEdgeMesh& HalfEdgeMesh, const FDynamicMesh3& InputMesh, FDynamicMesh3& RefinedMesh);

void ConvertToMeshPartitionMesh(const FHalfEdgeMesh& HalfEdgeMesh, const FDynamicMesh3& InputMesh, UE::MeshPartition::FMeshData& OutputMesh);

} // namespace Geometry
} // namespace UE