// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modifiers/Tessellation/MeshPostProcessing.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Containers/Array.h"
#include "Util/IndexUtil.h"
#include "VectorUtil.h"
#include "VectorTypes.h"
#include "HAL/IConsoleManager.h"
#include "QueueRemesher.h"
#include "Async/ParallelFor.h"

#include <cstdlib>

namespace UE {
namespace Geometry {

static TAutoConsoleVariable<float> CVarMegaMesh_FlipAngleThreshold(
	TEXT("MegaMesh.AdaptiveTessellation.FlipAngleThreshold"),
	0.1 * UE_PI,
	TEXT("Enable elliptic weight average filtering for displacement map filtering.")
);

static TAutoConsoleVariable<float> CVarTessellationRegularizationSteps(
	TEXT("MegaMesh.AdaptiveTessellation.RegularizationSteps"),
	1,
	TEXT("Number of mesh regularization iterations to run in during mesh postprocessing.")
);

static TAutoConsoleVariable<int32> CVarTessellationPostEdgeFlipOrderedCount(
	TEXT("MegaMesh.AdaptiveTessellation.EdgeFlipSteps"),
	1,
	TEXT("Number of edge flip iterations to run in during mesh postprocessing.")
);

static TAutoConsoleVariable<int32> CVarTessellationPostSplitLongEdgesCount(
	TEXT("MegaMesh.AdaptiveTessellation.SplitLongEdgesSteps"),
	1,
	TEXT("Number of edge split iterations (reducing the amount of long edges) to run in during mesh postprocessing.")
);

static TAutoConsoleVariable<float> CVarMegaMesh_SplitThreshold(
	TEXT("MegaMesh.AdaptiveTessellation.SplitThreshold"),
	0.9f,
	TEXT("Ratio of longest edge/sum of length of other edges to trigger edge splitting.")
);

static TAutoConsoleVariable<int32> CVarTessellationRemeshingIterations(
	TEXT("MegaMesh.AdaptiveTessellation.RemeshingSteps"),
	0,
	TEXT("Number of remeshing iterations to apply"));

void PostProcessMesh(FDynamicMesh3& Mesh, TArray<FVector3d>& Displacements, FDisplaceFunc DisplacementFunctor, const double MinimumEdgeLength, const double MaximumEdgeLength)
{
	const int32 PostSplitLongEdgesCount = CVarTessellationPostSplitLongEdgesCount.GetValueOnAnyThread();
	for (int32 Iter=0; Iter < PostSplitLongEdgesCount; ++Iter)
	{
		SplitLongEdges(Mesh, Displacements, DisplacementFunctor, CVarMegaMesh_SplitThreshold.GetValueOnAnyThread());
	}

	const int32 PostEdgeFlipOrderedCount = CVarTessellationPostEdgeFlipOrderedCount.GetValueOnAnyThread();
	for (int32 Iter = 0; Iter < PostEdgeFlipOrderedCount; ++Iter)
	{
		IntrinsicDelaunayFlipEdge(Mesh, Displacements, DisplacementFunctor, CVarMegaMesh_FlipAngleThreshold.GetValueOnAnyThread());
	}

	const int32 RemeshingIterations = CVarTessellationRemeshingIterations.GetValueOnAnyThread();
	if (RemeshingIterations > 0)
	{
		Geometry::FQueueRemesher Remesher(&Mesh);
		Remesher.MaxRemeshIterations = RemeshingIterations;
		Remesher.MinEdgeLength = MinimumEdgeLength;
		Remesher.MaxEdgeLength = MaximumEdgeLength;
		Remesher.BasicRemeshPass();
	}
}

void RegularizeMesh(FDynamicMesh3& Mesh)
{
	// DynamicMesh3::IsBoundaryVertex could be used for all vertices, but it iterates over all incident edges.
	// iterating over boundary edges only requires iterating once over all edges.
	//
	TBitArray<> IsInteriorVertex;
	IsInteriorVertex.Init(true, Mesh.MaxVertexID()); 
	for (const int32 EID : Mesh.BoundaryEdgeIndicesItr())
	{
		FDynamicMesh3::FEdge Edge = Mesh.GetEdge(EID);
		IsInteriorVertex[Edge.Vert[0]] = false;
		IsInteriorVertex[Edge.Vert[1]] = false;
	}

	const int32 RegularizationSteps = CVarTessellationRegularizationSteps.GetValueOnAnyThread();
	for (int32 Iter = 0; Iter < RegularizationSteps; ++Iter)
	{
		UE::Geometry::RegularizeVolumePreserving(Mesh, true, IsInteriorVertex);
	}
}

} // namespace Geometry
} // namespace UE