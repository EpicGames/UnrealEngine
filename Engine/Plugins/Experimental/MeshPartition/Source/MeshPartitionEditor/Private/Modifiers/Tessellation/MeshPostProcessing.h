// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tessellation/Regularization.h"
#include "Containers/Array.h"

namespace UE {
namespace Geometry {

class FDynamicMesh3;

// Improve mesh regularity by splitting long edges, applying Delaunay-flips and volume-preserving mesh smoothing.
// If the CVar CVarTessellationRemeshingIterations is set, the Remesher is run, using the specified edge lengths
// as target lengths
//
void PostProcessMesh(FDynamicMesh3& Mesh, TArray<FVector3d>& Displacements, FDisplaceFunc DisplacementFunctor, const double MinimumEdgeLength, const double MaximumEdgeLength);

// Apply volume preserving regularization multiple times, based on CVar MegaMesh.AdaptiveTessellation.RegularizationSteps
void RegularizeMesh(FDynamicMesh3& Mesh);

} // namespace Geometry
} // namespace UE
