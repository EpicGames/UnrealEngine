// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tasks/Task.h"
#include "Templates/Invoke.h"

/**
 * A specialized implementation of `UE::Tasks::FPipe` which supports concurrent readers. The execution of tasks in the FSharedPipe
 * will always be FIFO in the order that the prerequisites are satisfied.
 * FSharedPipe is similar to FRWLock allowing multiple concurrent readers or a single writer however it does so in a non-blocking manner.
 * 
 * `LaunchExclusive` behaves much like FPipe::Launch. Exclusive tasks enqueued into the FSharedPipe will never execute concurrently
 * with any other task launched into this pipe.
 * Where FSharedPipe differs from FPipe is the ability to have multiple shared readers for the same resource through the `LaunchShared` function.
 * Shared tasks will execute through the pipe concurrently.
 */
class FSharedPipe
{
	enum EExecutionMode
	{
		Shared,
		Exclusive,
	};

public:
	UE_NONCOPYABLE(FSharedPipe);

	explicit FSharedPipe(const TCHAR* InDebugName)
		: DebugName(InDebugName)
	{
	}

	~FSharedPipe()
	{
		check(!HasWork());
	}
	
	bool HasWork()
	{
		return TaskCount.load(std::memory_order_relaxed) != 0;
	}

	template<typename TaskBodyType, typename PrerequisitesCollectionType>
	UE::Tasks::FTask LaunchExclusive
	(
		const TCHAR* InDebugName, 
		TaskBodyType&& TaskBody, 
		PrerequisitesCollectionType&& Prerequisites,
		UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority ExtendedPriority = UE::Tasks::EExtendedTaskPriority::None
	)
	{
		TaskCount.fetch_add(1, std::memory_order_relaxed);

		return UE::Tasks::Launch(
			InDebugName,
			[this, InDebugName, TaskBody = MoveTemp(TaskBody)]() mutable
			{
				UE::TUniqueLock Lock(Mutex);

				TArray<UE::Tasks::FTask> Prereqs;
				// Reserve space for the active shared tasks plus 1 to avoid reallocating if there is a LastExclusiveTask.
				Prereqs.Reserve(ActiveSharedTasks.Num() + 1);
				Prereqs.Append(ActiveSharedTasks);

				if (LastExclusiveTaskInPipe.IsValid())
				{
					Prereqs.Emplace(LastExclusiveTaskInPipe);
				}

				UE::Tasks::FTask Inner = UE::Tasks::Launch(
					InDebugName,
					[this, TaskBody = MoveTemp(TaskBody)]()
					{
						UE_MT_SCOPED_WRITE_ACCESS(AccessDetector);

						Invoke(TaskBody);
						TaskCount.fetch_sub(1, std::memory_order_relaxed);
					},
					Prereqs
				);

				LastExclusiveTaskInPipe = Inner;

				UE::Tasks::AddNested(Inner);
			},
			Forward<PrerequisitesCollectionType>(Prerequisites),
			TaskPriority,
			ExtendedPriority
		);
	}

	template<typename TaskBodyType, typename PrerequisitesCollectionType>
	UE::Tasks::FTask LaunchShared(
		const TCHAR* InDebugName, 
		TaskBodyType&& TaskBody, 
		PrerequisitesCollectionType&& Prerequisites,
		UE::Tasks::ETaskPriority TaskPriority = UE::Tasks::ETaskPriority::Normal,
		UE::Tasks::EExtendedTaskPriority ExtendedPriority = UE::Tasks::EExtendedTaskPriority::None
	)
	{
		TaskCount.fetch_add(1, std::memory_order_relaxed);

		return UE::Tasks::Launch(
			InDebugName,
			[this, InDebugName, TaskBody = MoveTemp(TaskBody)]() mutable
			{
				UE::TUniqueLock Lock(Mutex);

				// An event to notify when the Inner task completes is required because there is no stable identifier for the inner task
				// until after it has launched, by which point we can no longer pass it it's own handle to remove from the list of active tasks
				// on completion.
				UE::Tasks::FTaskEvent Finished(TEXT("FinishShared"));

				ActiveSharedTasks.Emplace(Finished);

				TArray<UE::Tasks::FTask> Prereqs;

				if (LastExclusiveTaskInPipe.IsValid())
				{
					Prereqs.Emplace(LastExclusiveTaskInPipe);
				}

				UE::Tasks::FTask Inner = UE::Tasks::Launch(
					InDebugName,
					[this, TaskBody = MoveTemp(TaskBody), Finished]()
					{
						UE_MT_SCOPED_READ_ACCESS(AccessDetector);
						Invoke(TaskBody);

						TaskCount.fetch_sub(1, std::memory_order_relaxed);

						UE::TUniqueLock Lock(Mutex);
						int32 RemoveCount = ActiveSharedTasks.RemoveSingleSwap(Finished);
						check(RemoveCount == 1);
					},
					Prereqs
				);

				Finished.AddPrerequisites(Inner);

				Finished.Trigger();
				UE::Tasks::AddNested(Finished);
			},
			Forward<PrerequisitesCollectionType>(Prerequisites),
			TaskPriority,
			ExtendedPriority
		);
	}
private:
	UE::FMutex Mutex;

	/**
	 * Tail of the linked list of exclusive tasks.
	 * When a new exclusive task is pushed into the pipe, we immediately swap the last exclusive task to the new task
	 * to ensure that any subsequent tasks are correctly ordered after this one. The task graph's internal prerequisite system
	 * manages the rest of the linked list for us.
	 */
	UE::Tasks::FTask LastExclusiveTaskInPipe;

	/**
	 * Retain a list of all currently executing shared tasks.
	 * When a new exclusive task is pushed into the pipe, all of these are set as prerequisites for that task to ensure
	 * it only begins execution when the currently executing shared tasks have completed. 
	 * No new shared tasks are launched once this process begins until the exclusive task list completes.
	 */
	TArray<UE::Tasks::FTask> ActiveSharedTasks;
	
	std::atomic<uint64> TaskCount = { 0 };
	
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(AccessDetector);

	const TCHAR* DebugName;
};