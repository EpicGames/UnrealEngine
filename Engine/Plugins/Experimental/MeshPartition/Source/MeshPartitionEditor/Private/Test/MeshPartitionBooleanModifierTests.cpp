// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#if WITH_DEV_AUTOMATION_TESTS

#include "CQTest.h"
#include "Editor.h"
#include "FileHelpers.h"
#include "GameMapsSettings.h"
#include "LevelEditorSubsystem.h"
#include "MeshPartition.h"
#include "ScopedTransaction.h"
#include "Modifiers/MeshPartitionBooleanModifier.h"
#include "Test/MeshPartitionTestUtils.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

/**
 * Tests for MeshPartition Boolean Modifier
 */
TEST_CLASS_WITH_FLAGS(BooleanModifierTests, "MeshPartition.Modifier", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	UWorld* World = nullptr;
	AMeshPartition* MeshPartition = nullptr;
	AMeshPartition* MeshPartition2 = nullptr;
	UMeshPartitionDefinition* MeshPartitionDefinition = nullptr;
	AActor* BooleanModifierActor = nullptr;
	AActor* BooleanModifierActorB = nullptr;
	UBooleanModifier* BooleanModifier = nullptr;
	UBooleanModifier* BooleanModifierB = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionEditorComponent2 = nullptr;
	UStaticMesh* CylinderMesh = nullptr;
	UStaticMesh* ConeMesh = nullptr;

	// Test Configuration
	const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
	const FString ReferencePath = TEXT("/MeshPartition/UnitTests/BooleanModifierTests");
	const int32 MeshWidth = 1000;
	const int32 MeshHeight = 1000;
	const int32 MeshWidthVertexCount = 500;
	const int32 MeshHeightVertexCount = 500;

	const FString CylinderPath = TEXT("/Engine/BasicShapes/Cylinder.Cylinder");
	const FString ConePath = TEXT("/Engine/BasicShapes/Cone.Cone");

	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);

#if AUTOMATED_TESTING_EXECUTE_IN_EDITOR_WORLD
		World = GEditor->GetEditorWorldContext().World();
#else
		// Spawn world
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);
		ASSERT_THAT(IsNotNull(ScopedEditorWorld.Get(), TEXT("Failed to create scoped editor world")));

		World = ScopedEditorWorld->GetWorld();
#endif
		ASSERT_THAT(IsNotNull(World, TEXT("World is not valid")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));

		MeshPartition = TestUtils::CreateTestMesh(
			World, MeshWidth, MeshHeight, MeshWidthVertexCount, MeshHeightVertexCount
		);
		ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to create MeshPartition")));

		MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
		ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent, TEXT("Failed to get MeshPartitionComponent")));

		// Pausing the transformer pipeline during the test to avoid potential race condition
		MeshPartitionEditorComponent->SetPauseTransformerPipeline(true);

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));
	}

	// UE-TC-22590
	TEST_METHOD(Verify_Boolean_Modifier_UndoRedo_Creates_And_Dismisses_Actor)
	{
		TestCommandBuilder
			.Do(TEXT("Create BooleanModifier actor creation Transaction"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Create BooleanModifier")));

					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));
				})
			.Then(TEXT("Undo creation of BooleanModifier actor"), [this]()
				{
					GEditor->UndoTransaction();
					ASSERT_THAT(IsFalse(IsValid(BooleanModifier), TEXT("Undo operation didn't remove BooleanModifier")));
					ASSERT_THAT(IsFalse(IsValid(BooleanModifierActor), TEXT("Undo operation didn't remove BooleanModifier actor")));
				})
			.Then(TEXT("Redo creation of BooleanModifier actor"), [this]()
				{
					GEditor->RedoTransaction();
					ASSERT_THAT(IsTrue(IsValid(BooleanModifier), TEXT("Redo operation didn't recreate BooleanModifier")));
					ASSERT_THAT(IsTrue(IsValid(BooleanModifierActor), TEXT("Redo operation didn't recreate BooleanModifier actor")));
				});
	}

	// UE-TC-22590
	TEST_METHOD(Verify_Boolean_Modifier_Assignment_To_MeshTerrain)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Then(TEXT("Notify MeshPartition that Modifier has been assigned"), [this]()
				{
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Verify_Boolean_Modifier_Assignment_To_MeshTerrain_GroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22591
	TEST_METHOD(Verify_Boolean_Modifier_Translation_Applies_Operation_To_Terrain)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Then(TEXT("Notify MeshPartition that Modifier has been assigned"), [this]()
				{
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Apply translation transform to the modifier"), [this]()
				{
					const FVector NewLocation = FVector(50, 0, 0);
					BooleanModifier->SetWorldLocation(NewLocation);

					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Verify_Boolean_Modifier_Translation_Applies_Operation_To_Terrain_GroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22591
	TEST_METHOD(Verify_Boolean_Modifier_Translation_UndoRedo_Manages_Transform)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Then(TEXT("Notify MeshPartition that Modifier has been assigned"), [this]()
				{
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Apply translation transform to the modifier"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Transform BooleanModifier")));
					BooleanModifier->Modify();

					const FVector NewLocation = FVector(50, 0, 0);
					BooleanModifier->SetWorldLocation(NewLocation);

					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Undo transform"), [this]()
				{
					GEditor->UndoTransaction();
					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate undo transform"), [this]()
				{
					const FVector ExpectedPosition = FVector::ZeroVector;
					const FVector ActualPosition = BooleanModifier->GetRelativeLocation();
					ASSERT_THAT(IsTrue(ExpectedPosition.Equals(ActualPosition),
						TEXT("Undo operation didn't reset BooleanModifier's transform")));
				})
			.Then(TEXT("Redo transform"), [this]()
				{
					GEditor->RedoTransaction();
					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Verify_Boolean_Modifier_Translation_Applies_Operation_To_Terrain_GroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22591
	TEST_METHOD(Verify_Boolean_Modifier_Rotation_Applies_Operation_To_Terrain)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Then(TEXT("Notify MeshPartition that Modifier has been assigned"), [this]()
				{
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Apply rotation transform to the modifier"), [this]()
				{
					const FRotator NewRotation = FRotator(50, 0, 0);
					BooleanModifier->SetWorldRotation(NewRotation);

					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Verify_Boolean_Modifier_Rotation_Applies_Operation_To_Terrain_GroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22591
	TEST_METHOD(Verify_Boolean_Modifier_Rotation_UndoRedo_Manages_Transform)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Then(TEXT("Notify MeshPartition that Modifier has been assigned"), [this]()
				{
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Apply rotation transform to the modifier"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Transform BooleanModifier")));
					BooleanModifier->Modify();

					const FRotator NewRotation = FRotator(50, 0, 0);
					BooleanModifier->SetWorldRotation(NewRotation);

					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Undo transform"), [this]()
				{
					GEditor->UndoTransaction();
					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate undo transform"), [this]()
				{
					const FRotator ExpectedRotation = FRotator::ZeroRotator;
					const FRotator ActualRotation = BooleanModifier->GetRelativeRotation();
					ASSERT_THAT(IsTrue(ExpectedRotation.Equals(ActualRotation),
						TEXT("Undo operation didn't reset BooleanModifier's transform")));
				})
			.Then(TEXT("Redo transform"), [this]()
				{
					GEditor->RedoTransaction();
					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Verify_Boolean_Modifier_Rotation_Applies_Operation_To_Terrain_GroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22591
	TEST_METHOD(Verify_Boolean_Modifier_Scaling_Applies_Operation_To_Terrain)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Then(TEXT("Notify MeshPartition that Modifier has been assigned"), [this]()
				{
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Apply scale transform to the modifier"), [this]()
				{
					const FVector NewScale = FVector(2., 3., 0.5);
					BooleanModifier->SetWorldScale3D(NewScale);

					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Verify_Boolean_Modifier_Scaling_Applies_Operation_To_Terrain_GroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22591
	TEST_METHOD(Verify_Boolean_Modifier_Scaling_UndoRedo_Manages_Transform)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Then(TEXT("Notify MeshPartition that Modifier has been assigned"), [this]()
				{
					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Apply scale transform to the modifier"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Transform BooleanModifier")));
					BooleanModifier->Modify();

					const FVector NewScale = FVector(2., 3., 0.5);
					BooleanModifier->SetWorldScale3D(NewScale);

					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Undo transform"), [this]()
				{
					GEditor->UndoTransaction();
					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate undo transform"), [this]()
				{
					const FVector ExpectedScale = FVector(1.f, 1.f, 1.f);
					const FVector ActualScale = BooleanModifier->GetRelativeScale3D();
					ASSERT_THAT(IsTrue(ExpectedScale.Equals(ActualScale),
						TEXT("Undo operation didn't reset BooleanModifier's transform")));
				})
			.Then(TEXT("Redo transform"), [this]()
				{
					GEditor->RedoTransaction();
					BooleanModifier->UpdateFromMesh();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Verify_Boolean_Modifier_Scaling_Applies_Operation_To_Terrain_GroundTruth");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22592
	TEST_METHOD(Verify_Boolean_Modifier_Affects_Assigned_MeshTerrain_Only)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Do(TEXT("Create second MeshPartition and move it"), [this]()
				{
					MeshPartition2 = TestUtils::CreateTestMesh(
						World, MeshWidth, MeshHeight, MeshWidthVertexCount, MeshHeightVertexCount);
					ASSERT_THAT(IsNotNull(MeshPartition2, TEXT("Failed to create second MeshPartition")));

					MeshPartitionEditorComponent2 = Cast<UMeshPartitionEditorComponent>(MeshPartition2->GetMeshPartitionComponent());
					ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent2, TEXT("Failed to get MeshPartitionEditorComponent")));

					const FVector NewLocation = { 1100., 0., 0. };
					MeshPartition2->SetActorLocation(NewLocation);
					MeshPartitionEditorComponent2->SetPauseTransformerPipeline(true);
				})
			.Then(TEXT("Create Boolean actor and modifier"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);

					MeshPartitionEditorComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that only first MeshPartition is affected"), [this]()
				{
					// Verify that MeshTerrainA is modifier by Cylinder Mesh
					const FString GroundTruthAssetA = TEXT("Boolean_Modifier_Affected");
					CompareToReferenceAsset(GroundTruthAssetA, MeshPartitionEditorComponent);

					const FString GroundTruthAssetB = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetB, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Move modifier over to second MeshPartition"), [this]()
				{
					const FVector NewLocation = MeshPartition2->GetActorLocation();
					BooleanModifier->SetWorldLocation(NewLocation);

					BooleanModifier->PostEditComponentMove(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that now both MeshPartitions are unaffected"), [this]()
				{
					const FString GroundTruthAssetA = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetA, MeshPartitionEditorComponent);

					const FString GroundTruthAssetB = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetB, MeshPartitionEditorComponent2);
				});
	}

	// UE-TC-22592
	TEST_METHOD(Verify_Boolean_Modifier_AffectedMeshPartition_Reassignment_Affects_New_Terrain_Only)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Do(TEXT("Create second MeshPartition and move it"), [this]()
				{
					MeshPartition2 = TestUtils::CreateTestMesh(
						World, MeshWidth, MeshHeight, MeshWidthVertexCount, MeshHeightVertexCount);
					ASSERT_THAT(IsNotNull(MeshPartition2, TEXT("Failed to create second MeshPartition")));

					MeshPartitionEditorComponent2 = Cast<UMeshPartitionEditorComponent>(MeshPartition2->GetMeshPartitionComponent());
					ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent2, TEXT("Failed to get MeshPartitionEditorComponent")));

					const FVector NewLocation = { 1100., 0., 0. };
					MeshPartition2->SetActorLocation(NewLocation);
					MeshPartitionEditorComponent2->SetPauseTransformerPipeline(true);
				})
			.Then(TEXT("Create Boolean actor and modifier and assign it to second MeshPartition"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);

					BooleanModifier->SetAffectedMeshPartition(MeshPartition2);

					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that neither MeshPartition is affected when modifier is positioned inside first MeshPartition"), [this]()
				{
					const FString GroundTruthAsset = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent);

					CompareToReferenceAsset(GroundTruthAsset, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Move modifier over to second MeshPartition"), [this]()
				{
					const FVector NewLocation = MeshPartition2->GetActorLocation();
					BooleanModifier->SetWorldLocation(NewLocation);

					BooleanModifier->PostEditComponentMove(true);
					MeshPartitionEditorComponent2->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that now only second MeshPartition is affected"), [this]()
				{
					const FString GroundTruthAssetA = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetA, MeshPartitionEditorComponent);

					const FString GroundTruthAssetB = TEXT("Boolean_Modifier_Affected_B");
					CompareToReferenceAsset(GroundTruthAssetB, MeshPartitionEditorComponent2);
				});
	}

	// UE-TC-22592
	TEST_METHOD(Verify_Boolean_Modifier_AffectedMeshPartition_Assignment_UndoRedo_RevertsAndReapplies)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Do(TEXT("Create second MeshPartition and move it"), [this]()
				{
					MeshPartition2 = TestUtils::CreateTestMesh(
						World, MeshWidth, MeshHeight, MeshWidthVertexCount, MeshHeightVertexCount);
					ASSERT_THAT(IsNotNull(MeshPartition2, TEXT("Failed to create second MeshPartition")));

					MeshPartitionEditorComponent2 = Cast<UMeshPartitionEditorComponent>(MeshPartition2->GetMeshPartitionComponent());
					ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent2, TEXT("Failed to get MeshPartitionEditorComponent")));

					const FVector NewLocation = { 1100., 0., 0. };
					MeshPartition2->SetActorLocation(NewLocation);
					MeshPartitionEditorComponent2->SetPauseTransformerPipeline(true);
				})
			.Then(TEXT("Create Boolean actor and modifier and assign it to MeshPartition"), [this]()
				{
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that first MeshPartition is affected by modifier"), [this]()
				{
					const FString GroundTruthAssetA = TEXT("Boolean_Modifier_Affected");
					CompareToReferenceAsset(GroundTruthAssetA, MeshPartitionEditorComponent);

					const FString GroundTruthAssetB = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetB, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Create transaction from changing affected MeshPartition"), [this]()
				{
					const FScopedTransaction ScopedTransaction(FText::FromString(TEXT("Change affected MeshPartition")));
					BooleanModifier->Modify();

					BooleanModifier->SetAffectedMeshPartition(MeshPartition2);

					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
					MeshPartitionEditorComponent->OnModifierAssigned();
					MeshPartitionEditorComponent2->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that neither MeshPartition is affected"), [this]()
				{
					const FString GroundTruthAssetA = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetA, MeshPartitionEditorComponent);

					const FString GroundTruthAssetB = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetB, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Undo transaction and update components"), [this]()
				{
					ASSERT_THAT(IsTrue(GEditor->UndoTransaction(), TEXT("Failed to undo transaction")));
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that first MeshPartition is affected after undo transaction"), [this]()
				{
					const FString GroundTruthAssetA = TEXT("Boolean_Modifier_Affected");
					CompareToReferenceAsset(GroundTruthAssetA, MeshPartitionEditorComponent);

					const FString GroundTruthAssetB = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetB, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Redo transaction and update components"), [this]()
				{
					ASSERT_THAT(IsTrue(GEditor->RedoTransaction(), TEXT("Failed to redo transaction")));
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !IsActiveBuild(BooleanModifier, MeshPartitionEditorComponent, MeshPartitionEditorComponent2);
				})
			.Then(TEXT("Verify that neither MeshPartition is affected after redo transaction"), [this]()
				{
					const FString GroundTruthAssetA = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetA, MeshPartitionEditorComponent);

					const FString GroundTruthAssetB = TEXT("Boolean_Modifier_Unaffected");
					CompareToReferenceAsset(GroundTruthAssetB, MeshPartitionEditorComponent2);
				});
	}

	// UE-TC-22594
	TEST_METHOD(Verify_Boolean_Modifier_SubPriority_Does_Not_Affect_Order_Across_Priority_Layer_Groups)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actors and modifiers"), [this]()
				{
					// Create BooleanModifier with Cylinder StaticMesh
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);

					// Create BooleanModifierB with Cone StaticMesh
					BooleanModifierActorB = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActorB, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifierB = CreateBooleanModifier(BooleanModifierActorB);
					ASSERT_THAT(IsNotNull(BooleanModifierB, TEXT("Failed to create BooleanModifier")));

					BooleanModifierB->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					ConeMesh = LoadObject<UStaticMesh>(nullptr, ConePath);
					ASSERT_THAT(IsNotNull(ConeMesh, TEXT("Failed to load ConeMesh")));

					BooleanModifierB->SetStaticMesh(ConeMesh);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Move Cone BooleanModifier along Z axis so that it intersects with Cylinder"), [this]()
				{
					const FVector NewLocationB = { 0.0, 0.0, 60.0 };
					BooleanModifierB->SetWorldLocation(NewLocationB);
					BooleanModifierB->PostEditComponentMove(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Set new MeshPartitionDefinition that has 2 priority layers"), [this]()
				{
					const FString PriorityLayerMeshPartitionDefinitionPath = TEXT("/MeshPartition/UnitTests/DataAssets/MeshPartitionPriorityLayerDefinition.MeshPartitionPriorityLayerDefinition");
					MeshPartitionDefinition = LoadObject<UMeshPartitionDefinition>(nullptr, PriorityLayerMeshPartitionDefinitionPath);
					ASSERT_THAT(IsNotNull(MeshPartitionDefinition, TEXT("Failed to load MeshPartitionDefinition")));

					MeshPartition->SetMeshPartitionDefinition(MeshPartitionDefinition);
				})
			.Then(TEXT("Assign priority layers to each Modifier"), [this]()
				{
					BooleanModifier->SetPriorityLayer(FName("GroupOne"));
					BooleanModifierB->SetPriorityLayer(FName("GroupTwo"));
				})
			.Then(TEXT("Change sub-priority values for both modifiers"), [this]()
				{
					BooleanModifier->SetPriority(2.0);
					BooleanModifierB->SetPriority(1.0);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Verify that changing sub-priority value in one group doesn't affect rendering order"), [this]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(BooleanModifier->GetPriority(), 2.0),
						TEXT("Failed to set sub-priority value for BooleanModifier")));
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(BooleanModifierB->GetPriority(), 1.0),
						TEXT("Failed to set sub-priority value for BooleanModifierB")));

					const FString GroundTruth = TEXT("Different_Priority_Layer_Unaffected");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Switch sub-priority for modifiers"), [this]()
				{
					BooleanModifierB->SetPriority(3.0);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Verify that changing sub-priority value in one group doesn't affect rendering order"), [this]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(BooleanModifierB->GetPriority(), 3.0),
						TEXT("Failed to set sub-priority value for BooleanModifierB")));

					const FString GroundTruth = TEXT("Different_Priority_Layer_Unaffected");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22594
	TEST_METHOD(Verify_Boolean_Modifier_SubPriority_UndoRedo_Manages_Value)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actors and modifiers"), [this]()
				{
					// Create BooleanModifier with Cylinder StaticMesh
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);

					// Create BooleanModifierB with Cone StaticMesh
					BooleanModifierActorB = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActorB, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifierB = CreateBooleanModifier(BooleanModifierActorB);
					ASSERT_THAT(IsNotNull(BooleanModifierB, TEXT("Failed to create BooleanModifier")));

					BooleanModifierB->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					ConeMesh = LoadObject<UStaticMesh>(nullptr, ConePath);
					ASSERT_THAT(IsNotNull(ConeMesh, TEXT("Failed to load ConeMesh")));

					BooleanModifierB->SetStaticMesh(ConeMesh);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Set new MeshPartitionDefinition that has 2 priority layers"), [this]()
				{
					const FString PriorityLayerMeshPartitionDefinitionPath = TEXT("/MeshPartition/UnitTests/DataAssets/MeshPartitionPriorityLayerDefinition.MeshPartitionPriorityLayerDefinition");
					MeshPartitionDefinition = LoadObject<UMeshPartitionDefinition>(nullptr, PriorityLayerMeshPartitionDefinitionPath);
					ASSERT_THAT(IsNotNull(MeshPartitionDefinition, TEXT("Failed to load MeshPartitionDefinition")));

					MeshPartition->SetMeshPartitionDefinition(MeshPartitionDefinition);
				})
			.Then(TEXT("Set sub-priority for BooleanModifier"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Set sub-priority")));

					BooleanModifier->Modify(true);
					BooleanModifier->SetPriority(2.0);
				})
			.Then(TEXT("Verify sub-priority value for BooleanModifier"), [this]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(BooleanModifier->GetPriority(), 2.0),
						TEXT("Failed to set sub-priority value for BooleanModifier")));
				})
			.Then(TEXT("Undo setting of sub-priority value"), [this]()
				{
					GEditor->UndoTransaction(true);
				})
			.Then(TEXT("Verify sub-priority value for BooleanModifier after undo transaction"), [this]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(BooleanModifier->GetPriority(), 0.0),
						TEXT("Failed to undo setting of sub-priority value for BooleanModifier")));
				})
			.Then(TEXT("Redo setting of sub-priority value"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Then(TEXT("Verify sub-priority value for BooleanModifier"), [this]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(BooleanModifier->GetPriority(), 2.0),
						TEXT("Failed to redo setting of sub-priority value for BooleanModifier")));
				});
	}

	// UE-TC-22602
	TEST_METHOD(Verify_Boolean_Modifier_ExpandBounds_Intersection_Behavior_Depends_On_BaseGrowth)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actors and modifiers"), [this]()
				{
					// Create BooleanModifier with Cylinder StaticMesh
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);

					// Create BooleanModifierB with Cylinder StaticMesh
					BooleanModifierActorB = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActorB, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifierB = CreateBooleanModifier(BooleanModifierActorB);
					ASSERT_THAT(IsNotNull(BooleanModifierB, TEXT("Failed to create BooleanModifier")));

					BooleanModifierB->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					BooleanModifierB->SetStaticMesh(CylinderMesh);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Move BooleanModifierB along Z axis"), [this]()
				{
					const FVector NewLocationB = { 0.0, 0.0, 450.0 };
					BooleanModifierB->SetWorldLocation(NewLocationB);
					BooleanModifierB->PostEditComponentMove(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Turn off BaseGrowth XYZ components"), [this]()
				{
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = false, .Y = false, .Z = false });
					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Then(TEXT("Increase Expand Operator Bounds to 1000"), [this]()
				{
					BooleanModifier->SetExpandOperatorBounds(FVector3d{ 100.0, 100.0, 1000.0 });
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Verify that intersected modifier is not affected while growth is set to false"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Unaffected_Z");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Turn on BaseGrowth Z component"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Set Base Growth")));
					BooleanModifier->Modify();
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = false, .Y = false, .Z = true });
					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Verify that intersected modifier is affected while growth is set to true"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Affected_Z");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22602
	TEST_METHOD(Verify_Boolean_Modifier_BaseGrowth_UndoRedo_Manages_Value)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actors and modifiers"), [this]()
				{
					// Create BooleanModifier with Cylinder StaticMesh
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);

					// Create BooleanModifierB with Cylinder StaticMesh
					BooleanModifierActorB = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActorB, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifierB = CreateBooleanModifier(BooleanModifierActorB);
					ASSERT_THAT(IsNotNull(BooleanModifierB, TEXT("Failed to create BooleanModifier")));

					BooleanModifierB->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					BooleanModifierB->SetStaticMesh(CylinderMesh);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Move BooleanModifierB along Z axis"), [this]()
				{
					const FVector NewLocationB = { 0.0, 0.0, 450.0 };
					BooleanModifierB->SetWorldLocation(NewLocationB);
					BooleanModifierB->PostEditComponentMove(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Turn off BaseGrowth XYZ components"), [this]()
				{
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = false, .Y = false, .Z = false });
					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Then(TEXT("Increase Expand Operator Bounds to 1000"), [this]()
				{
					BooleanModifier->SetExpandOperatorBounds(FVector3d{ 100.0, 100.0, 1000.0 });
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Verify that intersected modifier is not affected while growth is set to false"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Unaffected_Z");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Turn on BaseGrowth Z component"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Set Base Growth")));
					BooleanModifier->Modify();
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = false, .Y = false, .Z = true });
					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Verify that intersected modifier is affected while growth is set to true"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Affected_Z");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Undo SetBaseGrowth transaction"), [this]()
				{
					GEditor->UndoTransaction(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Verify that intersected modifier is not affected while growth is set to false after Undo"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Unaffected_Z");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Redo SetBaseGrowth transaction"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(BooleanModifier);
				})
			.Then(TEXT("Verify that intersected modifier is affected while growth is set to true after Redo"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Affected_Z");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				});
	}

	// UE-TC-22602
	TEST_METHOD(Verify_Boolean_Modifier_ExpandBounds_With_BaseGrowth_Affects_Other_Modifier_Across_Axes)
	{
		TestCommandBuilder
			.Do(TEXT("Create Boolean actors and modifiers"), [this]()
				{
					// Create BooleanModifier with Cylinder StaticMesh
					BooleanModifierActor = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActor, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifier = CreateBooleanModifier(BooleanModifierActor);
					ASSERT_THAT(IsNotNull(BooleanModifier, TEXT("Failed to create BooleanModifier")));

					BooleanModifier->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					CylinderMesh = LoadObject<UStaticMesh>(nullptr, CylinderPath);
					ASSERT_THAT(IsNotNull(CylinderMesh, TEXT("Failed to load CylinderMesh")));

					BooleanModifier->SetStaticMesh(CylinderMesh);

					// Create BooleanModifierB with Cylinder StaticMesh
					BooleanModifierActorB = CreateBooleanModifierActor();
					ASSERT_THAT(IsNotNull(BooleanModifierActorB, TEXT("Failed to create BooleanModifier actor")));

					BooleanModifierB = CreateBooleanModifier(BooleanModifierActorB);
					ASSERT_THAT(IsNotNull(BooleanModifierB, TEXT("Failed to create BooleanModifier")));

					BooleanModifierB->SetMeshSourceMode(EModifierMeshSourceMode::StaticMesh);

					BooleanModifierB->SetStaticMesh(CylinderMesh);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Move BooleanModifierB along Y axis outside of MeshPartition"), [this]()
				{
					const FVector NewLocationB = { 0.0, 1000.0, 0.0 };
					BooleanModifierB->SetWorldLocation(NewLocationB);
					BooleanModifierB->PostEditComponentMove(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Turn off BaseGrowth XYZ components"), [this]()
				{
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = false, .Y = false, .Z = false });

					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Then(TEXT("Increase Expand Operator Bounds to 3000"), [this]()
				{
					BooleanModifier->SetExpandOperatorBounds(FVector3d{ 100.0, 3000.0, 100.0 });
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Verify that intersected modifier is not affected while growth is set to false"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Unaffected_Y");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Turn on BaseGrowth Y component"), [this]()
				{
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = false, .Y = true, .Z = false });
					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Verify that intersected modifier is affected while growth is set to true"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Affected_Y");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Move BooleanModifierB along X axis outside of MeshPartition"), [this]()
				{
					const FVector NewLocationB = { 1000.0, 0.0, 0.0 };
					BooleanModifierB->SetWorldLocation(NewLocationB);
					BooleanModifierB->PostEditComponentMove(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Increase Expand Operator Bounds to 3000"), [this]()
				{
					BooleanModifier->SetExpandOperatorBounds(FVector3d{ 3000.0, 100.0, 100.0 });
				})
			.Then(TEXT("Turn off BaseGrowth XYZ components"), [this]()
				{
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = false, .Y = false, .Z = false });
					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Verify that intersected modifier is not affected while growth is set to false"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Unaffected_X");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				})
			.Then(TEXT("Turn on BaseGrowth X component"), [this]()
				{
					BooleanModifier->SetBaseGrowth(FBaseGrowth{ .X = true, .Y = false, .Z = false });
					BooleanModifier->OnChanged(BooleanModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Verify that intersected modifier is affected while growth is set to true"), [this]()
				{
					const FString GroundTruth = TEXT("ExpandBounds_Affected_X");
					CompareToReferenceAsset(GroundTruth, MeshPartitionEditorComponent);
				});
	}

	AFTER_EACH()
	{
		if (MeshPartitionEditorComponent)
		{
			MeshPartitionEditorComponent->SetPauseTransformerPipeline(false);
		}
		if (MeshPartitionEditorComponent2)
		{
			MeshPartitionEditorComponent2->SetPauseTransformerPipeline(false);
		}
		
		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

private:
	template <typename... Components>
	static bool IsActiveBuild(const UBooleanModifier* Modifier, Components*... Comps)
	{
		return (... && Comps->IsModifierParticipatingInActivePreviewSectionBuild(Modifier));
	}

	AActor* CreateBooleanModifierActor(const FActorSpawnParameters& InSpawnParams = FActorSpawnParameters()) const
	{
		AActor* Actor = World->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity, InSpawnParams);
		if (!IsValid(Actor))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateBooleanModifierActor: Failed to create BooleanModifier actor");
			return nullptr;
		}

		USceneComponent* SceneComponent = NewObject<USceneComponent>(Actor, TEXT("SceneComponent"), RF_Transactional);
		Actor->AddInstanceComponent(SceneComponent);
		Actor->SetRootComponent(SceneComponent);
		SceneComponent->RegisterComponent();

		Actor->AttachToActor(MeshPartition, FAttachmentTransformRules::KeepWorldTransform);

		return Actor;
	}

	UBooleanModifier* CreateBooleanModifier(AActor* ModifierActor, const FVector& ModifierWorldLocation = FVector::ZeroVector) const
	{
		if (!IsValid(ModifierActor))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateBooleanModifier: Actor is null or invalid");
			return nullptr;
		}

		if (!IsValid(MeshPartition))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateBooleanModifier: MeshPartition is null or invalid");
			return nullptr;
		}

		UBooleanModifier* Modifier = NewObject<UBooleanModifier>(
			ModifierActor,
			UBooleanModifier::StaticClass(),
			NAME_None,
			RF_Transactional
		);
		if (!Modifier)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateBooleanModifier: Failed to create UBooleanModifier object");
			return nullptr;
		}

		Modifier->Modify();

		Modifier->SetWorldLocation(ModifierWorldLocation);
		ModifierActor->AddInstanceComponent(Modifier);
		Modifier->SetAffectedMeshPartition(MeshPartition);
		if (USceneComponent* RootComp = ModifierActor->GetRootComponent())
		{
			Modifier->AttachToComponent(RootComp, FAttachmentTransformRules::KeepWorldTransform);
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Warning,
				"CreateBooleanModifier: Actor has no root component, component will not be attached");
		}

		Modifier->RegisterComponent();

		ModifierActor->Modify();

		return Modifier;
	}
	
	void CompareToReferenceAsset(const FString& TestName, const UMeshPartitionEditorComponent* MeshPartitionComponent)
	{
		const FString ReferencePackagePath = ReferencePath / TestName;
		const FString ReferenceAssetPath = ReferencePackagePath + "." + TestName;

		if (TestUtils::ShouldUpdateReferenceAssets())
		{
			[[maybe_unused]] const bool bSuccess =
				TestUtils::CreateOrUpdateReference(MeshPartitionComponent, ReferencePackagePath, {TestName}, TestName);
		}

		FString ErrorMessage;
		bool bValid = TestUtils::ValidatePreviewMeshAgainstGroundTruth(
			MeshPartitionComponent,
			{ ReferenceAssetPath },
			ErrorMessage);

		ASSERT_THAT(IsTrue(bValid, *ErrorMessage));
	}
};

#endif
#endif