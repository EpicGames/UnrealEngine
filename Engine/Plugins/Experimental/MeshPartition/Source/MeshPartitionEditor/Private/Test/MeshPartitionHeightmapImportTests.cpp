// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "CQTest.h"
#include "Test/MeshPartitionTestUtils.h"

#include "EditorWorldUtils.h"
#include "MeshPartitionHeightmapImporter.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "Interfaces/IPluginManager.h"

#include "Engine/StaticMesh.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

/**
 * Tests for MeshPartition Heightmap Import
 */
TEST_CLASS_WITH_FLAGS(HeightmapImport, "MeshPartition", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
private:
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	// Core test objects
	UWorld* World = nullptr;
	AMeshPartition* MeshPartition = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;

	// Test data
	TArray<TObjectPtr<UStaticMesh>> GroundTruthBaseMeshes;
	TArray<UModifierComponent*> Modifiers;

	// Test constants
	static constexpr int32 MeshResolution = 256;
	static constexpr int32 SectionsResolutionX = 2;
	static constexpr int32 SectionsResolutionY = 2;
	static constexpr int32 ExpectedModifierCount = 4;
	static constexpr int32 ExpectedPreviewSectionCount = 1;
	static constexpr double MeshSize = 256.0;

	static inline const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
	static inline const FString TestHeightmapFilename = TEXT("Content/UnitTests/TestHeightmap.png");

	// Ground truth mesh paths
	static inline const TArray<FString> GroundTruthMeshPaths = {
		TEXT("/MeshPartition/UnitTests/HeightmapImport/x0y0.x0y0"),
		TEXT("/MeshPartition/UnitTests/HeightmapImport/x1y0.x1y0"),
		TEXT("/MeshPartition/UnitTests/HeightmapImport/x0y1.x0y1"),
		TEXT("/MeshPartition/UnitTests/HeightmapImport/x1y1.x1y1")
	};

public:
	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);

		// Spawn world
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);
		ASSERT_THAT(IsNotNull(ScopedEditorWorld.Get(), TEXT("Failed to create scoped editor world")));

		World = ScopedEditorWorld->GetWorld();
		ASSERT_THAT(IsNotNull(World, TEXT("Failed to create valid test world")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));
	}

	/**
	 * Verifies that heightmap import correctly generates the expected MeshPartition structure
	 * with proper modifiers and preview sections that match ground truth meshes.
	 */
	TEST_METHOD(Verify_HeightmapImport_Creates_Expected_Modifiers_And_PreviewSections)
	{
		TestCommandBuilder
			.Do(TEXT("Import heightmap using MeshPartitionHeightmapImporter"), [this]()
				{
					const FString PluginDirectory = IPluginManager::Get().FindPlugin(TEXT("MeshPartition"))->GetBaseDir();

					FHeightmapImportParams ImportParams;
					ImportParams.World = World;
					ImportParams.HeightmapFilename = FPaths::Combine(PluginDirectory, TestHeightmapFilename);
					ImportParams.MeshResolution = FInt32Point(MeshResolution);
					ImportParams.MeshSize = FVector(MeshSize);
					ImportParams.SectionsResolution = { SectionsResolutionX, SectionsResolutionY };
					ImportParams.bSaveAndUnload = false;
					ImportParams.LocationVolumesResolution = FInt32Point(0);

					// Verify heightmap file exists
					ASSERT_THAT(IsTrue(FPaths::FileExists(ImportParams.HeightmapFilename),
						*FString::Printf(TEXT("Heightmap file does not exist: %s"), *ImportParams.HeightmapFilename)));

					// Import Heightmap
					FHeightmapImporter Importer(ImportParams);

					FPackageSourceControlHelper* SourceControlHelper = nullptr;
					Importer.Import(SourceControlHelper);

					MeshPartition = Importer.GetMegaMesh();
					ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to get MeshPartition from imported heightmap")));
				})
			.Then(TEXT("Build preview sections and configure editor component"), [this]()
				{
					MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
					ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent,
						TEXT("Failed to get MeshPartitionEditorComponent from imported MeshPartition")));

					// Pausing Transformer pipeline during the test to avoid potential race condition
					MeshPartitionEditorComponent->SetPauseTransformerPipeline(true);

					// Build preview sections with infinite bounds
					TArray InfiniteBounds = {
						FBox(FVector3d(-HALF_WORLD_MAX, -HALF_WORLD_MAX, -HALF_WORLD_MAX),
							FVector3d(HALF_WORLD_MAX, HALF_WORLD_MAX, HALF_WORLD_MAX))
					};

					MeshPartitionEditorComponent->BuildMegaMeshPreviewSections(InfiniteBounds);
					MeshPartitionEditorComponent->Update();
				})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate expected number of base modifiers were created"), [this]()
				{
					// Get all base modifiers
					Modifiers = MeshPartitionEditorComponent->GetModifiersFiltered([](const UModifierComponent* Modifier)
					{
						return Modifier->IsBase() && Cast<UMeshProviderModifier>(Modifier);
					});

					ASSERT_THAT(AreEqual(Modifiers.Num(), ExpectedModifierCount,
						*FString::Printf(TEXT("Expected number of modifiers to be %d"), ExpectedModifierCount)));

					// Validate each modifier is valid
					for (int32 Index = 0; Index < Modifiers.Num(); ++Index)
					{
						ASSERT_THAT(IsNotNull(Modifiers[Index],
							*FString::Printf(TEXT("Base modifier at index %d should not be null"), Index)));
					}
				})
			.Then(TEXT("Load ground truth meshes for comparison"), [this]()
				{
					// Convert string paths to soft object pointers
					TArray<TSoftObjectPtr<UStaticMesh>> MeshSoftPaths;
					for (const FString& PathString : GroundTruthMeshPaths)
					{
						MeshSoftPaths.Add(TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(PathString)));
					}

					const bool bSuccessfullyLoadedMeshes = TestUtils::LoadMeshes(
						GroundTruthBaseMeshes, MeshSoftPaths);

					ASSERT_THAT(IsTrue(bSuccessfullyLoadedMeshes, TEXT("Failed to load the ground meshes")));
					ASSERT_THAT(AreEqual(ExpectedModifierCount, GroundTruthBaseMeshes.Num(),
						TEXT("Ground truth mesh count should match expected modifier count")));
				})
			.Then(TEXT("Compare modifier meshes against ground truth and validate preview sections"), [this]()
				{
					for (int32 SectionIndex = 0; SectionIndex < Modifiers.Num(); ++SectionIndex)
					{
						const UModifierComponent* Modifier = Modifiers[SectionIndex];
						const UStaticMesh* GroundTruthMesh = GroundTruthBaseMeshes[SectionIndex];
						const FDynamicMesh3* GeneratedMesh = Cast<UMeshProviderModifier>(Modifier)->GetMesh();

						ASSERT_THAT(IsNotNull(GroundTruthMesh,
							*FString::Printf(TEXT("Ground truth mesh at index %d should not be null"), SectionIndex)));
						ASSERT_THAT(IsNotNull(GeneratedMesh,
							*FString::Printf(TEXT("Generated mesh at index %d should not be null"), SectionIndex)));

						const bool bAreMeshesEqual = TestUtils::CompareMesh(GroundTruthMesh, GeneratedMesh);
						ASSERT_THAT(IsTrue(bAreMeshesEqual,
							*FString::Printf(TEXT("Generated mesh at index %d does not match ground truth."), SectionIndex)));
					}

					// Validate preview sections were created
					int32 ActualPreviewSectionCount = 0;
					MeshPartitionEditorComponent->ForAllPreviewSections([&ActualPreviewSectionCount](const APreviewSection* PreviewSection)
						{
							if (IsValid(PreviewSection))
							{
								ActualPreviewSectionCount++;
							}
							return true;
						});

					ASSERT_THAT(AreEqual(ActualPreviewSectionCount, ExpectedPreviewSectionCount,
						*FString::Printf(TEXT("Expected number of preview sections to be %d"), ExpectedPreviewSectionCount)));
				});
	}

	AFTER_EACH()
	{
		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
