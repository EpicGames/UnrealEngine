// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "Test/MeshPartitionTestUtils.h"
#include "EditorWorldUtils.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "Editor.h"
#include "Engine/World.h"
#include "MeshPartitionTestModifier.h"
#include "CQTest.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

/**
 * Tests for MeshPartition Modifier lifecycle initialization and cleanup
 */
TEST_CLASS_WITH_FLAGS(Initialization, "MeshPartition.Modifier", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	UWorld* World = nullptr;
	AActor* TestActor = nullptr;
	UE::MeshPartition::AMeshPartition* MeshPartition = nullptr;
	UE::MeshPartition::UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;
	UE::MeshPartition::UTestModifierComponent* TestModifier = nullptr;

	// Lifecycle tracking
	TSharedPtr<int32> InitCount;
	TSharedPtr<int32> UninitCount;

	// Fixed delta time ticking world
	const float TickDelta = 0.016f;

	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);

		// Spawn world
		const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);

		World = ScopedEditorWorld->GetWorld();
		ASSERT_THAT(IsNotNull(World, TEXT("Failed to create valid test world.")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = UE::MeshPartition::TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));
	}

	/**
	 * Verify modifier init/uninit counts update correctly during lifecycle events
	 */
	TEST_METHOD(Verify_MeshPartition_Modifier_Lifecycle_Updates_InitCounts)
	{
		TestCommandBuilder
			.Do(TEXT("Create test mesh and modifier"), [this]()
			{
				MeshPartition = UE::MeshPartition::TestUtils::CreateTestMesh(World, 20000, 20000, 100, 100);
				ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to create MeshPartition.")));

				MeshPartitionEditorComponent = Cast<UE::MeshPartition::UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
				ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent, TEXT("MeshPartitionEditorComponent is null.")));

				TestActor = World->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity);
				ASSERT_THAT(IsNotNull(TestActor, TEXT("Failed to spawn a test actor.")));

				TestModifier = NewObject<UE::MeshPartition::UTestModifierComponent>(TestActor, UE::MeshPartition::UTestModifierComponent::StaticClass());
				ASSERT_THAT(IsNotNull(TestModifier, TEXT("Failed to create a test modifier.")));

				TestModifier->SetAffectedMeshPartition(MeshPartition);
				TestActor->AddInstanceComponent(TestModifier);
				TestModifier->RegisterComponent();

				InitCount = TestModifier->GetInitCount();
				UninitCount = TestModifier->GetUninitCount();

				ASSERT_THAT(AreEqual(0, *InitCount, TEXT("Modifier should not be initialized yet.")));
				ASSERT_THAT(AreEqual(0, *UninitCount, TEXT("Modifier should not be uninitialized yet.")));

				MeshPartitionEditorComponent->OnModifierAssigned();
			})
			.Until(TEXT("Wait for preview section build"), [this]()
			{
				return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(TestModifier);
			})
			.Then(TEXT("Verify initial assignment"), [this]()
			{
				ASSERT_THAT(AreEqual(1, *InitCount, TEXT("Test modifier assigned once, modifier should have been initialized.")));
				ASSERT_THAT(AreEqual(0, *UninitCount, TEXT("Test modifier assigned once, modifier should not be uninitialized yet.")));
				TestModifier->ResetCounts();

				MeshPartitionEditorComponent->OnModifierAssigned();
			})
			.Until(TEXT("Wait for preview section build"), [this]()
			{
				return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(TestModifier);
			})
			.Then(TEXT("Verify that there are no re-initializations, unassign modifier"), [this]()
			{
				ASSERT_THAT(AreEqual(0, *InitCount, TEXT("Second rebuild, test modifier should not be re-initialized.")));
				ASSERT_THAT(AreEqual(0, *UninitCount, TEXT("Second rebuild, test modifier should not be uninitialized.")));
				TestModifier->ResetCounts();

				TestModifier->SetAffectedMeshPartition(nullptr);
				MeshPartitionEditorComponent->OnModifierAssigned();
			})
			.Until(TEXT("Wait for preview section build"), [this]()
			{
				return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(TestModifier);
			})
			.Then(TEXT("Verify un-initialization, re-initialize"), [this]()
			{
				ASSERT_THAT(AreEqual(0, *InitCount, TEXT("Unassigned, test modifier should not be initialized.")));
				ASSERT_THAT(AreEqual(1, *UninitCount, TEXT("Unassigned, test modifier should be uninitialized.")));
				TestModifier->ResetCounts();

				TestModifier->SetAffectedMeshPartition(MeshPartition);
				MeshPartitionEditorComponent->OnModifierAssigned();
			})
			.Until(TEXT("Wait for preview section build"), [this]()
			{
				return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(TestModifier);
			})
			.Then(TEXT("Verify that there has been a re-initialization"), [this]()
			{
				ASSERT_THAT(AreEqual(1, *InitCount, TEXT("Reassigned, test modifier should be initialized again.")));
				ASSERT_THAT(AreEqual(0, *UninitCount, TEXT("Reassigned, test modifier should not be uninitialized.")));
				TestModifier->ResetCounts();

				World->DestroyActor(TestActor);
				MeshPartitionEditorComponent->OnModifierAssigned();
			})
			.Until(TEXT("Wait for preview section build"), [this]()
			{
				return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(TestModifier);
			})
			.Then(TEXT("Verify un-initialization after DestroyActor"), [this]()
			{
				ASSERT_THAT(AreEqual(0, *InitCount, TEXT("Actor destroyed, test modifier should not be initialized.")));
				ASSERT_THAT(AreEqual(1, *UninitCount, TEXT("Actor destroyed, test modifier should be uninitialized.")));
			});
	}

	AFTER_EACH()
	{
		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS