// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS
#if WITH_EDITOR

#include "CQTest.h"
#include "Test/MeshPartitionTestUtils.h"
#include "EditorWorldUtils.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "Engine/World.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

/**
 * Tests for MeshPartition Patch modifier
 */
TEST_CLASS_WITH_FLAGS(Patch, "MeshPartition.Modifier", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
private:
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	// Core test objects
	UWorld* World = nullptr;
	AMeshPartition* MeshPartition = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;

	// Test constants
	static constexpr int32 MeshWidth = 20000;
	static constexpr int32 MeshHeight = 20000;
	static constexpr int32 MeshWidthVertexCount = 100;
	static constexpr int32 MeshHeightVertexCount = 100;
	static constexpr int32 ExpectedPreviewSectionCount = 1;
	static inline const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
	const FString ReferencePath = TEXT("/MeshPartition/UnitTests/PatchModifierTests");

public:
	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);

		// Spawn world
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);
		ASSERT_THAT(IsNotNull(ScopedEditorWorld.Get(), TEXT("Failed to create scoped editor world")));

		World = ScopedEditorWorld->GetWorld();
		ASSERT_THAT(IsNotNull(World, TEXT("Failed to create valid test world")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));
	}

	/**
	 * Verifies that when two patch modifiers with overlapping areas are applied, 
	 * the modifier with higher priority takes precedence over the lower one.
	 */
	TEST_METHOD(Verify_PatchModifier_Priority_System_Applies_Higher_Priority_First)
	{
		TestCommandBuilder
			.Do(TEXT("Create MeshPartition with test dimensions"), [this]()
			{
				// Create MeshPartition
				MeshPartition = TestUtils::CreateTestMesh(
					World, MeshWidth, MeshHeight, MeshWidthVertexCount, MeshHeightVertexCount);
				ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to create MeshPartition")));

				MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
				ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent,
					TEXT("Failed to get MeshPartitionEditorComponent from MeshPartition")));
			})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Create overlapping patch modifiers with different priorities"), [this]()
			{
				// Create high priority patch modifier
				{
					const FVector ModifierWorldLocation = FVector(0.0, 2000.0, 1500.0);
					constexpr double HighPriority = 1.0;
					AActor* HighPriorityPatchActor = World->SpawnActor<AActor>(AActor::StaticClass(),
						FTransform::Identity);
					ASSERT_THAT(IsNotNull(HighPriorityPatchActor, TEXT("Failed to spawn high priority MeshPartitionModifier")));

					TestUtils::CreatePatchModifier(HighPriorityPatchActor, MeshPartition, ModifierWorldLocation, HighPriority);
				}
				// Create low priority patch modifier
				{
					const FVector ModifierWorldLocation = FVector(0.0, 1000.0, 2500.0);
					AActor* LowPriorityPatchActor = World->SpawnActor<AActor>(AActor::StaticClass(),
						FTransform::Identity);
					ASSERT_THAT(IsNotNull(LowPriorityPatchActor, TEXT("Failed to spawn low priority MeshPartitionModifier")));

					TestUtils::CreatePatchModifier(LowPriorityPatchActor, MeshPartition, ModifierWorldLocation);
				}

				// Notify the editor component that modifiers have been assigned
				MeshPartitionEditorComponent->OnModifierAssigned();
			})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
			{
				return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
			})
			.Then(TEXT("Validate preview section matches expected mesh"), [this]()
			{
				const APreviewSection* PreviewSection = nullptr;
				int32 PreviewSectionCount = 0;

				// Count and capture preview sections
				MeshPartitionEditorComponent->ForAllPreviewSections([&](const APreviewSection* InPreviewSection)
				{
					PreviewSection = InPreviewSection;
					++PreviewSectionCount;
					return true;
				});

				// Validate preview section count
				ASSERT_THAT(AreEqual(ExpectedPreviewSectionCount, PreviewSectionCount,
					*FString::Printf(TEXT("PreviewSectionCount should be %d"), ExpectedPreviewSectionCount)));
				ASSERT_THAT(IsNotNull(PreviewSection, TEXT("MeshPartition PreviewSection is null")));

				// Validate preview mesh matches ground truth
				const FString GroundTruthAsset = TEXT("Verify_PatchModifier_Priority_System_Applies_Higher_Priority_First_GroundTruth");
				CompareToReferenceAsset(GroundTruthAsset);
			});
	}

	AFTER_EACH()
	{
		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

private:
	void CompareToReferenceAsset(const FString& TestName)
	{
		const FString ReferencePackagePath = ReferencePath / TestName;
		const FString ReferenceAssetPath = ReferencePackagePath + "." + TestName;

		if (TestUtils::ShouldUpdateReferenceAssets())
		{
			[[maybe_unused]] const bool bSuccess =
				TestUtils::CreateOrUpdateReference(MeshPartitionEditorComponent, ReferencePackagePath, { TestName }, TestName);
		}

		FString ErrorMessage;
		bool bValid = TestUtils::ValidatePreviewMeshAgainstGroundTruth(
			MeshPartitionEditorComponent,
			{ ReferenceAssetPath },
			ErrorMessage);

		ASSERT_THAT(IsTrue(bValid, *ErrorMessage));
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
