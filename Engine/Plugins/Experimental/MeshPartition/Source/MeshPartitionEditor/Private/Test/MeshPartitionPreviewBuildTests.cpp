// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#if WITH_DEV_AUTOMATION_TESTS

#include "Test/MeshPartitionTestUtils.h"
#include "CQTest.h"
#include "EditorWorldUtils.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionPreviewSection.h"
#include "Engine/StaticMesh.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

TEST_CLASS_WITH_FLAGS(PreviewSection, "MeshPartition", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	UWorld* World = nullptr;
	UE::MeshPartition::AMeshPartition* MeshPartition = nullptr;
	UE::MeshPartition::UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;
	TArray<TObjectPtr<UStaticMesh>> GroundTruthPreviewMeshes;

	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	BEFORE_EACH()
	{
		using namespace UE::MeshPartition;

		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);

		// Spawn world
		const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);

		World = ScopedEditorWorld->GetWorld();
		ASSERT_THAT(IsNotNull(World, TEXT("Failed to create valid test world.")));

		MeshPartition = World->SpawnActor<AMeshPartition>(AMeshPartition::StaticClass(), FTransform::Identity);
		ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to spawn MeshPartition actor.")));

		MeshPartitionEditorComponent = NewObject<UMeshPartitionEditorComponent>(
			MeshPartition, UMeshPartitionEditorComponent::StaticClass(), TEXT("MeshPartitionEditorComponent"));
		ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent, TEXT("Failed to create editor component for MeshPartition.")));

		// Set up component hierarchy
		MeshPartition->SetMeshPartitionComponent(MeshPartitionEditorComponent);
		MeshPartitionEditorComponent->RegisterComponent();

		// Pausing Transformer pipeline during the test to avoid potential race condition
		MeshPartitionEditorComponent->SetPauseTransformerPipeline(true);

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));
	}

	/**
	 * Verifies that MeshPartition correctly creates preview sections based on complexity settings,
	 * merging low-complexity sections and separating high-complexity ones
	 */
	TEST_METHOD(Verify_PreviewSection_Complexity_Threshold_Behavior)
	{
		using namespace UE::MeshPartition;

		TestCommandBuilder
			.Do(TEXT("Create test geometry with varying complexity"), [this]()
			{
				// Make a new transient definition for this test to override the max section complexity for preview section builds.
				UMeshPartitionDefinition* Definition = NewObject<UMeshPartitionDefinition>(GetTransientPackage());
				UE::MeshPartition::FCommonBuildVariant Variant = Definition->GetPreviewSectionBuildVariant();
				Variant.MaxSectionComplexity = 50;
				Definition->SetPreviewSectionBuildVariant(Variant);
				MeshPartition->SetMeshPartitionDefinition(Definition);

				TArray<FBox> Bounds;
				// Create two base modifiers with two triangle quads
				// We expect these two to get merged into a single preview section based on our complexity settings.
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 100, 100, 2, 2, Bounds);
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 100, 100, 2, 2, Bounds);
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 100, 100, 100, 100, Bounds);

				MeshPartitionEditorComponent->BuildMegaMeshPreviewSections(Bounds);
			})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
			{
				return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
			})
			.Then(TEXT("Verify correct number of preview sections created"), [this]()
			{
				// Count the actual number of preview sections
				int32 PreviewSectionCount = 0;

				MeshPartitionEditorComponent->ForAllPreviewSections([&](UE::MeshPartition::APreviewSection* InPreviewSection)
				{
					++PreviewSectionCount;
					return true;
				});

				// We expect 2 sections
				constexpr int32 ExpectedCount = 2;
				ASSERT_THAT(AreEqual(ExpectedCount, PreviewSectionCount,
					FString::Printf(TEXT("Expected preview count to be %d, found %d."), ExpectedCount, PreviewSectionCount)));
			});
	}

	/**
	 * Verifies that MeshPartition correctly applies transforms when creating preview sections,
	 * ensuring the generated mesh matches the expected ground truth
	 */
	TEST_METHOD(Verify_PreviewSection_Preserves_Transforms)
	{
		using namespace UE::MeshPartition;

		TestCommandBuilder
			.Do(TEXT("Create transformed mesh and build preview section"), [this]()
			{
				// Load reference mesh for comparison
				const bool bSuccessfullyLoadedMeshes = UE::MeshPartition::TestUtils::LoadMeshes(GroundTruthPreviewMeshes, {
					TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(TEXT("/MeshPartition/UnitTests/PreviewSectionTransforms_Test.PreviewSectionTransforms_Test")))
					});
				ASSERT_THAT(IsTrue(bSuccessfullyLoadedMeshes, TEXT("Failed to load GroundTruthPreviewMeshes.")));

				// Create modifier with specific transform
				TArray<FBox> Bounds;
				const FTransform Transform(FVector(1000., 2000., 3000.));
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 100, 100, 100, 100, Bounds, Transform);

				// Build preview sections
				MeshPartitionEditorComponent->BuildMegaMeshPreviewSections({Bounds});
			})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
			{
				return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
			})
			.Then(TEXT("Verify preview section mesh matches ground truth"), [this]()
			{
				// Collect preview sections
				TArray<UE::MeshPartition::APreviewSection*> PreviewSections;
				MeshPartitionEditorComponent->ForAllPreviewSections([&](UE::MeshPartition::APreviewSection* InPreviewSection)
				{
					PreviewSections.Add(InPreviewSection);
					return true;
				});

				// Verify section count
				constexpr int32 ExpectedCount = 1;
				ASSERT_THAT(AreEqual(ExpectedCount, PreviewSections.Num(),
					FString::Printf(TEXT("Expected preview count to be %d, found %d."), ExpectedCount, PreviewSections.Num())));

				// Compare with ground truth
				const bool bAreMeshesEqual = UE::MeshPartition::TestUtils::CompareMesh(GroundTruthPreviewMeshes[0], PreviewSections[0]->GetPreviewMesh().Get());
				ASSERT_THAT(IsTrue(bAreMeshesEqual, TEXT("Preview mesh doesn't match expected ground truth with transforms applied.")));
			});
	}

	/**
	 * Verifies that MeshPartition correctly merges overlapping geometry into a single preview section
	 */
	TEST_METHOD(Verify_PreviewSection_Merges_Overlapping_Geometry)
	{
		using namespace UE::MeshPartition;

		TestCommandBuilder
			.Do(TEXT("Create overlapping modifiers and build preview section"), [this]()
			{
				TArray<FBox> Bounds;

				// Create two base modifiers, a box on top of a plane, we expect them to be merged into a single preview section.
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 10000, 10000, 2, 2, Bounds);
				{
					UE::Geometry::FMinimalBoxMeshGenerator BoxGen;
					BoxGen.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), 1000.0 * FVector3d::One());

					const AActor* BaseModifier = MeshPartitionEditorComponent->SpawnBaseModifier(
						FDynamicMesh3(&BoxGen.Generate()),
						{},
						FTransform::Identity);

					Bounds.Emplace(BaseModifier->GetComponentsBoundingBox(/*bNonColliding = */ true, /* bIncludeChildActors = */ false));
				}

				// Build preview sections
				MeshPartitionEditorComponent->BuildMegaMeshPreviewSections(Bounds);
			})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
			{
				return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
			})
			.Then(TEXT("Verify overlapping geometry was merged into a single section"), [this]()
			{
				int32 PreviewSectionCount = 0;
				MeshPartitionEditorComponent->ForAllPreviewSections([&](UE::MeshPartition::APreviewSection* InPreviewSection)
				{
					++PreviewSectionCount;
					return true;
				});

				// We expect that two of the base modifiers got merged into a single preview section due to overlapping.
				constexpr int32 ExpectedCount = 1;
				ASSERT_THAT(AreEqual(ExpectedCount, PreviewSectionCount,
					FString::Printf(TEXT("Expected preview count to be %d, found %d."), ExpectedCount, PreviewSectionCount)));
			});
	}

	/**
	 * Verifies that MeshPartition correctly maintains separate preview sections for non-overlapping geometry
	 */
	TEST_METHOD(Verify_PreviewSection_Preserves_NonOnverlapping_Geometry)
	{
		using namespace UE::MeshPartition;

		TestCommandBuilder
			.Do(TEXT("Create non-overlapping modifiers and build preview sections"), [this]()
			{
				TArray<FBox> Bounds;

				// Create two base modifiers, a box on top of a plane, we expect them to NOT be merged into a single preview section since they are not touching.
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 10000, 10000, 2, 2, Bounds);
				{
					UE::Geometry::FMinimalBoxMeshGenerator BoxGen;
					BoxGen.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), 1000.0 * FVector3d::One());
					const AActor* BaseModifier = MeshPartitionEditorComponent->SpawnBaseModifier(
						FDynamicMesh3(&BoxGen.Generate()),
						{},
						FTransform(FVector3d(0.f, 0.f, 2000.f)));
					Bounds.Emplace(BaseModifier->GetComponentsBoundingBox(/*bNonColliding = */ true, /* bIncludeChildActors = */ false));
				}

				// Build preview sections
				MeshPartitionEditorComponent->BuildMegaMeshPreviewSections(Bounds);
			})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
			{
				return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
			})
			.Then(TEXT("Verify overlapping geometry was merged into a single section"), [this]()
			{
				int32 PreviewSectionCount = 0;
				MeshPartitionEditorComponent->ForAllPreviewSections([&](UE::MeshPartition::APreviewSection* InPreviewSection)
				{
					++PreviewSectionCount;
					return true;
				});

				// We expect that two of the base modifiers weren't merged into a single preview section due to not overlapping.
				constexpr int32 ExpectedCount = 2;
				ASSERT_THAT(AreEqual(ExpectedCount, PreviewSectionCount,
					FString::Printf(TEXT("Expected preview count to be %d, found %d."), ExpectedCount, PreviewSectionCount)));
			});
	}

	/**
	 * Verifies that MeshPartition correctly handles aggregated bounds from multiple modifiers
	 */
	TEST_METHOD(Verify_PreviewSection_Handles_Aggregated_Bounds)
	{
		using namespace UE::MeshPartition;

		TestCommandBuilder
			.Do(TEXT("Create multiple modifiers with different orientations"), [this]()
			{
				TArray<FBox> Bounds;

				// Create a base plane at the origin
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 10000, 10000, 2, 2, Bounds);

				// Create a second plane rotated 90 degrees around Z and offset
				FTransform TransformA(FRotator3d(0.f, 0.f, 90.f));
				TransformA.SetLocation(FVector3d(0.f, -5000.f, 5000.f));
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 10000, 10000, 2, 2, Bounds, TransformA);

				// Create a third plane rotated 90 degrees around Z and 180 degrees around Y and offset
				FTransform TransformB(FRotator3d(0.f, 180.f, 90.f));
				TransformB.SetLocation(FVector3d(0.f, 5000.f, 5000.f));
				UE::MeshPartition::TestUtils::CreateRectangleModifier(MeshPartitionEditorComponent, 10000, 10000, 2, 2, Bounds, TransformB);

				{
					UE::Geometry::FMinimalBoxMeshGenerator BoxGen;
					BoxGen.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), 1000.0 * FVector3d::One());
					AActor* BaseModifier = MeshPartitionEditorComponent->SpawnBaseModifier(
						FDynamicMesh3(&BoxGen.Generate()),
						{},
						FTransform(FVector3d(0.f, 0.f, 5000.f)));
					Bounds.Emplace(BaseModifier->GetComponentsBoundingBox(/*bNonColliding = */ true, /* bIncludeChildActors = */ false));
				}

				// Build preview sections
				MeshPartitionEditorComponent->BuildMegaMeshPreviewSections(Bounds);
			})
			.Until(TEXT("Wait for preview section build to complete"), [this]()
			{
				return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
			})
			.Then(TEXT("Verify correct preview section grouping"), [this]()
			{
				int32 PreviewSectionCount = 0;
				MeshPartitionEditorComponent->ForAllPreviewSections([&](UE::MeshPartition::APreviewSection* InPreviewSection)
				{
					++PreviewSectionCount;
					return true;
				});

				// We expect that two elevated planes will be grouped together, making total preview section to be 2
				constexpr int32 ExpectedCount = 2;
				ASSERT_THAT(AreEqual(ExpectedCount, PreviewSectionCount,
					FString::Printf(TEXT("Expected preview count to be %d, found %d."), ExpectedCount, PreviewSectionCount)));
			});
	}

	AFTER_EACH()
	{
		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}
};

#endif // WITH_EDITOR
#endif // WITH_DEV_AUTOMATION_TESTS
