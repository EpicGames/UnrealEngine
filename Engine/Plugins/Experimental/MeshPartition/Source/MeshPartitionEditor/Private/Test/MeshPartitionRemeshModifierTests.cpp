// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#if WITH_DEV_AUTOMATION_TESTS

#include "CQTest.h"
#include "Editor.h"
#include "MeshPartition.h"
#include "MeshPartitionHeightmapImporter.h"
#include "Interfaces/IPluginManager.h"
#include "Modifiers/MeshPartitionRemeshModifier.h"
#include "Test/MeshPartitionTestUtils.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

/**
 * Tests for MeshTerrain Remesh Modifier
 */
TEST_CLASS_WITH_FLAGS(Remesh, "MeshPartition.Modifier", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test Environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;

	UWorld* World = nullptr;
	AMeshPartition* MeshPartition = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionComponent = nullptr;
	AActor* RemeshModifierActor = nullptr;
	URemeshModifier* RemeshModifier = nullptr;

	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	// Test configuration
	const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
	const FString ReferencePath = TEXT("/MeshPartition/UnitTests/RemeshModifierTests");
	const FString HeightmapLocation = TEXT("Content/UnitTests/TestHeightmap.png");
	const FVector3d UnscaledCoverage = { 100., 100., 300. };
	const FName ComponentTagName = TEXT("RemeshComponent");

	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);


#if AUTOMATED_TESTING_EXECUTE_IN_EDITOR_WORLD
		World = GEditor->GetEditorWorldContext().World();
#else
		// Spawn world
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);
		ASSERT_THAT(IsNotNull(ScopedEditorWorld.Get(), TEXT("Failed to create scoped editor world")));

		World = ScopedEditorWorld->GetWorld();
#endif
		ASSERT_THAT(IsNotNull(World, TEXT("World is not valid")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));

		// Setup MeshPartition
		MeshPartition = ImportHeightmap();
		ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to import heightmap into mesh terrain.")));

		MeshPartitionComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
		ASSERT_THAT(IsNotNull(MeshPartitionComponent, TEXT("Failed to get MeshPartitionEditorComponent")));

		// Setting default MeshPartitionDefinition
		const FString TestAutomationDefaultDefinitionPath = TEXT("/MeshPartition/UnitTests/DataAssets/MPD_AutomationTestDefault.MPD_AutomationTestDefault");
		UMeshPartitionDefinition* MeshPartitionDefinition = LoadObject<UMeshPartitionDefinition>(nullptr, TestAutomationDefaultDefinitionPath);
		if (!MeshPartitionDefinition)
		{
			UE_LOGF(LogMegaMeshEditor, Warning,
				"Failed to load MeshPartitionDefinition (%ls)", *TestAutomationDefaultDefinitionPath);
		}
		MeshPartition->SetMeshPartitionDefinition(MeshPartitionDefinition);

		// Pausing Transformer pipeline during the test to avoid potential race condition
		MeshPartitionComponent->SetPauseTransformerPipeline(true);

		// Create RemeshModifierActor
		const FActorSpawnParameters SpawnParams;
		RemeshModifierActor = World->SpawnActor<AActor>(AActor::StaticClass(),
			FTransform::Identity, SpawnParams);
		USceneComponent* SceneComponent = NewObject<USceneComponent>(RemeshModifierActor, TEXT("SceneComponent"));
		RemeshModifierActor->SetRootComponent(SceneComponent);
		SceneComponent->RegisterComponent();

		// Create Remesh modifier and assign it to RemeshModifierActor as component
		RemeshModifier = TestUtils::CreateRemeshModifier(
			RemeshModifierActor,
			MeshPartition, 
			FVector::ZeroVector,
			UnscaledCoverage
		);
		RemeshModifier->SetTargetEdgeLength(10);

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));
	}

	TEST_METHOD(Verify_RemeshModifier_Can_Be_Added)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Can_Be_Added_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_RemeshModifier_Translation_Functions_Correctly)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Apply translation to the actor"), [this]()
				{
					const FVector NewActorLocation = { 40., 0., 0. };
					RemeshModifierActor->SetActorLocation(NewActorLocation);
					RemeshModifier->OnChanged(RemeshModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Translation_Functions_Correctly_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_RemeshModifier_Rotation_Functions_Correctly)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Apply rotation to the actor"), [this]()
				{
					const FRotator NewActorRotation = { 20., 0., 0. };
					RemeshModifierActor->SetActorRotation(NewActorRotation);
					RemeshModifier->OnChanged(RemeshModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Rotation_Functions_Correctly_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_RemeshModifier_Scaling_Functions_Correctly)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Apply scaling to the actor"), [this]()
				{
					const FVector NewActorScale = { 0.75, 0.75, 0.75 };
					RemeshModifierActor->SetActorScale3D(NewActorScale);
					RemeshModifier->OnChanged(RemeshModifier->ComputeBounds(), EChangeType::StateChange);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Scaling_Functions_Correctly_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_RemeshModifier_EdgeLength_Affects_Triangulation)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Apply new target edge length to the actor"), [this]()
				{
					constexpr float NewTargetEdgeLength = 50.0;
					RemeshModifier->SetTargetEdgeLength(NewTargetEdgeLength);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_EdgeLength_Affects_Triangulation_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_RemeshModifier_Iterations_Are_Adjustable)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Apply new iteration count to the actor"), [this]()
				{
					constexpr int32 NewIterationsCount = 1;
					RemeshModifier->SetRemeshIterations(NewIterationsCount);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Iterations_Are_Adjustable_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_RemeshModifier_Smoothing_Is_Adjustable)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Apply new smoothing strength to the modifier"), [this]()
				{
					constexpr float NewSmoothingStrength = 1.0;
					RemeshModifier->SetSmoothingStrength(NewSmoothingStrength);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Smoothing_Is_Adjustable_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_RemeshModifier_Supports_Component_Tags)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Add component tag to the modifier"), [this]()
				{
					TArray<FName>& ComponentTags = RemeshModifier->ComponentTags;
					ComponentTags.Add(ComponentTagName);
				})
			.Then(TEXT("Verify that we can use ComponentTag to lookup component"), [this]()
				{
					const UActorComponent* FoundComponent =
						RemeshModifierActor->FindComponentByTag(URemeshModifier::StaticClass(), ComponentTagName);
					ASSERT_THAT(IsTrue(RemeshModifier->ComponentHasTag(ComponentTagName),
						*FString::Printf(TEXT("Failed to find component with tag %s"), *ComponentTagName.ToString())));
				});
	}

	TEST_METHOD(Verify_RemeshModifier_Can_Be_Disabled_And_Enabled)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(RemeshModifier);
				})
			.Then(TEXT("Set disabled flag to true for the modifier"), [this]()
				{
					RemeshModifier->SetIsDisabledFlag(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Can_Be_Disabled_GroundTruth");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Set disabled flag to false for the modifier"), [this]()
				{
					RemeshModifier->SetIsDisabledFlag(false);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Can_Be_Added_GroundTruth");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Set disabled flag to true for the modifier"), [this]()
				{
					RemeshModifier->SetIsDisabledFlag(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_RemeshModifier_Can_Be_Disabled_GroundTruth");
					CompareToReferenceAsset(TestName);
				});
	}

	AFTER_EACH()
	{
		MeshPartitionComponent->SetPauseTransformerPipeline(false);

		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	private:
	/**
	 * Imports a heightmap and creates a MeshPartition from it
	 * 
	 * @return	Pointer to the created AMeshPartition, or nullptr if import failed
	 */
	AMeshPartition* ImportHeightmap() const
	{
		if (!IsValid(World))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "ImportHeightmap: Invalid World");
			return nullptr;
		}
		const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("MeshPartition"));
		if (!Plugin.IsValid())
		{
			UE_LOGF(LogMegaMeshEditor, Error, "ImportHeightmap: MeshPartition plugin not found");
			return nullptr;
		}

		FHeightmapImportParams Params;
		Params.World = World;
		Params.HeightmapFilename = FPaths::Combine(Plugin->GetBaseDir(), HeightmapLocation);
		Params.MeshResolution = FInt32Point(256);
		Params.MeshSize = FVector(256.0);
		Params.SectionsResolution = { 1, 1 };
		Params.bSaveAndUnload = false;
		Params.LocationVolumesResolution = FInt32Point(0);

		// Import Heightmap
		FHeightmapImporter Importer(Params);
		FPackageSourceControlHelper* SourceControlHelper = nullptr;

		if (!Importer.Import(SourceControlHelper))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "ImportHeightmap failed: %ls",
				*Importer.GetErrorText().ToString());
			return nullptr;
		}

		return Importer.GetMegaMesh();
	}

	void CompareToReferenceAsset(const FString& TestName)
	{
		const FString ReferencePackagePath = ReferencePath / TestName;
		const FString ReferenceAssetPath = ReferencePackagePath + "." + TestName;

		if (TestUtils::ShouldUpdateReferenceAssets())
		{
			[[maybe_unused]] const bool bSuccess =
				TestUtils::CreateOrUpdateReference(MeshPartitionComponent, ReferencePackagePath, {TestName}, TestName);
		}

		FString ErrorMessage;
		bool bValid = TestUtils::ValidatePreviewMeshAgainstGroundTruth(
			MeshPartitionComponent,
			{ ReferenceAssetPath },
			ErrorMessage);

		ASSERT_THAT(IsTrue(bValid, *ErrorMessage));
	}
};

#endif
#endif