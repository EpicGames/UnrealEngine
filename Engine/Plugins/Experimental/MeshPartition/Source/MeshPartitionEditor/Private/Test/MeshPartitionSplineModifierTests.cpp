// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#if WITH_DEV_AUTOMATION_TESTS

#include "CQTest.h"
#include "Editor.h"
#include "MeshPartition.h"
#include "Components/SplineComponent.h"
#include "Modifiers/MeshPartitionSplineModifier.h"
#include "Test/MeshPartitionTestUtils.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"
#include "ScopedTransaction.h"

using namespace UE::MeshPartition;

/**
 * Tests for MeshPartition Spline Modifier
 */
TEST_CLASS_WITH_FLAGS(SplineModifierTests, "MeshPartition.Modifier.Splines", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	UWorld* World = nullptr;
	AActor* SplineActor = nullptr;
	USplineComponent* SplineComponent = nullptr;
	AMeshPartition* MeshPartition = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = nullptr;
	USplineModifier* SplineModifier = nullptr;

	// Test Configuration
	const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
	const FString ReferencePath = TEXT("/MeshPartition/UnitTests/SplineModifierTests");
	const int32 MeshWidth = 1000;
	const int32 MeshHeight = 1000;
	const int32 MeshWidthVertexCount = 500;
	const int32 MeshHeightVertexCount = 500;

	/*
	 * Defines the types of spline transform tests
	 */
	enum class SplineTransformTestType
	{
		Translation,
		Rotation,
		Scale
	};

	/*
	 * Specifies the priority values for priority update tests.
	 */
	const FName PriorityLayer = "TestType2";
	const double SubPriority = 2.0;

	/*
	 * Specifies the falloff bounds multiplier for the relevant test.
	 */
	const float FalloffBounds = 10.0f;

	/* 
	 * Specifies the Z axis value to be set for projection bounds tests.
	 */
	const float ZAxisBoundsValue = 1000.0f;

	/*
	* Specifies the Max Projection Height Extent value to be set for the projection bound test.
	*/
	const float MaxProjectionHeightExtent = 2000.0f;

	/* 
	* Original bounds for the projection bound test.
	*/
	TArray<FBox> OriginalBounds;

	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);

#if AUTOMATED_TESTING_EXECUTE_IN_EDITOR_WORLD
		World = GEditor->GetEditorWorldContext().World();
#else
		// Spawn world
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);
		ASSERT_THAT(IsNotNull(ScopedEditorWorld.Get(), TEXT("Failed to create scoped editor world")));

		World = ScopedEditorWorld->GetWorld();
#endif
		ASSERT_THAT(IsNotNull(World, TEXT("Failed to create valid test world")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));
		
		// Create MeshPartition
		MeshPartition = TestUtils::CreateTestMesh(World, MeshWidth, MeshHeight, MeshWidthVertexCount, MeshHeightVertexCount);
		ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to create MeshPartition")));

		MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
		ASSERT_THAT(IsNotNull(MeshPartitionEditorComponent, TEXT("Failed to get MeshPartitionEditorComponent")));
		
		// Pausing the transformer pipeline during the test to avoid potential race condition
		MeshPartitionEditorComponent->SetPauseTransformerPipeline(true);

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));

		// Create spline actor and component
		SplineActor = World->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity);
		ASSERT_THAT(IsNotNull(SplineActor, TEXT("Failed to create a spline actor")));

		SplineComponent = NewObject<USplineComponent>(SplineActor, USplineComponent::StaticClass());
		ASSERT_THAT(IsNotNull(SplineComponent, TEXT("Failed to create spline component")));

		// Configure spline component
		SplineActor->AddInstanceComponent(SplineComponent);
		SplineActor->SetRootComponent(SplineComponent);
		SplineComponent->RegisterComponent();

		// Setup spline points, create a simple 3-point spline
		SplineComponent->ClearSplinePoints();
		SplineComponent->AddSplinePoint(FVector(0.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World);
		SplineComponent->AddSplinePoint(FVector(100.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World);
		SplineComponent->AddSplinePoint(FVector(200.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World);
		SplineComponent->UpdateSpline();

		// Create and configure spline modifier
		SplineModifier = NewObject<USplineModifier>(SplineActor, USplineModifier::StaticClass(), NAME_None, RF_Transactional);
		ASSERT_THAT(IsNotNull(SplineModifier, TEXT("Failed to create spline modifier")));

		SplineModifier->SetWorldLocation(FVector::ZeroVector);
		SplineModifier->SetPriority(1.0);
		SplineModifier->SetAffectedMeshPartition(MeshPartition);
		SplineActor->AddInstanceComponent(SplineModifier);

		SplineModifier->AttachToComponent(SplineComponent, FAttachmentTransformRules::KeepWorldTransform);
		SplineModifier->SetSplineComponent(SplineComponent, true);
		SplineModifier->RegisterComponent();

		// Notify MeshPartition of modifier assignment
		MeshPartitionEditorComponent->OnModifierAssigned();
	}

	// QMetry: UE - TC - 20603
	TEST_METHOD(Verify_Adding_A_Spline_Modifier_To_The_Existing_Mesh_Terrain)
	{
		TestCommandBuilder
			.Then(TEXT("Verify that spline modifier has been attached to the existing mesh partition"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetAffectedMeshPartition() == MeshPartition, TEXT("Mesh partition does not contain specified modifier")));
				});
	}

	// QMetry: UE - TC - 20607	
	TEST_METHOD(Verify_Manually_Updating_Spline_Data_Translation_GroundTruth)
	{
		ManualUpdateSplineDataTest(SplineTransformTestType::Translation);
	}

	// QMetry: UE - TC - 20607	
	TEST_METHOD(Verify_Manually_Updating_Spline_Data_Rotation_GroundTruth)
	{
		ManualUpdateSplineDataTest(SplineTransformTestType::Rotation);
	}

	// QMetry: UE - TC - 20607	
	TEST_METHOD(Verify_Manually_Updating_Spline_Data_Scale_GroundTruth)
	{
		ManualUpdateSplineDataTest(SplineTransformTestType::Scale);
	}

	// QMetry: UE - TC - 20618	
	TEST_METHOD(Verify_Specifying_A_Priority_Layer_For_Spline_Modifier)
	{
		SpecifyPriorityLayerTest(PriorityLayer);
	}

	// QMetry: UE - TC - 20619	
	TEST_METHOD(Verify_Specifying_Sub_Priority_For_The_Spline_Modifier)
	{
		SpecifySubPriorityTest(SubPriority);
	}

	// QMetry: UE - TC - 20617	
	TEST_METHOD(Verify_Extending_Spline_Falloff_Bounds)
	{
		ExtendSplineFalloffBoundsTest(FalloffBounds);
	}

	// QMetry: UE - TC - 20615	
	TEST_METHOD(Verify_Adjusting_Spline_Modifier_Projection_Bounds_By_Z_Axis)
	{
		AdjustProjectionBoundsByZAxisTest(ZAxisBoundsValue);
	}

	// QMetry: UE - TC - 20616	
	TEST_METHOD(Verify_Extending_Spline_Modifier_Projection_Bounds_In_All_Directions)
	{
		ExtendProjectionBoundsTest(ZAxisBoundsValue, MaxProjectionHeightExtent);
	}

	// QMetry: UE - TC - 20605	
	TEST_METHOD(Verify_Using_Relative_Projection_Direction_For_Mesh_Projection_Translation_GroundTruth)
	{
		RelativeProjectionDirectionTest(SplineTransformTestType::Translation);
	}

	// QMetry: UE - TC - 20605	
	TEST_METHOD(Verify_Using_Relative_Projection_Direction_For_Mesh_Projection_Rotation_GroundTruth)
	{
		RelativeProjectionDirectionTest(SplineTransformTestType::Rotation);
	}

	// QMetry: UE - TC - 20605	
	TEST_METHOD(Verify_Using_Relative_Projection_Direction_For_Mesh_Projection_Scale_GroundTruth)
	{
		RelativeProjectionDirectionTest(SplineTransformTestType::Scale);
	}

	// QMetry: UE - TC - 20614
	TEST_METHOD(Verify_Using_Nearest_Spline_Frame_For_Displacement)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Enable Displace Nearest Frame and modify spline rotations"), [this]()
				{
					// Pre-Setup: Set rotations at spline points for validation
					SplineComponent->Modify();
					SplineComponent->SetRotationAtSplinePoint(0, FRotator(0.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World);
					SplineComponent->SetRotationAtSplinePoint(1, FRotator(45.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World);
					SplineComponent->SetRotationAtSplinePoint(2, FRotator(90.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World);
					SplineComponent->UpdateSpline();
				})
			.Until(TEXT("Wait for modifier to finish processing rebuild"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Enable Displace Nearest Frame and modify spline rotations"), [this]()
				{
					//Enable Displace Nearest Frame
					const FScopedTransaction Transaction(FText::FromString(TEXT("Set Displace Nearest Frame to true")));
					SplineModifier->Modify();

					SplineModifier->SetUseNearestSplineFrameForDisplacement(true);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier to finish processing rebuild"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_Using_Nearest_Spline_Frame_For_Displacement_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: Displace Nearest Frame should be set to false"), [this]()
				{
					const FString TestName = TEXT("Verify_Using_Nearest_Spline_Frame_For_Displacement_Initial");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: Displace Nearest Frame should be set to true"), [this]()
				{
					const FString TestName = TEXT("Verify_Using_Nearest_Spline_Frame_For_Displacement_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Set spline Closed Loop value to true"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Set spline Closed Loop value to true")));
					SplineComponent->Modify();

					SplineComponent->SetClosedLoop(true);
					SplineComponent->UpdateSpline();
				})
			.Until(TEXT("Wait for modifier to finish processing rebuild"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate Displace Nearest Frame is ignored when spline is closed loop"), [this]()
				{
					const FString TestName = TEXT("Verify_Display_Nearest_Frame_Ignored_When_Closed_Loop");
					CompareToReferenceAsset(TestName);
				});
	}

	// QMetry: UE - TC - 20600
	TEST_METHOD(Verify_Specifying_A_Falloff_Distance_For_The_Spline_Modifier)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Increase Falloff Distance value"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Modify Falloff Distance")));
					SplineModifier->Modify();

					SplineModifier->SetFalloffDistance(SplineModifier->GetFalloffDistance() + 100.0f);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_Specifying_A_Falloff_Distance_For_The_Spline_Modifier_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: Falloff Distance should be set to initial value of 500.0f"), [this]()
				{
					const FString TestName = TEXT("Verify_Specifying_A_Falloff_Distance_For_The_Spline_Modifier_Initial");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: Falloff Distance should be set to updated value of 600.0f"), [this]()
				{
					const FString TestName = TEXT("Verify_Specifying_A_Falloff_Distance_For_The_Spline_Modifier_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Decrease Falloff Distance value to initial value"), [this]()
				{
					SplineModifier->SetFalloffDistance(SplineModifier->GetFalloffDistance() - 100.0f);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_Specifying_A_Falloff_Distance_For_The_Spline_Modifier_Initial");
					CompareToReferenceAsset(TestName);
				});
	}

	// QMetry: UE - TC - 20623
	TEST_METHOD(Verify_Adjusting_Error_Tolerance_For_The_Spline_Modifier_Interior)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Increase Spline Polygon Error Tolerance value to 100.0f"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Increase Spline Polygon Error Tolerance value")));
					SplineModifier->Modify();

					SplineModifier->SetSplinePolygonErrorTolerance(100.0f);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the increased Spline Polygon Error Tolerance value on the spline modifier"), [this]()
				{
					const FString TestName = TEXT("Verify_Adjusting_Error_Tolerance_For_The_Spline_Modifier_Interior_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: Spline Polygon Error Tolerance value should be reverted back to the initial value"), [this]()
				{
					const FString TestName = TEXT("Verify_Adjusting_Error_Tolerance_For_The_Spline_Modifier_Interior_Initial");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: Spline Polygon Error Tolerance value should be updated to the increased value"), [this]()
				{
					const FString TestName = TEXT("Verify_Adjusting_Error_Tolerance_For_The_Spline_Modifier_Interior_Updated");
					CompareToReferenceAsset(TestName);
				});
	}

	// QMetry: UE - TC - 20624	
	TEST_METHOD(Verify_Adjusting_Tri_Count_For_The_Spline_Modifier_Interior)
	{
		const double SharedOriginalNumTriTarget = SplineModifier->GetMeshedInteriorNumTriTarget();
		const double SharedTargetNumTriTarget = SharedOriginalNumTriTarget + 500.0;

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Increase Num Tri Target value"), [this, SharedTargetNumTriTarget]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Increase Num Tri Target value")));
					SplineModifier->Modify();

					SplineModifier->SetMeshedInteriorNumTriTarget(SharedTargetNumTriTarget);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Verify that the Num Tri Target value have been updated, and validate generated mesh against ground truth"), [this, SharedTargetNumTriTarget]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMeshedInteriorNumTriTarget(), SharedTargetNumTriTarget), TEXT("Updated Num Tri Target value does not match the target value")));
					const FString TestName = TEXT("Verify_Adjusting_Tri_Count_For_The_Spline_Modifier_Interior_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: Num Tri Target value should be reverted back to the initial value, and the geometry should be reverted"), [this, SharedOriginalNumTriTarget]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMeshedInteriorNumTriTarget(), SharedOriginalNumTriTarget), TEXT("Undo did not revert the Num Tri Target value to the original value")));
					const FString TestName = TEXT("Verify_Adjusting_Tri_Count_For_The_Spline_Modifier_Interior_Initial");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: Num Tri Target value should be updated to the increased value, and the geometry should be updated"), [this, SharedTargetNumTriTarget]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMeshedInteriorNumTriTarget(), SharedTargetNumTriTarget), TEXT("Redo did not update the Num Tri Target value to the updated value")));
					const FString TestName = TEXT("Verify_Adjusting_Tri_Count_For_The_Spline_Modifier_Interior_Updated");
					CompareToReferenceAsset(TestName);
				});
	}

	// QMetry: UE - TC - 20621	
	TEST_METHOD(Verify_Completely_Disabling_Processing_Of_The_Spline_Modifier)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Set disabled flag to true for the modifier"), [this]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Disable Spline Modifier")));
					SplineModifier->Modify();

					SplineModifier->SetIsDisabledFlag(true);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Verify that Spline modifier is disabled"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->IsDisabled(), TEXT("Spline modifier is not disabled")));

					const FString TestName = TEXT("Verify_Completely_Disabling_Processing_Of_The_Spline_Modifier_Disabled");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate the effects of the Undo function: Spline Modifier should be enabled"), [this]()
				{
					ASSERT_THAT(IsFalse(SplineModifier->IsDisabled(), TEXT("Undo did not enable the modifier")));

					const FString TestName = TEXT("Verify_Completely_Disabling_Processing_Of_The_Spline_Modifier_Enabled");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsAnyPreviewSectionBuildActive();
				})
			.Then(TEXT("Validate the effects of the Redo function: Spline Modifier should be disabled"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->IsDisabled(), TEXT("Redo did not disable the modifier")));

					const FString TestName = TEXT("Verify_Completely_Disabling_Processing_Of_The_Spline_Modifier_Disabled");
					CompareToReferenceAsset(TestName);
				});
	}

	// QMetry: UE - TC - 20622
	TEST_METHOD(Verify_Applying_Smooth_Modes_For_The_Spline_Modifier_Interior)
	{
		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Set Interior Smooth Mode to Smooth"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetInteriorSmoothMode() == ESplineModifierInteriorSmoothMode::Simple, TEXT("Default Interior Smooth Mode is not set to Simple")));

					const FScopedTransaction Transaction(FText::FromString(TEXT("Set Interior Smooth Mode to Smooth")));
					SplineModifier->Modify();

					SplineModifier->SetInteriorSmoothMode(ESplineModifierInteriorSmoothMode::Smooth);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth: Interior Smooth Mode should be set to Smooth"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetInteriorSmoothMode() == ESplineModifierInteriorSmoothMode::Smooth, TEXT("Interior Smooth Mode is not set to Smooth")));
					const FString TestName = TEXT("Verify_Applying_Smooth_Modes_For_The_Spline_Modifier_Interior_Smooth");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: Interior Smooth Mode should be set to Simple"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetInteriorSmoothMode() == ESplineModifierInteriorSmoothMode::Simple, TEXT("Interior Smooth Mode is not set to Simple")));
					const FString TestName = TEXT("Verify_Applying_Smooth_Modes_For_The_Spline_Modifier_Interior_Simple");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: Interior Smooth Mode should be set to Smooth"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetInteriorSmoothMode() == ESplineModifierInteriorSmoothMode::Smooth, TEXT("Interior Smooth Mode is not set to Smooth")));
					const FString TestName = TEXT("Verify_Applying_Smooth_Modes_For_The_Spline_Modifier_Interior_Smooth");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Set Interior Smooth Mode to Detail Preserving"), [this]()
				{
					SplineModifier->SetInteriorSmoothMode(ESplineModifierInteriorSmoothMode::DetailPreserving);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth: Interior Smooth Mode should be set to Detail Preserving"), [this]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetInteriorSmoothMode() == ESplineModifierInteriorSmoothMode::DetailPreserving, TEXT("Interior Smooth Mode is not set to Detail Preserving")));
					const FString TestName = TEXT("Verify_Applying_Smooth_Modes_For_The_Spline_Modifier_Interior_DetailPreserving");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_Bounds_Expand_By_Uniform_Spline_Scale)
	{
		const float UniformScaleY = 3.0f;
		for (int32 PointIndex = 0; PointIndex < SplineComponent->GetNumberOfSplinePoints(); ++PointIndex)
		{
			SplineComponent->SetScaleAtSplinePoint(PointIndex, FVector(1.0f, UniformScaleY, 1.0f));
		}
		SplineComponent->UpdateSpline();

		VerifyBoundsContainScaledFalloff(UniformScaleY);
	}

	TEST_METHOD(Verify_Bounds_Expand_By_Max_Spline_Scale)
	{
		// Set varying scale values — bounds should use the maximum.
		SplineComponent->SetScaleAtSplinePoint(0, FVector(1.0f, 1.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(1, FVector(1.0f, 5.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(2, FVector(1.0f, 2.0f, 1.0f));
		SplineComponent->UpdateSpline();

		VerifyBoundsContainScaledFalloff(5.0f);
	}

	TEST_METHOD(Verify_Bounds_Account_For_Hermite_Tangent_Overshoot)
	{
		// Create a spline with a sharp step in scale: 1, 1, 10, 10, 1.
		// The transition from 10 back to 1 at point 3->4 gives point 3 a large negative
		// auto-tangent (from the asymmetry of its neighbors 10 and 1), which causes the
		// Hermite interpolation to overshoot above 10 in the segment between points 2 and 3.
		SplineComponent->ClearSplinePoints();
		SplineComponent->AddSplinePoint(FVector(0.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World, false);
		SplineComponent->AddSplinePoint(FVector(100.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World, false);
		SplineComponent->AddSplinePoint(FVector(200.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World, false);
		SplineComponent->AddSplinePoint(FVector(300.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World, false);
		SplineComponent->AddSplinePoint(FVector(400.0f, 0.0f, 0.0f), ESplineCoordinateSpace::World, false);
		SplineComponent->UpdateSpline();

		SplineComponent->SetScaleAtSplinePoint(0, FVector(1.0f, 1.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(1, FVector(1.0f, 1.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(2, FVector(1.0f, 10.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(3, FVector(1.0f, 10.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(4, FVector(1.0f, 1.0f, 1.0f));
		SplineComponent->UpdateSpline();

		SplineModifier->SetUseSplineScaleForFalloff(true);
		SplineModifier->SetExpandBoundsBySplineScale(true);
		SplineModifier->UpdateSplineData();

		const FSplineCurves SplineCurves = SplineComponent->GetSplineCurves();

		// Get the max absolute scale value at the curve control points

		float MaxControlPointScaleY = 0.0f;
		for (const FInterpCurvePoint<FVector>& ScalePoint : SplineCurves.Scale.Points)
		{
			MaxControlPointScaleY = FMath::Max(MaxControlPointScaleY, FMath::Abs(ScalePoint.OutVal.Y));
		}

		// Sample the scale curve to verify that overshoot actually occurs — the interpolated
		// scale should exceed the maximum control point value somewhere along the spline.

		float MaxSampledScaleY = 0.0f;
		const int32 NumSamples = 9;
		for (int32 SampleIndex = 0; SampleIndex <= NumSamples; ++SampleIndex)
		{
			const float T = SplineCurves.Scale.Points[0].InVal +
				(SplineCurves.Scale.Points.Last().InVal - SplineCurves.Scale.Points[0].InVal) * SampleIndex / NumSamples;
			const float SampledScaleY = SplineCurves.Scale.Eval(T).Y;
			MaxSampledScaleY = FMath::Max(MaxSampledScaleY, FMath::Abs(SampledScaleY));
		}

		// Verify the test setup actually produces overshoot — if not, the test is not meaningful.
		ASSERT_THAT(IsTrue(MaxSampledScaleY > MaxControlPointScaleY,
			*FString::Printf(TEXT("Test setup should produce interpolation overshoot: MaxSampledScaleY (%.2f) should be > MaxControlPointScaleY (%.2f)"),
				MaxSampledScaleY, MaxControlPointScaleY)));

		// Now verify the bounds account for it.
		const TArray<FBox> Bounds = SplineModifier->ComputeBounds();
		ASSERT_THAT(IsTrue(Bounds.Num() > 0, TEXT("ComputeBounds returned no bounds")));

		const float FalloffDistance = SplineModifier->GetFalloffDistance();
		const FVector Extent = Bounds[0].GetExtent();

		ASSERT_THAT(IsTrue(Extent.Y > FalloffDistance * MaxControlPointScaleY,
			*FString::Printf(TEXT("Bounds Y extent (%.2f) should be > FalloffDistance * MaxControlPointScaleY (%.2f) due to tangent overshoot"),
				Extent.Y, FalloffDistance * MaxControlPointScaleY)));
	}

	TEST_METHOD(Verify_Bounds_Use_FalloffBoundsMultiplier_When_ExpandBySplineScale_Disabled)
	{
		// When bExpandBoundsBySplineScale is false, the bounds should use FalloffBoundsMultiplier
		// instead of computing from spline scale.
		const float MultiplierValue = 4.0f;

		// Set a large spline scale that would produce bigger bounds if bExpandBoundsBySplineScale were true.
		SplineComponent->SetScaleAtSplinePoint(0, FVector(1.0f, 8.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(1, FVector(1.0f, 8.0f, 1.0f));
		SplineComponent->SetScaleAtSplinePoint(2, FVector(1.0f, 8.0f, 1.0f));
		SplineComponent->UpdateSpline();

		SplineModifier->SetUseSplineScaleForFalloff(true);
		SplineModifier->SetExpandBoundsBySplineScale(false);
		SplineModifier->SetFalloffBoundsMultiplier(MultiplierValue);
		SplineModifier->UpdateSplineData();

		const TArray<FBox> Bounds = SplineModifier->ComputeBounds();
		ASSERT_THAT(IsTrue(Bounds.Num() > 0, TEXT("ComputeBounds returned no bounds")));

		const FVector Extent = Bounds[0].GetExtent();
		const float FalloffDistance = SplineModifier->GetFalloffDistance();

		// Bounds should reflect FalloffBoundsMultiplier (4.0), not the spline scale (8.0).
		const float ExpectedExtent = FalloffDistance * MultiplierValue;
		const float OverExpectedExtent = FalloffDistance * 8.0f;

		ASSERT_THAT(IsTrue(Extent.Y >= ExpectedExtent,
			*FString::Printf(TEXT("Bounds Y extent (%.2f) should be >= FalloffDistance * FalloffBoundsMultiplier = %.2f"),
				Extent.Y, ExpectedExtent)));
		ASSERT_THAT(IsTrue(Extent.Y < OverExpectedExtent,
			*FString::Printf(TEXT("Bounds Y extent (%.2f) should be < FalloffDistance * SplineScale = %.2f (should not use spline scale)"),
				Extent.Y, OverExpectedExtent)));
	}

	AFTER_EACH()
	{
		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	private:
	void ManualUpdateSplineDataTest(SplineTransformTestType Type)
	{
		TSharedPtr<FString> TransformTestName = MakeShared<FString>();

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Modify spline point and update"), [this, Type, TransformTestName]()
				{
					const int32 NumPoints = SplineComponent->GetNumberOfSplinePoints();
					ASSERT_THAT(IsTrue(NumPoints > 0, TEXT("No spline points available")));

					const int32 LastPointIndex = NumPoints - 1;
					const FVector CurrentLocation = SplineComponent->GetLocationAtSplinePoint(
						LastPointIndex,
						ESplineCoordinateSpace::World
					);

					switch (Type)
					{
					case SplineTransformTestType::Translation:
						{
							const FVector SplinePointOffset = FVector(0.0f, 0.0f, 50.0f);
							SplineComponent->SetLocationAtSplinePoint(
								LastPointIndex,
								CurrentLocation + SplinePointOffset,
								ESplineCoordinateSpace::World
							);

							*TransformTestName = TEXT("Manually_Updating_Spline_Data_Translation_GroundTruth");
							break;
						}
					case SplineTransformTestType::Rotation:
						{
							const FRotator SplinePointRotator = FRotator(90.0f, 0.0f, 0.0f);
							SplineComponent->SetRotationAtSplinePoint(
								LastPointIndex,
								SplinePointRotator,
								ESplineCoordinateSpace::World
							);

							*TransformTestName = TEXT("Manually_Updating_Spline_Data_Rotation_GroundTruth");
							break;
						}
					case SplineTransformTestType::Scale:
						{
							const FVector Offset = FVector(0.0f, 0.0f, 50.0f);
							SplineComponent->SetLocationAtSplinePoint(
								LastPointIndex,
								CurrentLocation + Offset,
								ESplineCoordinateSpace::World
							);
							const FVector NewScale = FVector(1.0f, 1.0f, 2.0f);
							SplineComponent->SetScaleAtSplinePoint(
								LastPointIndex,
								NewScale,
								ESplineCoordinateSpace::World
							);

							*TransformTestName = TEXT("Manually_Updating_Spline_Data_Scale_GroundTruth");
							break;
						}

					default: ASSERT_FAIL(TEXT("SplineTestType is not set"));
					}

					SplineComponent->UpdateSpline();
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this, TransformTestName]()
				{
					CompareToReferenceAsset(*TransformTestName);
				});
	}

	void SpecifyPriorityLayerTest(FName ExpectedPriorityLayer)
	{
		const FName SharedOriginalPriorityLayer = SplineModifier->GetType();
		const FName SharedExpectedPriorityLayer = ExpectedPriorityLayer;

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Set a new priority layer"), [this, SharedExpectedPriorityLayer]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Modify Priority")));
					SplineModifier->Modify();
					SplineModifier->SetType(SharedExpectedPriorityLayer);
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the updated priority on the modifier"), [this, SharedExpectedPriorityLayer]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetType() == SharedExpectedPriorityLayer, TEXT("Updated Priority Layer value does not match the expected text")));
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function on the priority layer"), [this, SharedOriginalPriorityLayer]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetType() == SharedOriginalPriorityLayer, TEXT("Undo did not revert the priority layer value")));
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function on the priority layer value"), [this, SharedExpectedPriorityLayer]()
				{
					ASSERT_THAT(IsTrue(SplineModifier->GetType() == SharedExpectedPriorityLayer , TEXT("Redo did not update the priority layer value")));
				});
	}

	void SpecifySubPriorityTest(double TargetSubPriority)
	{
		const double SharedOriginalSubPriority = SplineModifier->GetPriority();
		const double SharedTargetSubPriority = TargetSubPriority;

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Modify modifier sub priority"), [this, SharedOriginalSubPriority, SharedTargetSubPriority]()
				{
					ASSERT_THAT(IsFalse(FMath::IsNearlyEqual(SharedOriginalSubPriority, SharedTargetSubPriority), TEXT("Target sub priority value must differ from the current sub priority for this test to be meaningful")));

					const FScopedTransaction Transaction(FText::FromString(TEXT("Modify Sub Priority")));
					SplineModifier->Modify();
					SplineModifier->SetPriority(SharedTargetSubPriority);
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the updated sub priority on the modifier"), [this, SharedTargetSubPriority]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetPriority(), SharedTargetSubPriority), TEXT("Updated Sub priority does not match the target value")));
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function on the sub priority value"), [this, SharedOriginalSubPriority]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetPriority(), SharedOriginalSubPriority), TEXT("Undo did not revert the Sub priority to the original value")));
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function on the sub priority value"), [this, SharedTargetSubPriority]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetPriority(), SharedTargetSubPriority), TEXT("Redo did not update the Sub priority to the updated value")));
				});
	}

	void ExtendSplineFalloffBoundsTest(float TargetFalloffBoundsMultiplier)
	{
		const float SharedOriginalFalloffBoundsMultiplier = SplineModifier->GetFalloffBoundsMultiplier();

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Increase Falloff Bounds Multipler value"), [this, TargetFalloffBoundsMultiplier]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Set Falloff Bounds Multiple to an increased value")));
					SplineModifier->Modify();

					SplineModifier->SetFalloffBoundsMultiplier(TargetFalloffBoundsMultiplier);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier to finish participating in rebuild"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_Extending_Spline_Falloff_Bounds_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: Falloff Bounds Multipler value has been reverted, and Spline Falloff is being cut on the bounds"), [this, SharedOriginalFalloffBoundsMultiplier]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetFalloffBoundsMultiplier(), SharedOriginalFalloffBoundsMultiplier), TEXT("Undo did not revert the Falloff Bounds Multipler value to the original value")));
					const FString TestName = TEXT("Verify_Extending_Spline_Falloff_Bounds_Initial");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: Falloff Bounds Multipler value has been reapplied, and Spline Falloff smoothly reaches the ground"), [this, TargetFalloffBoundsMultiplier]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetFalloffBoundsMultiplier(), TargetFalloffBoundsMultiplier), TEXT("Redo did not update the Falloff Bounds Multipler value to the updated value")));
					const FString TestName = TEXT("Verify_Extending_Spline_Falloff_Bounds_Updated");
					CompareToReferenceAsset(TestName);
				});
	}

	void AdjustProjectionBoundsByZAxisTest(float TargetZValue)
	{
		const float SharedOriginalZValue = SplineModifier->GetMaxZDistance();
		const float SharedTargetZValue = TargetZValue;

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Move modifier along Z axis out of bounds "), [this, SharedTargetZValue]()
				{
					// Move the spline modifier Z location far above the mesh terrain
					SplineModifier->SetWorldLocation(FVector(0.0f, 0.0f, SharedTargetZValue));

					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})	
			.Then(TEXT("Validate generated mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_Adjusting_Projection_Bounds_Out_Of_Bounds");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Expand Max Z Distance to set the modifier within bounds"), [this, SharedTargetZValue]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Increase Max Z Distance to include within bounds")));
					SplineModifier->Modify();

					SplineModifier->SetMaxZDistance(SharedTargetZValue);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier to finish processing rebuild"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Verify that the Maz Z Distance have been updated, and validate generated mesh against ground truth"), [this, SharedTargetZValue]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMaxZDistance(), SharedTargetZValue), TEXT("Updated Maz Z Distance does not match the target value")));
					const FString TestName = TEXT("Verify_Adjusting_Projection_Bounds_Expand_Bounds");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: Max Z Distance value should be reverted"), [this, SharedOriginalZValue]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMaxZDistance(), SharedOriginalZValue), TEXT("Undo did not revert the Maz Z Distance value to the original value")));
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: Max Z Distance value should be updated to be within bounds"), [this, SharedTargetZValue]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMaxZDistance(), SharedTargetZValue), TEXT("Redo did not update the Maz Z Distance value to the updated value")));
				});
	}

	void ExtendProjectionBoundsTest(float TargetZValue, float TargetMaxProjectionHeightExtent)
	{
		float OriginalMaxProjectionHeightExtent = SplineModifier->GetMaxProjectionHeightExtent();

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Set Location Z value to target value"), [this, TargetZValue]()
				{
					SplineModifier->SetWorldLocation(FVector(0.0f, 0.0f, TargetZValue));
					SplineModifier->SetUseNearestSplineFrameForDisplacement(true);
					SplineModifier->SetMeshClosedInterior(false);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Increase Max Projection Height Extent value"), [this, TargetMaxProjectionHeightExtent]()
				{
					OriginalBounds = SplineModifier->ComputeBounds();
					ASSERT_THAT(IsTrue(OriginalBounds.Num() > 0, TEXT("OriginalBounds is empty — ComputeBounds returned no bounds")));

					const FScopedTransaction Transaction(FText::FromString(TEXT("Set Max Projection Height Extent value to >=1000")));
					SplineModifier->Modify();

					SplineModifier->SetMaxProjectionHeightExtent(TargetMaxProjectionHeightExtent);
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier to finish processing rebuild"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Verify that the spline modifier projection bounds have been increased in all axes"), [this]()
				{
					TArray<FBox> UpdatedBounds = SplineModifier->ComputeBounds();
					ASSERT_THAT(IsTrue(UpdatedBounds.Num() > 0, TEXT("UpdatedBounds is empty — ComputeBounds returned no bounds")));
					
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Min.X < OriginalBounds[0].Min.X, TEXT("The Min.X bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Min.Y < OriginalBounds[0].Min.Y, TEXT("The Min.Y bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Min.Z < OriginalBounds[0].Min.Z, TEXT("The Min.Z bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Max.X > OriginalBounds[0].Max.X, TEXT("The Max.X bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Max.Y > OriginalBounds[0].Max.Y, TEXT("The Max.Y bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Max.Z > OriginalBounds[0].Max.Z, TEXT("The Max.Z bound range did not expand")));
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Undo function: The projection bounds have been reverted"), [this, OriginalMaxProjectionHeightExtent]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMaxProjectionHeightExtent(), OriginalMaxProjectionHeightExtent), TEXT("The Max Projection Height Extent value was not reverted to the original value")));

					TArray<FBox> UpdatedBounds = SplineModifier->ComputeBounds();
					ASSERT_THAT(IsTrue(UpdatedBounds.Num() > 0, TEXT("UpdatedBounds is empty — ComputeBounds returned no bounds")));

					ASSERT_THAT(IsNear(UpdatedBounds[0].Min.X, OriginalBounds[0].Min.X, UE_DOUBLE_SMALL_NUMBER, TEXT("The Min.X bound range did not revert to the original value")));
					ASSERT_THAT(IsNear(UpdatedBounds[0].Min.Y, OriginalBounds[0].Min.Y, UE_DOUBLE_SMALL_NUMBER, TEXT("The Min.Y bound range did not revert to the original value")));
					ASSERT_THAT(IsNear(UpdatedBounds[0].Min.Z, OriginalBounds[0].Min.Z, UE_DOUBLE_SMALL_NUMBER, TEXT("The Min.Z bound range did not revert to the original value")));
					ASSERT_THAT(IsNear(UpdatedBounds[0].Max.X, OriginalBounds[0].Max.X, UE_DOUBLE_SMALL_NUMBER, TEXT("The Max.X bound range did not revert to the original value")));
					ASSERT_THAT(IsNear(UpdatedBounds[0].Max.Y, OriginalBounds[0].Max.Y, UE_DOUBLE_SMALL_NUMBER, TEXT("The Max.Y bound range did not revert to the original value")));
					ASSERT_THAT(IsNear(UpdatedBounds[0].Max.Z, OriginalBounds[0].Max.Z, UE_DOUBLE_SMALL_NUMBER, TEXT("The Max.Z bound range did not revert to the original value")));
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate the effects of the Redo function: The projection bounds have been expanded again"), [this, TargetMaxProjectionHeightExtent]()
				{
					ASSERT_THAT(IsTrue(FMath::IsNearlyEqual(SplineModifier->GetMaxProjectionHeightExtent(), TargetMaxProjectionHeightExtent), TEXT("The Max Projection Height Extent value was not reverted to the expanded value")));

					TArray<FBox> UpdatedBounds = SplineModifier->ComputeBounds();
					ASSERT_THAT(IsTrue(UpdatedBounds.Num() > 0, TEXT("UpdatedBounds is empty — ComputeBounds returned no bounds")));

					ASSERT_THAT(IsTrue(UpdatedBounds[0].Min.X < OriginalBounds[0].Min.X, TEXT("The Min.X bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Min.Y < OriginalBounds[0].Min.Y, TEXT("The Min.Y bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Min.Z < OriginalBounds[0].Min.Z, TEXT("The Min.Z bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Max.X > OriginalBounds[0].Max.X, TEXT("The Max.X bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Max.Y > OriginalBounds[0].Max.Y, TEXT("The Max.Y bound range did not expand")));
					ASSERT_THAT(IsTrue(UpdatedBounds[0].Max.Z > OriginalBounds[0].Max.Z, TEXT("The Max.Z bound range did not expand")));
				});
	}

	void RelativeProjectionDirectionTest(SplineTransformTestType Type)
	{
		TSharedPtr<FString> TransformTestName = MakeShared<FString>();

		TestCommandBuilder
			.Until(TEXT("Wait for initial modifier processing to complete"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Modify spline point and update"), [this, Type, TransformTestName]()
				{
					const FScopedTransaction Transaction(FText::FromString(TEXT("Modify spline transform")));
					SplineComponent->Modify();

					const FTransform CurrentTransform = SplineComponent->GetComponentTransform();

					switch (Type)
					{
					case SplineTransformTestType::Translation:
					{
						FTransform NewTransform = CurrentTransform;
						NewTransform.SetLocation(CurrentTransform.GetLocation() + FVector(0.0, 0.0, 50.0f));
						SplineComponent->SetWorldTransform(NewTransform);

						*TransformTestName = TEXT("Verify_Relative_Projection_Direction_Translation");
						break;
					}
					case SplineTransformTestType::Rotation:
					{
						FTransform NewTransform = CurrentTransform;
						const FRotator NewRotator = CurrentTransform.GetRotation().Rotator() + FRotator(90.0f, 0.0f, 0.0f);
						NewTransform.SetRotation(NewRotator.Quaternion());
						SplineComponent->SetWorldTransform(NewTransform);

						*TransformTestName = TEXT("Verify_Relative_Projection_Direction_Rotation");
						break;
					}
					case SplineTransformTestType::Scale:
					{
						FTransform NewTransform = CurrentTransform;
						NewTransform.SetLocation(CurrentTransform.GetLocation() + FVector(0.0, 0.0, 50.0f));
						NewTransform.SetScale3D(FVector(1.0f, 1.0f, 2.0f));
						SplineComponent->SetWorldTransform(NewTransform);

						*TransformTestName = TEXT("Verify_Relative_Projection_Direction_Scale");
						break;
					}

					default: ASSERT_FAIL(TEXT("SplineTestType is not set"));
					}

					SplineComponent->UpdateSpline();
					SplineModifier->UpdateSplineData();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this, TransformTestName]()
				{
					const FString TestName = *TransformTestName + TEXT("_Updated");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Undo function on the feature"), [this]()
				{
					GEditor->UndoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this, TransformTestName]()
				{
					const FString TestName = *TransformTestName + TEXT("_Initial");
					CompareToReferenceAsset(TestName);
				})
			.Then(TEXT("Test Redo function on the feature"), [this]()
				{
					GEditor->RedoTransaction();
				})
			.Until(TEXT("Wait for modifier update to process"), [this]()
				{
					return !MeshPartitionEditorComponent->IsModifierParticipatingInActivePreviewSectionBuild(SplineModifier);
				})
			.Then(TEXT("Validate generated mesh against ground truth"), [this, TransformTestName]()
				{
					const FString TestName = *TransformTestName + TEXT("_Updated");
					CompareToReferenceAsset(TestName);
				});
	}

	/**
	 * Verifies that ComputeBounds produces a box whose Y extent is at least
	 * FalloffDistance * ExpectedMinMultiplier when bUseSplineScaleForFalloff and
	 * bExpandBoundsBySplineScale are both enabled.
	 */
	void VerifyBoundsContainScaledFalloff(float ExpectedMinMultiplier)
	{
		SplineModifier->SetUseSplineScaleForFalloff(true);
		SplineModifier->SetExpandBoundsBySplineScale(true);
		SplineModifier->UpdateSplineData();

		const TArray<FBox> Bounds = SplineModifier->ComputeBounds();
		ASSERT_THAT(IsTrue(Bounds.Num() > 0, TEXT("ComputeBounds returned no bounds")));

		const FVector Extent = Bounds[0].GetExtent();
		const float FalloffDistance = SplineModifier->GetFalloffDistance();
		const float ExpectedMinExtent = FalloffDistance * ExpectedMinMultiplier;

		// The bounds extent in Y should be at least the scaled falloff distance.
		// (The spline runs along X, so falloff expands primarily in Y.)
		ASSERT_THAT(IsTrue(Extent.Y >= ExpectedMinExtent,
			*FString::Printf(TEXT("Bounds Y extent (%.2f) should be >= FalloffDistance * %.2f = %.2f"),
				Extent.Y, ExpectedMinMultiplier, ExpectedMinExtent)));
	}

	void CompareToReferenceAsset(const FString& TestName)
	{
		const FString ReferencePackagePath = ReferencePath / TestName;
		const FString ReferenceAssetPath = ReferencePackagePath + "." + TestName;

		if (TestUtils::ShouldUpdateReferenceAssets())
		{
			[[maybe_unused]] const bool bSuccess =
				TestUtils::CreateOrUpdateReference(MeshPartitionEditorComponent, ReferencePackagePath, { TestName }, TestName);
		}

		FString ErrorMessage;
		bool bValid = TestUtils::ValidatePreviewMeshAgainstGroundTruth(
			MeshPartitionEditorComponent,
			{ ReferenceAssetPath },
			ErrorMessage);

		ASSERT_THAT(IsTrue(bValid, *ErrorMessage));
	}
};

#endif
#endif