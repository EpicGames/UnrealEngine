// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionTestModifier.generated.h"

namespace UE::MeshPartition
{
UCLASS(Hidden)
class UTestModifierComponent : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()
public:
	
	UTestModifierComponent()
	{
		InitCount = MakeShared<int32>(0);
		UninitCount = MakeShared<int32>(0);
	}
	
	// MeshPartition::UModifierComponent Implementation
	virtual void InitializeModifier() override { (*InitCount)++; }
	virtual void UninitializeModifier() override { (*UninitCount)++; };
	
	TSharedPtr<int32> GetInitCount() const { return InitCount; }
	TSharedPtr<int32> GetUninitCount() const { return UninitCount; }
	void ResetCounts()
	{
		*InitCount = 0;
		*UninitCount = 0;
	}
protected:
	// keep track of the number of times init/uninit are called
	// these are sharedptrs so they can be read after the modifier is destroyed.
	TSharedPtr<int32> InitCount;
	TSharedPtr<int32> UninitCount;
};
} // namespace UE::MeshPartition