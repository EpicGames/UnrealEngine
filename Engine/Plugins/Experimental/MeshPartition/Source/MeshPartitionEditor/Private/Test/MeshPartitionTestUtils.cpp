// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_DEV_AUTOMATION_TESTS

#include "Test/MeshPartitionTestUtils.h"

#include "Editor.h"
#include "MeshDescriptionBase.h"
#include "Engine/World.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "StaticMeshAttributes.h"
#include "AssetUtils/CreateStaticMeshUtil.h"
#include "Generators/RectangleMeshGenerator.h"
#include "Modifiers/MeshPartitionPatchModifier.h"
#include "Modifiers/MeshPartitionRemeshModifier.h"
#include "Subsystems/EditorAssetSubsystem.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionEditorLoaderAdapter.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterShape.h"

namespace UE::MeshPartition::TestUtils
{

	static bool bUpdateReferenceAssets = false;
	static FAutoConsoleVariableRef CVarUpdateReferenceAsset(
		TEXT("Automation.MeshPartition.UpdateReferenceAssets"),
		bUpdateReferenceAssets,
		TEXT("If true, automation tests will update the stored reference assets for all executed Mesh Partition tests."),
		ECVF_Default);


	bool ShouldUpdateReferenceAssets()
	{
		return bUpdateReferenceAssets;
	}


	AMeshPartition* CreateTestMesh(UWorld* InWorld, int InWidth, int InHeight, int InWidthVertexCount, int InHeightVertexCount)
	{
		AMeshPartition* MeshPartition = InWorld->SpawnActor<AMeshPartition>(AMeshPartition::StaticClass(), FTransform::Identity);
		UMeshPartitionEditorComponent* MeshPartitionEditorComponent = NewObject<UMeshPartitionEditorComponent>(MeshPartition, UMeshPartitionEditorComponent::StaticClass(), TEXT("MegaMeshEditorComponent"));
		MeshPartitionEditorComponent->SetForceSynchronousPreviewSectionBuild(true);
		MeshPartition->SetMeshPartitionComponent(MeshPartitionEditorComponent);

		const FString TestAutomationDefaultDefinitionPath = TEXT("/MeshPartition/UnitTests/DataAssets/MPD_AutomationTestDefault.MPD_AutomationTestDefault");
		UMeshPartitionDefinition* MeshPartitionDefinition = LoadObject<UMeshPartitionDefinition>(nullptr, TestAutomationDefaultDefinitionPath);
		if (!MeshPartitionDefinition)
		{
			UE_LOGF(LogMegaMeshEditor, Warning,
				"CreateTestMesh: Failed to load MeshPartitionDefinition (%ls)", *TestAutomationDefaultDefinitionPath);
		}
		MeshPartition->SetMeshPartitionDefinition(MeshPartitionDefinition);

		Geometry::FRectangleMeshGenerator RectGen;
		RectGen.Width = InWidth;
		RectGen.Height = InHeight;
		RectGen.WidthVertexCount = InWidthVertexCount;
		RectGen.HeightVertexCount = InHeightVertexCount;
		MeshPartitionEditorComponent->SpawnBaseModifier(FDynamicMesh3(&RectGen.Generate()), {}, FTransform::Identity);

		return MeshPartition;
	}

	AActor* CreateRectangleModifier(UMeshPartitionEditorComponent* MeshPartitionEditorComponent, const double Width, const double Height, const int32 WidthVertexCount,
		const int32 HeightVertexCount, TArray<FBox>& OutBounds, const FTransform& Transform)
	{
		// Configure rectangle mesh generator
		Geometry::FRectangleMeshGenerator RectGen;
		RectGen.Width = Width;
		RectGen.Height = Height;
		RectGen.WidthVertexCount = WidthVertexCount;
		RectGen.HeightVertexCount = HeightVertexCount;

		// Spawn a base modifier actor using the generated mesh
		AActor* BaseModifier = MeshPartitionEditorComponent->SpawnBaseModifier(
			FDynamicMesh3(&RectGen.Generate()),
			{},
			Transform
		);

		// Add the modifier's bounds to the output array
		OutBounds.Emplace(BaseModifier->GetComponentsBoundingBox(
			/* bNonColliding = */ true, /* bIncludeChildActors = */ false
		));

		return BaseModifier;
	}

	UPatchModifier* CreatePatchModifier(AActor* PatchActor, AMeshPartition* MeshPartition, const FVector& ModifierWorldLocation, const float Priority)
	{
		if (!IsValid(PatchActor))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreatePatchModifier: PatchActor is null or invalid");
			return nullptr;
		}

		if (!IsValid(MeshPartition))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreatePatchModifier: MeshPartition is null or invalid");
			return nullptr;
		}

		USceneComponent* SceneComponent = NewObject<USceneComponent>(PatchActor, TEXT("SceneComponent"));
		PatchActor->SetRootComponent(SceneComponent);
		SceneComponent->RegisterComponent();

		// Create the component
		UPatchModifier* PatchModifier = NewObject<UPatchModifier>(
			PatchActor,
			UPatchModifier::StaticClass()
		);

		// NewObject should not return null in normal circumstances, but check for safety
		if (!PatchModifier)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreatePatchModifier: Failed to create UPatchModifier object");
			return nullptr;
		}

		// Configure the component
		PatchModifier->SetWorldLocation(ModifierWorldLocation);
		PatchModifier->SetPriority(Priority);
		PatchActor->AddInstanceComponent(PatchModifier);
		PatchModifier->SetAffectedMeshPartition(MeshPartition);
		// Attach to root component
		if (USceneComponent* RootComp = PatchActor->GetRootComponent())
		{
			PatchModifier->AttachToComponent(RootComp, FAttachmentTransformRules::KeepWorldTransform);
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Warning, "CreatePatchModifier: PatchActor has no root component, component will not be attached");
		}

		// Register the component
		PatchModifier->RegisterComponent();

		return PatchModifier;
	}

	URemeshModifier* CreateRemeshModifier(AActor* Actor, AMeshPartition* MeshPartition, const FVector& ModifierWorldLocation,
		const FVector3d& UnscaledCoverage)
	{
		if (!IsValid(Actor))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateRemeshModifier: Actor is null or invalid");
			return nullptr;
		}

		if (!IsValid(MeshPartition))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateRemeshModifier: MeshPartition is null or invalid");
			return nullptr;
		}

		// Create the component
		URemeshModifier* RemeshModifier = NewObject<URemeshModifier>(
			Actor,
			URemeshModifier::StaticClass()
		);

		// NewObject should not return null in normal circumstances, but check for safety
		if (!RemeshModifier)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateRemeshModifier: Failed to create UMegaMeshRemeshModifier object");
			return nullptr;
		}

		// Configure the component
		RemeshModifier->SetWorldLocation(ModifierWorldLocation);
		RemeshModifier->SetUnscaledCoverage(UnscaledCoverage);
		Actor->AddInstanceComponent(RemeshModifier);
		RemeshModifier->SetAffectedMeshPartition(MeshPartition);
		// Attach to root component
		if (USceneComponent* RootComp = Actor->GetRootComponent())
		{
			RemeshModifier->AttachToComponent(RootComp, FAttachmentTransformRules::KeepWorldTransform);
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Warning,
				"CreateRemeshModifier: Actor has no root component, component will not be attached");
		}

		// Register the component
		RemeshModifier->RegisterComponent();

		return RemeshModifier;
	}

	AMeshPartition* CreateTestMeshSectioned(UWorld* InWorld, int InSectionCountX, int InSectionCountY, int InSectionWidth, int InSectionHeight, int InSectionWidthVertexCount, int InSectionHeightVertexCount)
	{
		AMeshPartition* MeshPartition = InWorld->SpawnActor<AMeshPartition>(AMeshPartition::StaticClass(), FTransform::Identity);
		UMeshPartitionEditorComponent* MeshPartitionEditorComponent = NewObject<UMeshPartitionEditorComponent>(MeshPartition, UMeshPartitionEditorComponent::StaticClass(), TEXT("MegaMeshEditorComponent"));
		MeshPartitionEditorComponent->SetForceSynchronousPreviewSectionBuild(true);
		MeshPartition->SetMeshPartitionComponent(MeshPartitionEditorComponent);

		for (int SectionIndexY = 0; SectionIndexY < InSectionCountY; ++SectionIndexY)
		{
			for (int SectionIndexX = 0; SectionIndexX < InSectionCountX; ++SectionIndexX)
			{
				Geometry::FRectangleMeshGenerator RectGen;
				RectGen.Width = InSectionWidth;
				RectGen.Height = InSectionHeight;
				RectGen.WidthVertexCount = InSectionWidthVertexCount;
				RectGen.HeightVertexCount = InSectionHeightVertexCount;
				MeshPartitionEditorComponent->SpawnBaseModifier(FDynamicMesh3(&RectGen.Generate()), {}, FTransform(FVector(SectionIndexX * InSectionWidth, SectionIndexY * InSectionHeight, 0.)));
			}
		}

		return MeshPartition;
	}
	
	bool LoadMeshes(TArray<TObjectPtr<UStaticMesh>>& OutMeshes, TArrayView<const TSoftObjectPtr<UStaticMesh>> InSoftPointers)
	{
		OutMeshes.Reserve(InSoftPointers.Num());

		for (TSoftObjectPtr<UStaticMesh> StaticMeshPtr : InSoftPointers)
		{
			UStaticMesh* Mesh = StaticMeshPtr.LoadSynchronous();
			if (Mesh == nullptr)
			{
				return false;
			}

			OutMeshes.Add(Mesh);
		}

		return true;
	}

	bool LoadWorldPartitionRegion(UWorld* InWorld, const FVector& InWorldBoxExtent)
	{
		FBox WorldBox(-InWorldBoxExtent, InWorldBoxExtent);
		UWorldPartition* WorldPartition = InWorld->GetWorldPartition();
		if (!WorldPartition)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "WorldPartition is null.");
			return false;
		}

		UWorldPartitionEditorLoaderAdapter* EditorLoaderAdapter =
			WorldPartition->CreateEditorLoaderAdapter<FLoaderAdapterShape>(InWorld, WorldBox, TEXT("Loaded Region"));
		if (!EditorLoaderAdapter)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "Editor loader adapter creation failed.");
			return false;
		}
		EditorLoaderAdapter->GetLoaderAdapter()->SetUserCreated(true);
		EditorLoaderAdapter->GetLoaderAdapter()->Load();

		return true;
	}

	bool ValidatePreviewMeshAgainstGroundTruth(const UMeshPartitionEditorComponent* InMeshPartitionComponent,
		const TArray<FString>& InGroundTruthPaths, FString& OutErrorMessage)
	{
		// Collect generated preview sections
		TArray<APreviewSection*> PreviewSections;
		InMeshPartitionComponent->ForAllPreviewSections([&](APreviewSection* InPreviewSection)
			{
				if (IsValid(InPreviewSection))
				{
					PreviewSections.Add(InPreviewSection);
				}
				return true;
			});
		if (PreviewSections.IsEmpty())
		{
			OutErrorMessage = TEXT("No preview sections generated");
			return false;
		}

		TArray<TSoftObjectPtr<UStaticMesh>> GroundTruthPaths;
		for (const FString& Path : InGroundTruthPaths)
		{
			GroundTruthPaths.Add(TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(Path)));
		}

		TArray<TObjectPtr<UStaticMesh>> GroundTruthPreviewMeshes;
		const bool bMeshesLoaded = TestUtils::LoadMeshes(
			GroundTruthPreviewMeshes,
			GroundTruthPaths);
		if (!bMeshesLoaded)
		{
			OutErrorMessage = TEXT("Failed to load ground truth meshes");
			return false;
		}

		// Validate counts match
		if (PreviewSections.Num() != GroundTruthPreviewMeshes.Num())
		{
			OutErrorMessage = FString::Printf(
				TEXT("Preview section count (%d) doesn't match ground truth count (%d)"),
				PreviewSections.Num(),
				GroundTruthPreviewMeshes.Num()
			);
			return false;
		}

		// Compare each generated mesh with corresponding ground truth
		for (int32 i = 0; i < PreviewSections.Num(); ++i)
		{
			const TSharedPtr<const FMeshData> GeneratedMesh = PreviewSections[i]->GetPreviewMesh();

			if (!GeneratedMesh.IsValid())
			{
				OutErrorMessage = FString::Printf(
					TEXT("Generated preview mesh at index %d is invalid"),
					i
				);
				return false;
			}

			if (!CompareMesh(GroundTruthPreviewMeshes[i], GeneratedMesh.Get()))
			{
				OutErrorMessage = FString::Printf(
					TEXT("Generated mesh at index %d doesn't match expected ground truth"),
					i
				);
				return false;
			}
		}

		return true;
	}

	bool CreateOrUpdateReference(const UMeshPartitionEditorComponent* MeshPartitionComponent, const FString& ReferencePackage,
	                             const TArray<FString>& ReferenceMeshNames, const FString& TestName)
	{
		FMessageLog MessageLog("AutomationTestingLog");

		auto Error = [&TestName, &MessageLog](const FString& Contents)
		{
			MessageLog.Error(FText::FromString(FString::Printf(
				TEXT("Errors when updating references for test '%s': %s."), *TestName, *Contents)));
		};

		auto ErrorForPath = [&TestName, &MessageLog](const FString& Contents, const FString& Path)
		{
			MessageLog.Error(FText::FromString(FString::Printf(
				TEXT("Errors when updating reference mesh for test '%s' in path '%s': %s."), *TestName, *Path, *Contents)));
		};

		const TArray<APreviewSection*> PreviewSections = [&MeshPartitionComponent]
		{
			TArray<APreviewSection*> Result;
			MeshPartitionComponent->ForAllPreviewSections([&Result](APreviewSection* InPreviewSection)
			{
				if (IsValid(InPreviewSection))
				{
					Result.Add(InPreviewSection);
				}
				return true;
			});
			return Result;
		}();

		if (PreviewSections.IsEmpty())
		{
			Error(TEXT("No preview sections available"));
			return false;
		}

		if (PreviewSections.Num() != ReferenceMeshNames.Num())
		{
			Error(FString::Printf(
				TEXT("Preview section count (%d) does not match reference mesh count (%d)"), PreviewSections.Num(), ReferenceMeshNames.Num()));
			return false;
		}

		UEditorAssetSubsystem* EditorAssetSubsystem = GEditor->GetEditorSubsystem<UEditorAssetSubsystem>();
		if (!EditorAssetSubsystem)
		{
			Error(TEXT("Failed to access Editor Asset Subsystem"));
			return false;
		}

		TArray<TObjectPtr<UStaticMesh>> ReferenceMeshes;

		for (const FString& ReferenceMeshName : ReferenceMeshNames)
		{
			const FString ReferenceMeshPath = ReferencePackage + "." + ReferenceMeshName;
			TSoftObjectPtr<UStaticMesh> ReferenceMeshPtr = TSoftObjectPtr<UStaticMesh>(FSoftObjectPath(ReferenceMeshPath));

			if (ReferenceMeshPtr.IsNull())
			{
				ErrorForPath(TEXT("Path for reference mesh is not valid"), ReferenceMeshPath);
				return false;
			}

			if (!EditorAssetSubsystem->DoesAssetExist(ReferenceMeshPath))
			{
				// Create reference asset

				FPackageName::EErrorCode ErrorCode;
				if (!FPackageName::IsValidTextForLongPackageName(ReferencePackage, &ErrorCode))
				{
					Error(FString::Printf(
						TEXT("Reference package path is not valid as long package name (%s)"),
						*FPackageName::FormatErrorAsString(ReferencePackage, ErrorCode)));
					return false;
				}

				AssetUtils::FStaticMeshAssetOptions Options;
				Options.NewAssetPath = FPackageName::ObjectPathToPackageName(ReferenceMeshPath);;
				Options.NumSourceModels = 1;
				Options.bEnableRecomputeNormals = false;
				Options.bEnableRecomputeTangents = false;
				Options.bCreatePhysicsBody = true;
				Options.CollisionType = CTF_UseComplexAsSimple;
				Options.bGenerateNaniteEnabledMesh = true;
				Options.NaniteSettings.FallbackPercentTriangles = 1.0f;

				if (FPackageName::DoesPackageExist(ReferencePackage))
				{
					Options.UsePackage = FindPackage(nullptr, *ReferencePackage);
					if (Options.UsePackage == nullptr)
					{
						Options.UsePackage = LoadPackage(nullptr, *ReferencePackage, LOAD_None);
					}
				}

				AssetUtils::FStaticMeshResults Results;
				if (AssetUtils::CreateStaticMeshAsset(Options, Results) != AssetUtils::ECreateStaticMeshResult::Ok)
				{
					ErrorForPath(TEXT("Could not create reference mesh asset"), ReferenceMeshPath);
					return false;
				}

				check(Results.StaticMesh);

				ReferenceMeshes.Add(Results.StaticMesh);
			}
			else
			{
				// Load reference asset

				UStaticMesh* ReferenceMesh = ReferenceMeshPtr.LoadSynchronous();
				if (ReferenceMesh == nullptr)
				{
					ErrorForPath(TEXT("Could not load reference mesh asset"), ReferenceMeshPath);
					return false;
				}

				ReferenceMeshes.Add(ReferenceMesh);
			}
		}

		check(PreviewSections.Num() == ReferenceMeshes.Num());

		// Setup reference contents

		for (int32 i = 0; i < ReferenceMeshes.Num(); ++i)
		{
			TObjectPtr<UStaticMesh> ReferenceMesh = ReferenceMeshes[i];

			// Clear any pre-existing data
			const int32 NumLODs = ReferenceMesh->GetNumSourceModels();
			for (int32 LODIndex = 0; LODIndex < NumLODs; ++LODIndex)
			{
				FStaticMeshSourceModel& SourceModel = ReferenceMesh->GetSourceModel(LODIndex);
				if (SourceModel.StaticMeshDescriptionBulkData)
				{
					SourceModel.StaticMeshDescriptionBulkData->RemoveMeshDescription();
					SourceModel.StaticMeshDescriptionBulkData->CommitMeshDescription(false);
				}
			}

			// Set new reference data from preview mesh
			const TSharedPtr<const FMeshData> PreviewMesh = PreviewSections[i]->GetPreviewMesh();
			check(PreviewMesh.IsValid());
			FMeshDescription PreviewMeshDescription;
			FStaticMeshAttributes(PreviewMeshDescription).Register();
			PreviewMesh->ConvertToMeshDescription(PreviewMeshDescription);
			ReferenceMesh->SetNumSourceModels(1);
			FStaticMeshSourceModel& LOD0 = ReferenceMesh->GetSourceModel(0);
			check(LOD0.StaticMeshDescriptionBulkData);
			UMeshDescriptionBase* ReferenceMeshDescription = LOD0.StaticMeshDescriptionBulkData->CreateMeshDescription();
			ReferenceMeshDescription->SetMeshDescription(MoveTemp(PreviewMeshDescription));
			LOD0.StaticMeshDescriptionBulkData->CommitMeshDescription(true);

			ReferenceMesh->MarkPackageDirty();
			ReferenceMesh->PostEditChange();
		}

		MessageLog.Warning(FText::FromString(FString::Printf(
			TEXT("Updated reference assets for test '%s' in package at path '%s'."), *TestName, *ReferencePackage)));

		return true;
	}
}

#endif // WITH_DEV_AUTOMATION_TESTS
