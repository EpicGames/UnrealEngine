// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR
#if WITH_DEV_AUTOMATION_TESTS

#include "Test/MeshPartitionTestUtils.h"
#include "CQTest.h"
#include "Editor.h"
#include "EditorWorldUtils.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierActor.h"
#include "MeshPartitionPreviewSection.h"
#include "Engine/StaticMesh.h"
#include "Curves/CurveFloat.h"
#include "Engine/Texture2D.h"
#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Tests/AutomationCommon.h"
#include "Tests/AutomationEditorCommon.h"

using namespace UE::MeshPartition;

/*
 * Tests for Mesh Terrain Texture Patch Modifier - Golden Path
 * QMetry: UE-TC-21020
 */
TEST_CLASS_WITH_FLAGS(TexturePatchGoldenPath, "MeshPartition.Modifier", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
{
	// Test environment and core objects
	TUniquePtr<FScopedEditorWorld> ScopedEditorWorld;
	// CVars
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCWrite;
	TOptional<TestUtils::FScopedCVarOverride> DisableDDCRead;

	UWorld* World = nullptr;
	AMeshPartition* MeshPartition = nullptr;
	UMeshPartitionEditorComponent* MeshPartitionComponent = nullptr;
	AActor* TexturePatchActor = nullptr;
	UTexturePatchModifier* TexturePatchModifier = nullptr;
	UTexturePatchHeightEntry* TexturePatchHeightEntry = nullptr;

	// Test Configuration
	const FString LevelPath = TEXT("/MeshPartition/UnitTests/MeshPartitionUnitTests");
	const FString ReferencePath = TEXT("/MeshPartition/UnitTests/TexturePatchModifierTests");
	const int32 MeshWidth = 1000;
	const int32 MeshHeight = 1000;
	const int32 MeshWidthVertexCount = 500;
	const int32 MeshHeightVertexCount = 500;

	BEFORE_EACH()
	{
		// Configure minimal editor world
		FWorldInitializationValues InitializationValues;
		InitializationValues.RequiresHitProxies(false);
		InitializationValues.ShouldSimulatePhysics(false);
		InitializationValues.EnableTraceCollision(false);
		InitializationValues.CreateNavigation(false);
		InitializationValues.CreateAISystem(false);
		InitializationValues.AllowAudioPlayback(false);
		InitializationValues.CreatePhysicsScene(true);
		InitializationValues.CreateWorldPartition(true);

#if AUTOMATED_TESTING_EXECUTE_IN_EDITOR_WORLD
		World = GEditor->GetEditorWorldContext().World();
#else
		// Spawn world
		ScopedEditorWorld = FAutomationEditorCommonUtils::CreateScopedEditorWorld(LevelPath, InitializationValues);
		ASSERT_THAT(IsNotNull(ScopedEditorWorld.Get(), TEXT("Failed to create scoped editor world")));

		World = ScopedEditorWorld->GetWorld();
#endif

		ASSERT_THAT(IsNotNull(World, TEXT("Failed to create valid test world")));

		// Need to disable DDC read/write, so that generated preview sections aren't written to DDC
		DisableDDCWrite.Emplace(TEXT("MegaMesh.Preview.EnableDDCWrite"), TEXT("0"));
		DisableDDCRead.Emplace(TEXT("MegaMesh.Preview.EnableDDCRead"), TEXT("0"));

		MeshPartition = TestUtils::CreateTestMesh(
			World, MeshWidth, MeshHeight, MeshWidthVertexCount, MeshHeightVertexCount
		);
		ASSERT_THAT(IsNotNull(MeshPartition, TEXT("Failed to create MeshPartition")));

		MeshPartitionComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
		ASSERT_THAT(IsNotNull(MeshPartitionComponent, TEXT("Failed to get MeshPartitionComponent")));

		// Pausing Transformer pipeline during the test to avoid potential race condition
		MeshPartitionComponent->SetPauseTransformerPipeline(true);

		TexturePatchActor = CreateTexturePatchModifierActor();
		ASSERT_THAT(IsNotNull(TexturePatchActor, TEXT("Failed to create texture patch actor")));

		TexturePatchModifier = CreateTexturePatchModifier(FVector::ZeroVector);
		ASSERT_THAT(IsNotNull(TexturePatchModifier, TEXT("Failed to create Texture Patch Modifier")));

		TexturePatchHeightEntry = NewObject<UTexturePatchHeightEntry>(
			TexturePatchModifier,
			UTexturePatchHeightEntry::StaticClass()
		);

		UTexture2D* Texture = LoadObject<UTexture2D>(nullptr, TEXT("/MeshPartition/UnitTests/Textures/T_UELogo_FromTG.T_UELogo_FromTG"));
		TexturePatchHeightEntry->SetTextureAsset(Texture);

		TexturePatchModifier->SetHeightChannel(TexturePatchHeightEntry);

		// Load WorldPartition Region
		const bool bSuccessfullyLoadedWorldPartition = TestUtils::LoadWorldPartitionRegion(World);
		ASSERT_THAT(IsTrue(bSuccessfullyLoadedWorldPartition, TEXT("Failed to load WorldPartition region")));
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_TextureAsset_Input_Projects_To_MeshTerrain)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_TextureAsset_Input_Projects_To_MeshTerrain");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_UnscaledPatchCoverage_Adjusts_Terrain)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the new unscaled coverage value"), [this]()
				{
					const FVector2D NewUnscaledCoverage = {800.0, 800.0};
					TexturePatchModifier->SetUnscaledCoverage(NewUnscaledCoverage);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_UnscaledPatchCoverage_Adjusts_Terrain");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_Rotation_Applies_Correct_Angle)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the new rotation value"), [this]()
				{
					const FRotator NewRotation = FRotator(45.0f, 0.0f, 0.0f);
					TexturePatchModifier->SetRelativeRotation(NewRotation);

					TexturePatchModifier->TriggerUpdate();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_Rotation_Applies_Correct_Angle");
				CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_Translation_Moves_To_Target_Location)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the new translation value"), [this]()
				{
					const FVector Translation = FVector(50.0f, 0.0f, 0.0f);
					TexturePatchModifier->SetRelativeLocation(Translation);

					TexturePatchModifier->TriggerUpdate();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_Translation_Moves_To_Target_Location");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_TextureChannel_Projects_Specified_Channel)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the texture channel to alpha value"), [this]()
				{
					// We pick alpha (3) for texture channel to project
					constexpr int32 NewTextureChannelIndex = 3;
					TexturePatchHeightEntry->SetTextureChannelIndex(NewTextureChannelIndex);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_TextureChannel_Projects_Specified_Channel");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_AlphaMode_ThisAlpha_Uses_Specified_Channel)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the alpha blending mode to ThisAlphaChannel"), [this]()
				{
					constexpr ETexturePatchAlphaMode NewAlphaMode = ETexturePatchAlphaMode::ThisAlphaChannel;
					TexturePatchHeightEntry->SetAlphaBlendingMode(NewAlphaMode);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = "Verify_TexturePatch_Modifier_AlphaMode_ThisAlpha_Uses_Specified_Channel";
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_BlendMode_Max_Limits_Upward_Vertices)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the texture blend mode to max"), [this]()
				{
					constexpr ETexturePatchBlendMode NewBlendMode = ETexturePatchBlendMode::Max;
					TexturePatchHeightEntry->SetTexturePatchBlendMode(NewBlendMode);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_BlendMode_Max_Limits_Upward_Vertices");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_ApplyComponentZScale_Scales_ZAxis)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set ComponentZScale to true"), [this]()
				{
					constexpr bool bApplyComponentZScale = true;
					TexturePatchModifier->SetApplyComponentZScale(bApplyComponentZScale);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Change Z scale"), [this]()
				{
					const FVector NewScale = { 1.0f, 1.0f, 5.0f };
					TexturePatchModifier->SetRelativeScale3D(NewScale);
					TexturePatchModifier->TriggerUpdate();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_ApplyComponentZScale_Scales_ZAxis");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_ValueCurve_Remaps_TextureValues)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the value curve to pre-determined float curve"), [this]()
				{
					TexturePatchHeightEntry->SetUseValueCurve(true);

					const FString CurvePath = TEXT("/MeshPartition/UnitTests/TexturePatchModifierTests/TexturePatchModifierValueCurve.TexturePatchModifierValueCurve");
					UCurveFloat* NewModifierCurve = LoadObject<UCurveFloat>(nullptr, CurvePath);
					ASSERT_THAT(IsNotNull(NewModifierCurve, TEXT("Failed to load modifier float curve asset")));

					TexturePatchHeightEntry->SetValueCurve(NewModifierCurve);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_ValueCurve_Remaps_TextureValues");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_TessellationFast_Applies)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the adaptive tessellation mode to fast"), [this]()
				{
					constexpr ETexturePatchTessellationMode TessellationMode = ETexturePatchTessellationMode::AdaptiveFast;
					TexturePatchModifier->SetAdaptiveTessellationMode(TessellationMode);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_TessellationFast_Applies");
					CompareToReferenceAsset(TestName);
				});
	}

	TEST_METHOD(Verify_TexturePatch_Modifier_IsDisabled_Removes_Modifier)
	{
		TestCommandBuilder
			.Do(TEXT("Notify MeshPartition that modifier has been assigned"), [this]()
				{
					MeshPartitionComponent->OnModifierAssigned();
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Set the Disabled flag to true"), [this]()
				{
					TexturePatchModifier->SetIsDisabledFlag(true);
				})
			.Until(TEXT("Wait for preview build completion"), [this]()
				{
					return !MeshPartitionComponent->IsModifierParticipatingInActivePreviewSectionBuild(TexturePatchModifier);
				})
			.Then(TEXT("Validate preview mesh against ground truth"), [this]()
				{
					const FString TestName = TEXT("Verify_TexturePatch_Modifier_IsDisabled_Removes_Modifier");
					CompareToReferenceAsset(TestName);
				});
	}

	AFTER_EACH()
	{
		if (MeshPartitionComponent)
		{
			MeshPartitionComponent->SetPauseTransformerPipeline(false);
		}

		ScopedEditorWorld.Reset();
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

private:
	AActor* CreateTexturePatchModifierActor(const FActorSpawnParameters& InSpawnParams = FActorSpawnParameters()) const
	{
		AActor* Actor = World->SpawnActor<AActor>(AActor::StaticClass(), FTransform::Identity, InSpawnParams);
		if (!IsValid(Actor))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateTexturePatchModifierActor: Failed to create TexturePatchModifier actor");
			return nullptr;
		}

		USceneComponent* SceneComponent = NewObject<USceneComponent>(Actor, TEXT("SceneComponent"), RF_Transactional);
		Actor->AddInstanceComponent(SceneComponent);
		Actor->SetRootComponent(SceneComponent);
		SceneComponent->RegisterComponent();

		Actor->AttachToActor(MeshPartition, FAttachmentTransformRules::KeepWorldTransform);

		return Actor;
	}

	UTexturePatchModifier* CreateTexturePatchModifier(const FVector& ModifierWorldLocation) const
	{
		if (!IsValid(TexturePatchActor))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateTexturePatchModifier: Actor is null or invalid");
			return nullptr;
		}

		if (!IsValid(MeshPartition))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateTexturePatchModifier: MeshPartition is null or invalid");
			return nullptr;
		}

		// Create the component
		UTexturePatchModifier* Modifier = NewObject<UTexturePatchModifier>(
			TexturePatchActor,
			UTexturePatchModifier::StaticClass()
		);

		if (!Modifier)
		{
			UE_LOGF(LogMegaMeshEditor, Error, "CreateTexturePatchModifier: Failed to create UTexturePatchModifier object");
			return nullptr;
		}

		// Configure the component
		Modifier->SetWorldLocation(ModifierWorldLocation);
		TexturePatchActor->AddInstanceComponent(Modifier);
		Modifier->SetAffectedMeshPartition(MeshPartition);
		if (USceneComponent* RootComp = TexturePatchActor->GetRootComponent())
		{
			Modifier->AttachToComponent(RootComp, FAttachmentTransformRules::KeepWorldTransform);
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Warning,
				"CreateTexturePatchModifier: Actor has no root component, component will not be attached");
		}

		// Register the component
		Modifier->RegisterComponent();

		return Modifier;
	}

	void CompareToReferenceAsset(const FString& TestName)
	{
		const FString ReferencePackagePath = ReferencePath / TestName;
		const FString ReferenceAssetPath = ReferencePackagePath + "." + TestName;

		if (TestUtils::ShouldUpdateReferenceAssets())
		{
			[[maybe_unused]] const bool bSuccess =
				TestUtils::CreateOrUpdateReference(MeshPartitionComponent, ReferencePackagePath, {TestName}, TestName);
		}

		FString ErrorMessage;
		bool bValid = TestUtils::ValidatePreviewMeshAgainstGroundTruth(
			MeshPartitionComponent,
			{ ReferenceAssetPath },
			ErrorMessage);

		ASSERT_THAT(IsTrue(bValid, *ErrorMessage));
	}
};
#endif
#endif