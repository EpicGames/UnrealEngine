// Copyright Epic Games, Inc. All Rights Reserved.

#include "WorldPartitionMeshPartitionBuilder.h"

#include "AssetCompilingManager.h"
#include "EditorBuildUtils.h"
#include "Engine/StaticMesh.h"
#include "HAL/PlatformFileManager.h"
#include "Internationalization/Regex.h"
#include "ISourceControlModule.h"
#include "Engine/LevelStreaming.h"
#include "MeshPartition.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionModifierTaskGraph.h"
#include "MeshPartitionEditorSubsystem.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Misc/ScopedSlowTask.h"
#include "ObjectTools.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "UObject/SavePackage.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"
#include "EngineUtils.h"
#include "Misc/MessageDialog.h"
#include "MeshPartitionMeshBuilder.h"
#include "Algo/ForEach.h"
#include "MeshPartitionActorDescUtils.h"
#include "MeshPartitionModifierUtils.h"
#include "MeshPartitionCompiledSectionActorDesc.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/Optional.h"
#include "UObject/UObjectIterator.h"
#include "UObject/Class.h"
#include "Commandlets/Commandlet.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "MeshPartitionEditorUtils.h"
#include "MeshPartitionWorldPartitionHelpers.h"
#include "MeshPartitionDataLayerContainer.h"
#include "MeshPartitionPlatformCellTransformer.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

#define LOCTEXT_NAMESPACE "MeshPartitionBuilder"

static FAutoConsoleVariable CVarMeshPartitionMeshBuildCommandWaitForDebugger(
	TEXT("MeshPartition.Build.WaitForDebugger"),
	0,
	TEXT("Enables wait for debugger when starting a Mesh Partition build process"));

namespace UE::MeshPartition
{

namespace WorldPartitionMeshPartitionBuilderLocals
{
	int32 GetTotalNumberOfSectionsToBuild(MeshPartition::FModifierDescriptorCache* InDescriptorCache)
	{
		int32 TotalNumberOfSectionsToBuild = 0;

		if (!ensure(InDescriptorCache != nullptr))
		{
			return 0;
		}

		InDescriptorCache->ForAllCachedDescriptors([&TotalNumberOfSectionsToBuild](const FSoftObjectPath& MeshPartitionPath, const TSharedPtr<MeshPartition::FCachedDescriptors>& CachedDescriptors)
		{
			CachedDescriptors->ForAllBuildVariants([&TotalNumberOfSectionsToBuild](const MeshPartition::FModifierGrouping& BuildVariant)
			{
				TotalNumberOfSectionsToBuild += BuildVariant.GetModifierGroups().Num();
				return true;
			});

			return true;
		});

		return TotalNumberOfSectionsToBuild;
	}

	void LogBuildProgress(const int32 InTotalSectionNumber, const int32 InTotalNumberOfSectionsToBuild)
	{
		UE_LOGF(LogMegaMeshEditor, Display, "[%i / %i] Building compiled sections...", InTotalSectionNumber, InTotalNumberOfSectionsToBuild);
	}

	TNotNull<const UMeshPartitionDefinition*> ResolveDefinitionOrDefault(const AMeshPartition* InMeshPartition)
	{
		if (const UMeshPartitionDefinition* Definition = InMeshPartition->GetMeshPartitionDefinition())
		{
			return Definition;
		}

		UE_LOGF(LogMegaMeshEditor, Verbose, "No Mesh Partition Definition provided for %ls, using default settings.", *InMeshPartition->GetActorNameOrLabel());
		return UMeshPartitionDefinition::StaticClass()->GetDefaultObject<UMeshPartitionDefinition>();
	}
}

UWorldPartitionMeshPartitionBuilder::UWorldPartitionMeshPartitionBuilder(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
}

bool UWorldPartitionMeshPartitionBuilder::PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper)
{
	TArray<FString> Tokens, Switches;
	UCommandlet::ParseCommandLine(*GetBuilderArgs(), Tokens, Switches);
	return UCommandlet::ApplyCommandLineSwitches(this, Switches);
}

bool UWorldPartitionMeshPartitionBuilder::RunInternal(UWorld* InWorld, const FCellInfo& InCellInfo, FPackageSourceControlHelper& InPackageHelper)
{
	using namespace Utils;

	WorldPartition = InWorld->GetWorldPartition();
	if (WorldPartition == nullptr)
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Failed to retrieve WorldPartition, cannot run Mesh Partition Build for World %ls.", *InWorld->GetName());
		return false;
	}

	MeshPartition::FModifierDescriptorCache* DescriptorCache = UMeshPartitionEditorSubsystem::GetDescriptorCache();
	if (DescriptorCache == nullptr)
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Cannot access Mesh Partition DescriptorCache, cannot run Mesh Partition Build for World %ls.", *InWorld->GetName());
		return false;
	}

	TArray<FGuid> AllMeshPartitionGuidsInWorld;
	DescriptorCache->UpdateCacheForAllMeshPartitionsInWorld(InWorld, &AllMeshPartitionGuidsInWorld);

	// The BuildKey is a unique identifier to tag all compiled sections built by this build execution (allowing us to remove old ones).
	const FGuid BuildKey = FGuid::NewGuid();

	UE_LOGF(LogMegaMeshEditor, Display, "Build for World %ls (%d Mesh Partitions, BuildKey %.8ls)", *InWorld->GetName(), AllMeshPartitionGuidsInWorld.Num(), *BuildKey.ToString());

	CompiledSectionActorDescGuidsToKeep.Empty();
	TotalBuiltCompiledSections = 0;
	TotalFailedCompiledSections = 0;
	TotalReusedCompiledSections = 0;
	TotalNumberOfSectionsToBuild = WorldPartitionMeshPartitionBuilderLocals::GetTotalNumberOfSectionsToBuild(DescriptorCache);
	ModifiedFiles.Empty();

	UE_LOGF(LogMegaMeshEditor, Display, "[%i / %i] Initializing compiled sections build...", 0, TotalNumberOfSectionsToBuild);

	// Wrap the package helper with the file-tracking adapter so all checkouts/adds/saves performed via this helper
	// auto-record into ModifiedFiles for source control submission at the end of the build.
	FSourceControlHelper SourceControlHelper(InPackageHelper, ModifiedFiles);

	int32 TotalFailedMeshPartitions = 0;

	// Track definitions referenced by mesh partitions in this world so we can prune stale container entries after the build.
	TSet<const UMeshPartitionDefinition*> UsedDefinitions;

	// For each MeshPartition in the world
	for (const FGuid& MeshPartitionGUID  : AllMeshPartitionGuidsInWorld)
	{
		// Use FWorldPartitionReference to ensure the AMeshPartition actor is loaded, then invoke BuildMeshPartition on it
		FWorldPartitionReference MeshPartitionActorRef(WorldPartition.Get(), MeshPartitionGUID);
		const AMeshPartition* MeshPartition = Cast<AMeshPartition>(MeshPartitionActorRef.GetActor());

		// Note: Not every MeshPartition GUID references a valid mesh partition actor -- for example, there may be stale compiled sections that reference the GUID of old mesh partitions that have since been deleted
		if (MeshPartition != nullptr)
		{
			UsedDefinitions.Add(WorldPartitionMeshPartitionBuilderLocals::ResolveDefinitionOrDefault(MeshPartition));

			if (!BuildMeshPartition(MeshPartition, BuildKey, *DescriptorCache, SourceControlHelper))
			{
				UE_LOGF(LogMegaMeshEditor, Error, "BuildMeshPartition failed for '%ls' (%.6ls)", *MeshPartition->GetActorNameOrLabel(), *MeshPartitionGUID.ToString());
				++TotalFailedMeshPartitions;
			}
		}
	}

	// Prune stale references from the data layer container (e.g. left over after a map "Save As") and persist if it changed.
	// Skip when no partitions were visited -- an empty used-set would wipe every entry.
	if (!UsedDefinitions.IsEmpty())
	{
		if (AMeshPartitionDataLayerContainer* Container = AMeshPartitionDataLayerContainer::Get(InWorld))
		{
			const int32 NumPruned = Container->PruneUnusedDefinitions(UsedDefinitions);
			
			if (NumPruned > 0)
			{
				UE_LOGF(LogMegaMeshEditor, Display, "Pruned %d stale data layer container entries.", NumPruned);
				SavePackages({ Container->GetPackage() }, SourceControlHelper);
			}
		}
	}

	// Remove any out-of-date CompiledSections
	UE_LOGF(LogMegaMeshEditor, Display, "Removing out-of-date compiled sections from World %ls", *InWorld->GetName());

	TArray<UPackage*> PackagesToCleanup;
	int32 TotalRemovedCompiledSections = 0;
	int32 TotalFailedToRemoveCompiledSections = 0;
	TArray<TWeakObjectPtr<AActor>> ActorsToUnload;
	FWorldPartitionHelpers::ForEachActorDescInstance(WorldPartition, MeshPartition::ACompiledSection::StaticClass(),
	[InWorld, WorldPartition = WorldPartition, BuildKey, CompiledSectionActorDescGuidsToKeep = CompiledSectionActorDescGuidsToKeep, &TotalRemovedCompiledSections, &TotalFailedToRemoveCompiledSections, &PackagesToCleanup, &ActorsToUnload, &SourceControlHelper](const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		const FGuid& ActorDescGuid = ActorDescInstance->GetGuid();
		if (CompiledSectionActorDescGuidsToKeep.Contains(ActorDescGuid))
		{
			// we're keeping this existing compiled section as part of an incremental build, don't delete it.
			return true;
		}

		// if the actor has a build key, it is a compiled section we should check
		if (const MeshPartition::FCompiledSectionActorDesc* CompiledSectionActorDesc = MeshPartition::FCompiledSectionActorDesc::GetFromActorDescInstance(*ActorDescInstance))
		{
			if (CompiledSectionActorDesc->GetBuildInfo().BuildKey == BuildKey)
			{
				// this actor was just built by this builder, don't delete it.
				return true;
			}
		}

		// the actor is not in our keep list, and was not built by this builder -- if it is a compiled section, then delete it
		if (ActorDescInstance->GetActorNativeClass() == MeshPartition::ACompiledSection::StaticClass())
		{
			{
				// try to load the compiled section into memory (must be loaded to delete it...)
				FWorldPartitionReference CompiledSectionActorRef(WorldPartition.Get(), ActorDescGuid);
				MeshPartition::ACompiledSection* CompiledSection = CastChecked<MeshPartition::ACompiledSection>(ActorDescInstance->GetActor());
				if (CompiledSection != nullptr)					
				{
					const MeshPartition::FCompiledSectionBuildInfo& BuildInfo = CompiledSection->GetBuildInfo();
					UE_LOGF(LogMegaMeshEditor, Display, "  Deleting compiled section %.6ls (BuildKey %.8ls, %d Modifiers, Hash %.8ls, BuildVariant %ls %.8ls",
						*ActorDescGuid.ToString(),
						*BuildInfo.BuildKey.ToString(),
						BuildInfo.BaseModifierPaths.Num(),
						*BuildInfo.ModifiersHash.ToString(),
						*BuildInfo.BuildVariantName.ToString(),
						*BuildInfo.BuildVariantHash.ToString()
					);

					check(BuildInfo.BuildKey != BuildKey); // shouldn't be deleting any actors that were just built

					// before deleting, check if the package is exclusively checked out by someone else in source control.
					// If so, skip deletion to avoid a conflict
					// (we don't require deletion, the system can filter out old out-of-date files)
					// If performance becomes a concern, the file states could be batch-prefetched with SCCProvider.UpdateStatus(...) before the ForEachActorDescInstance call and then use EStateCacheUsage::Use here.
					{
						const FString PackageFilename = SourceControlHelpers::PackageFilename(CompiledSection->GetPackage());
						ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();
						if (SCCProvider.IsEnabled())
						{
							FSourceControlStatePtr SCCState = SCCProvider.GetState(PackageFilename, EStateCacheUsage::ForceUpdate);
							if (SCCState.IsValid())
							{
								if (SCCState->IsCheckedOutOther())
								{
									// don't attempt to delete if it is exclusively checked out by someone else
									UE_LOGF(LogMegaMeshEditor, Display,
										"  Skipping deletion of compiled section %.6ls: package '%ls' is exclusively checked out by another user (%ls).",
										*ActorDescGuid.ToString(), *PackageFilename, *SCCState->GetDisplayName().ToString());
									TotalFailedToRemoveCompiledSections++;
									return true;
								}
							}
						}
					}

					// Open the package for delete in source control (auto-records as FileDeleted in ModifiedFiles).
					if (!SourceControlHelper.Delete(CompiledSection->GetPackage()))
					{
						UE_LOGF(LogMegaMeshEditor, Warning,
							"  Failed to delete compiled section %.6ls: source control could not open package '%ls' for delete.",
							*ActorDescGuid.ToString(), *CompiledSection->GetPackage()->GetName());
						TotalFailedToRemoveCompiledSections++;
						return true;
					}
					PackagesToCleanup.Emplace(CompiledSection->GetPackage());
					ActorsToUnload.Add(CompiledSection);
					InWorld->DestroyActor(CompiledSection);
					TotalRemovedCompiledSections++;
				}
				// closing the scope releases the reference
			}

			// run a garbage collection tick to evict the deleted actor and related resources
			if (FWorldPartitionHelpers::ShouldCollectGarbage())
			{
				// cleanup any pending deleted packages
				ObjectTools::CleanupAfterSuccessfulDelete(PackagesToCleanup, true);
				PackagesToCleanup.Empty();

				// Simulate an engine tick to make sure engine & render resources that are queued for deletion are processed.
				FWorldPartitionHelpers::FakeEngineTick(InWorld);
				FWorldPartitionHelpers::DoCollectGarbage();

				// Double check that our deleted actors were actually garbage collected
				UE::MeshPartition::EditorUtils::ValidateObjectsAreUnloaded(ActorsToUnload);
				ActorsToUnload.Empty();
			}
		}

		return true;
	});
	
	// cleanup any remaining deleted packages
	if (PackagesToCleanup.Num() > 0)
	{
		ObjectTools::CleanupAfterSuccessfulDelete(PackagesToCleanup, true);
		PackagesToCleanup.Empty();
	}
	
	// If the platform cell transformer is not yet installed to the world and we have compiled sections, install it now:
	if ((TotalBuiltCompiledSections + TotalReusedCompiledSections) > 0)
	{
		if (!WorldPartition->HasRuntimeCellTransformerOfType(MeshPartition::UPlatformCellTransformer::StaticClass()))
		{
			WorldPartition->AddRuntimeCellTransformerOfType(TSubclassOf<UWorldPartitionRuntimeCellTransformer>(MeshPartition::UPlatformCellTransformer::StaticClass()));
			SavePackages( { WorldPartition->GetPackage() }, SourceControlHelper);
		}
	}

	UE_LOGF(LogMegaMeshEditor, Display, "Finished Build for World %ls (Built %d, Reused %d, Failed %d, Removed %d, Could Not Remove %d Compiled Sections, %d Failed Mesh Partitions)", *InWorld->GetName(),
		TotalBuiltCompiledSections,
		TotalReusedCompiledSections,
		TotalFailedCompiledSections,
		TotalRemovedCompiledSections,
		TotalFailedToRemoveCompiledSections,
		TotalFailedMeshPartitions);

	const bool bSubmitSuccess = SubmitToSourceControl(InWorld);
	if (!bSubmitSuccess)
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Failed to submit Mesh Partition build results to source control for World %ls", *InWorld->GetName());
	}

	return bSubmitSuccess && (TotalFailedCompiledSections == 0) && (TotalFailedMeshPartitions == 0);
}

bool UWorldPartitionMeshPartitionBuilder::BuildMeshPartition(const AMeshPartition* InMeshPartition, const FGuid& BuildKey, MeshPartition::FModifierDescriptorCache& DescriptorCache, FSourceControlHelper& InSourceControlHelper)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionMeshPartitionBuilder::BuildMeshPartition);

	const FGuid& MeshPartitionGUID = InMeshPartition->GetActorGuid();
	if (!MeshPartitionGUID.IsValid())
	{
		return false;
	}

	UWorld* World = InMeshPartition->GetWorld();
	check(World);

	// Create a new build info -- this gets recorded on all compiled sections produced here.
	MeshPartition::FCompiledSectionBuildInfo BuildInfo;
	BuildInfo.BuildKey = BuildKey;
	BuildInfo.MegaMeshGUID = MeshPartitionGUID;
	BuildInfo.MegaMeshPath = InMeshPartition;

	TSharedPtr<const MeshPartition::FCachedDescriptors> Descriptors = DescriptorCache.GetCachedDescriptors(InMeshPartition);
	if (!Descriptors.IsValid())
	{
		UE_LOGF(LogMegaMeshEditor, Warning, "Could not gather descriptors for the Mesh Partition '%ls'", *InMeshPartition->GetActorNameOrLabel());
		return false;
	}
	TArrayView<const FCompiledSectionDescriptor> OldCompiledSections = Descriptors->GetCompiledSections();

	// Use FWorldPartitionReference to load/unload everything in scope
	FWorldPartitionReference MeshPartitionActorRef(WorldPartition.Get(), MeshPartitionGUID);
	check(InMeshPartition == CastChecked<AMeshPartition>(MeshPartitionActorRef.GetActor()));

	TNotNull<const UMeshPartitionDefinition*> MeshPartitionDefinition = WorldPartitionMeshPartitionBuilderLocals::ResolveDefinitionOrDefault(InMeshPartition);
	BuildInfo.SetMegaMeshDefinition(MeshPartitionDefinition);

	TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariants = MeshPartitionDefinition->GetCompiledSectionBuildVariants();
	check(BuildVariants.Num() > 0);

	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(InMeshPartition->GetMeshPartitionComponent());
	check(MeshPartitionEditorComponent != nullptr);
	// avoid level streaming triggering a preview section build while we compiled section resources are being built.
	MeshPartitionEditorComponent->SetPreviewSectionBuildEnabled(false);

	const FTransform& Transform = InMeshPartition->GetTransform();

	UE_LOGF(LogMegaMeshEditor, Display, "-- %.6ls Build Mesh Partition %ls (%d Variants, %d Existing Compiled Sections",
		*MeshPartitionGUID.ToString(), *InMeshPartition->GetActorNameOrLabel(),
		BuildVariants.Num(), OldCompiledSections.Num());

	if (AMeshPartitionDataLayerContainer* Container = AMeshPartitionDataLayerContainer::GetOrCreate(World))
	{
		TArray<UPackage*> PackagesToSave;
		if (Container->UpdateDataLayersFromDefinition(MeshPartitionDefinition))
		{
			PackagesToSave.Add(Container->GetPackage());
		}
		if (Container->InitializeDataLayerInstancesWithWorld(MeshPartitionDefinition))
		{
			PackagesToSave.Add(World->GetWorldDataLayers()->GetPackage());
		}

		SavePackages(PackagesToSave, InSourceControlHelper);
	}

	// Iterate over all the BuildVariants of the MeshPartition
	for (const MeshPartition::FCompiledSectionBuildVariant& BuildVariant : BuildVariants)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(
			*FString::Format(TEXT("UWorldPartitionMeshPartitionBuilder::BuildMeshPartitionVariant {0}"), {BuildVariant.Name.ToString()}));

		// Record the build variant, so at cook time we can filter to the proper variant for each platform
		BuildInfo.BuildVariantName = BuildVariant.Name;

		FDependencyHash DependencyHash;
		InMeshPartition->GatherDependencies(DependencyHash);
		BuildInfo.GetMegaMeshDefinition()->GatherDependencies(DependencyHash, BuildVariant.Name);

		// Resolve grid configuration from the pipeline's WP transformer and hash it into the build variant
		FGridSettings GridSettings;

		if (BuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid)
		{
			GridSettings = WorldPartitionHelpers::ResolveGridFromPipeline(BuildVariant, World);
			GridSettings.GatherDependencies(DependencyHash);
		}

		BuildInfo.BuildVariantHash = DependencyHash.GetDependentDataHash();

		UE_LOGF(LogMegaMeshEditor, Display, "-- %.6ls -- %ls Build Variant", *MeshPartitionGUID.ToString(), *BuildVariant.Name.ToString());

		TArrayView<const MeshPartition::FModifierGroup> ModifierGroups = Descriptors->GetModifierGroups(BuildVariant);

		MeshPartitionEditorComponent->ResetGroupRegistry(ModifierGroups);

		MeshPartition::FBuilderSettings Settings;
		Settings.BuildType = MeshPartition::EBuildType::CompiledSection;
		Settings.Transform = Transform;
		Settings.TypePriorities = MeshPartitionDefinition->GetModifierTypePriorities();
		Settings.MaxSectionComplexity = BuildVariant.MaxSectionComplexity;
		Settings.bRecomputeNormals = true;
		Settings.bRecomputeTangents = true;
		Settings.TexcoordGenerationOptions = FChannelCollectionUVLayoutOptions::GetFromDefinition(MeshPartitionDefinition);
		// If builder is using the FastBoxProject channel UV option (intended for preview only), automatically switch it to a better method for compiled sections
		if (Settings.TexcoordGenerationOptions->UVLayoutMethod == EChannelCollectionUVLayoutMethod::FastBoxProject)
		{
			Settings.TexcoordGenerationOptions->UVLayoutMethod = EChannelCollectionUVLayoutMethod::ReferenceBoxProject;
		}

		// Using an indirect array so that the structures have a stable memory address
		TIndirectArray<FModifierGroupTask> ModifierGroupTasks;

		// Construct a ModifierGroupTask structure for each modifier group, to track the build state
		for (const MeshPartition::FModifierGroup& ModifierGroup : ModifierGroups)
		{
			BuildInfo.BaseModifierPaths.Empty();
			Algo::Transform(ModifierGroup.BaseDescs(), BuildInfo.BaseModifierPaths, [](const MeshPartition::FModifierDesc& BaseDescriptor) { return BaseDescriptor.ModifierPath; });

			FModifierGroupTask* ModifierGroupTask = new FModifierGroupTask(ModifierGroup, Settings, BuildInfo, BuildVariant, MeshPartitionEditorComponent, *this, OldCompiledSections);

			ModifierGroupTask->GridSettings = GridSettings;
			ModifierGroupTasks.Add(ModifierGroupTask);
			ModifierGroupTask->State = ETaskState::Ready;
		}

		ExecuteModifierGroupTasks(ModifierGroupTasks, World, InMeshPartition, InSourceControlHelper);

		int32 BuiltCompiledSections = 0;
		int32 FailedCompiledSections = 0;
		int32 ReusedCompiledSections = 0;
		for (FModifierGroupTask& ModifierGroupTask : ModifierGroupTasks)
		{
			switch (ModifierGroupTask.Result)
			{
			case ETaskResult::BuildSuccess:
				BuiltCompiledSections++;
				break;
			case ETaskResult::ReuseExisting:
				ReusedCompiledSections++;
				check(!ModifierGroupTask.ReuseCompiledSectionActorDescGuids.IsEmpty());

				for (const FGuid& Guid : ModifierGroupTask.ReuseCompiledSectionActorDescGuids)
				{
					CompiledSectionActorDescGuidsToKeep.Add(Guid);
				}
				break;
			default:
				FailedCompiledSections++;
				break;
			}
		}

		// destruct the tasks to free up the memory and actor references they are holding
		ModifierGroupTasks.Empty();
		MeshPartitionEditorComponent->ResetGroupRegistry();

		UE_LOGF(LogMegaMeshEditor, Display, "-- %.6ls -- %ls Finish Variant (Built %d, Reused %d, Failed %d Compiled Sections)",
			   *MeshPartitionGUID.ToString(), *BuildVariant.Name.ToString(),
			   BuiltCompiledSections, ReusedCompiledSections, FailedCompiledSections);

		// Update stats
		TotalBuiltCompiledSections += BuiltCompiledSections;
		TotalReusedCompiledSections += ReusedCompiledSections;
		TotalFailedCompiledSections += FailedCompiledSections;

		// tick the engine and garbage collect to evict actors
		FWorldPartitionHelpers::FakeEngineTick(World);
		FWorldPartitionHelpers::DoCollectGarbage();
	}

	UE_LOGF(LogMegaMeshEditor, Display, "-- %.6ls -- Finished Mesh Partition Build (%ls)", *MeshPartitionGUID.ToString(), *InMeshPartition->GetActorNameOrLabel());
	return true;
}

bool UWorldPartitionMeshPartitionBuilder::CanBuildMeshPartitions(const UWorld* InWorld, FName InBuildOption)
{
	return true;
}

EEditorBuildResult UWorldPartitionMeshPartitionBuilder::BuildMeshPartitions(UWorld* InWorld, EMeshPartitionReuseMode ReuseMode)
{
	check(InWorld);

	// first let's run some quick sanity checks to warn upfront if things are incorrect
	if (!InWorld->IsPartitionedWorld(InWorld))
	{
		const static FText MessageTitle = FText::FromString("Level is not using World Partition");
		FString MessageString = FString::Printf(TEXT("The Mesh Partition Builder requires World Partition to be enabled. Conversion to World Partition is available via \"Tools > Convert Level...\"."));
		FText MessageText = FText::FromString(MessageString);
		FMessageDialog::Open(EAppMsgCategory::Warning, EAppMsgType::Ok, EAppReturnType::Ok, MessageText, MessageTitle);
		return EEditorBuildResult::Skipped;
	}

	// check for definitions
	for (TActorIterator<AMeshPartition> MeshPartitionIterator(InWorld); MeshPartitionIterator; ++MeshPartitionIterator)
	{
		const AMeshPartition* MeshPartition = *MeshPartitionIterator;
		check(MeshPartition != nullptr);
		
		const UMeshPartitionDefinition* MeshPartitionDefinition = MeshPartition->GetMeshPartitionDefinition();
		if (MeshPartitionDefinition == nullptr)
		{
			const static FText MessageTitle = FText::FromString("No Mesh Partition Definition provided");
			FString MessageString = FString::Printf(TEXT("No Mesh Partition Definition provided for '%s'. Build using default settings?"), *MeshPartition->GetActorNameOrLabel());
			FText MessageText = FText::FromString(MessageString);
			EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgCategory::Warning, EAppMsgType::OkCancel, EAppReturnType::Ok, MessageText, MessageTitle);
			if (Result == EAppReturnType::Cancel)
			{
				return EEditorBuildResult::Skipped;
			}
		}
		else
		{
			TConstArrayView<MeshPartition::FCompiledSectionBuildVariant> BuildVariants = MeshPartitionDefinition->GetCompiledSectionBuildVariants();
			check(BuildVariants.Num() > 0);
		}
	}

	const FString& PackageToReloadOnSuccess = GetNameSafe(InWorld->GetPackage());

	// Try to provide the complete path. If we can't, try with project name.
	const FString ProjectPath = FPaths::IsProjectFilePathSet() ? FPaths::GetProjectFilePath() : FApp::GetProjectName();

	const ISourceControlProvider& SCCProvider = ISourceControlModule::Get().GetProvider();

	FString ReuseModeString = UEnum::GetValueAsString(ReuseMode);
	ReuseModeString = ReuseModeString.RightChop(ReuseModeString.Find("::") + 2);

	const FString LogFileName = TEXT("WPMeshPartitionBuilderLog.txt");
	const FString LogFileArg = TEXT("-log=") + LogFileName;

	const FString Arguments = FString::Printf(TEXT("\"%s\" -run=WorldPartitionBuilderCommandlet %s %s -ReuseMode=%s %s -SCCProvider=%s%s"),
		*ProjectPath,
		*PackageToReloadOnSuccess,
		TEXT("-AllowCommandletRendering -Builder=WorldPartitionMeshPartitionBuilder"),
		*ReuseModeString,
		*LogFileArg,
		*SCCProvider.GetName().ToString(),
		CVarMeshPartitionMeshBuildCommandWaitForDebugger->GetBool() ? TEXT(" -waitforattach") : TEXT(""));

	FString FullLogPath = FPaths::ProjectLogDir() + LogFileName;
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FDateTime LastCreationTime;
	int64 LastReadPos = 0;

	const FFileStatData FileStat = PlatformFile.GetStatData(*FullLogPath);
	LastCreationTime = (FileStat.bIsValid) ? FileStat.CreationTime : FDateTime();
	LastReadPos = (FileStat.bIsValid) ? FileStat.FileSize : 0;
	
	// Parse output, look for progress indicator in the log (in the form "Display: [i / N] Msg...\n")
	const FRegexPattern LogProgressPattern(TEXT("Display:\\s\\[([0-9]+)\\s\\/\\s([0-9]+)\\]\\s(.+)?(?=\\.{3})"));

	auto OnProcRunningTick = [&FullLogPath, &LastCreationTime, &LastReadPos, &LogProgressPattern, &PlatformFile](FScopedSlowTask& SlowTask)
	{
		const FFileStatData FileStat = PlatformFile.GetStatData(*FullLogPath);

		if (!FileStat.bIsValid)
		{
			return;
		}

		const bool bLogRotated = (FileStat.CreationTime > LastCreationTime) || (FileStat.FileSize < LastReadPos);

		if (bLogRotated)
		{
			LastCreationTime = FileStat.CreationTime;
			LastReadPos = 0;
		}

		if (FileStat.FileSize > LastReadPos)
		{
			TUniquePtr<FArchive> LogReader(IFileManager::Get().CreateFileReader(*FullLogPath, FILEREAD_AllowWrite));

			if (LogReader != nullptr)
			{
				// Cap the read to avoid int64->int32 narrowing on TArray size. We only need recent
				// progress lines, so reading the tail of the log is sufficient.
				constexpr int64 MaxReadSize = 64 * 1024 * 1024; // 64 MB
				const int64 BytesAvailable = FileStat.FileSize - LastReadPos;
				const int64 ReadStart = (BytesAvailable > MaxReadSize) ? (FileStat.FileSize - MaxReadSize) : LastReadPos;
				const int32 BytesToRead = static_cast<int32>(FMath::Min(BytesAvailable, MaxReadSize));

				TArray<uint8> Buffer;

				LogReader->Seek(ReadStart);
				Buffer.SetNumUninitialized(BytesToRead);
				LogReader->Serialize(Buffer.GetData(), Buffer.Num());

				FString LogString = FString(Buffer.Num(), StringCast<TCHAR>(reinterpret_cast<const char*>(Buffer.GetData()), Buffer.Num()).Get());
				FRegexMatcher Regex(LogProgressPattern, LogString);

				while (Regex.FindNext())
				{
					// Update slow task progress & message
					SlowTask.CompletedWork = FCString::Atoi(*Regex.GetCaptureGroup(1));
					SlowTask.TotalAmountOfWork = FCString::Atoi(*Regex.GetCaptureGroup(2));
					SlowTask.DefaultMessage = FText::FromString(Regex.GetCaptureGroup(3));
				}

				LastReadPos = FileStat.FileSize;
			}
		}
	};

	const bool bSuccess = FEditorBuildUtils::RunWorldPartitionBuilder(PackageToReloadOnSuccess,
		LOCTEXT("WorldPartitionBuildMeshPartitionProgress", "Building Mesh Partition..."),
		LOCTEXT("WorldPartitionBuildMeshPartitionCancelled", "Building Mesh Partition cancelled!"),
		LOCTEXT("WorldPartitionBuildMeshPartitionFailed", "Errors occured during the build process, please refer to the logs ('WPMeshPartitionBuilderLog.txt')."),
		Arguments, OnProcRunningTick);

	return bSuccess ? EEditorBuildResult::Success : EEditorBuildResult::Skipped;
}

void UWorldPartitionMeshPartitionBuilder::GatherDependenciesAndComputeHashes(FModifierGroupTask& InOutTask, IAssetRegistry& AssetRegistry)
{
	using namespace EditorUtils;
	FDependencyContext DependencyContext;

	// gather dependencies from the MeshPartition definition
	if (const UMeshPartitionDefinition* MeshPartitionDefinition = InOutTask.BuildInfo.GetMegaMeshDefinition())
	{
		DependencyContext.AddPackageDependency(MeshPartitionDefinition);
		DependencyContext.AddClassDependency(MeshPartitionDefinition->GetClass());
		MeshPartitionDefinition->GatherDependencies(DependencyContext, InOutTask.BuildVariant.Name);
	}

	// gather dependencies from the MeshPartition modifiers
	InOutTask.ModifierGroup.ForAllModifiers([&DependencyContext](MeshPartition::UModifierComponent* Modifier)
	{
		if (ensure(Modifier != nullptr))
		{
			DependencyContext.AddPackageDependency(Modifier);
			DependencyContext.AddClassDependency(Modifier->GetClass());
			Modifier->GatherDependencies(DependencyContext);
		}

		return true;
	});

	// populate the package tracking
	InOutTask.BuildInfo.PackageDependencies = DependencyContext.GetPackageDependencies(AssetRegistry);
	InOutTask.BuildInfo.PackageHash = ComputePackageHash(AssetRegistry, InOutTask.BuildInfo.PackageDependencies, &InOutTask.BuildInfo.PackageChecksums);

	// populate the class tracking
	TArray<const UClass*> Classes = DependencyContext.GetClassDependencies();
	Algo::Transform(Classes, InOutTask.BuildInfo.ClassDependencies, [](const UClass* Class)
		{
			FSoftClassPath Path(Class);
			return FName(Path.ToString());
		});
	InOutTask.BuildInfo.ClassHash = ComputeClassHash(Classes, &InOutTask.BuildInfo.ClassChecksums);
}

void UWorldPartitionMeshPartitionBuilder::ExecuteModifierGroupTasks(TIndirectArray<FModifierGroupTask>& ModifierGroupTasks, UWorld* InWorld, const AMeshPartition* InMeshPartition, FSourceControlHelper& InSourceControlHelper)
{
	using namespace EditorUtils;

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	if ((ReuseMode == EMeshPartitionReuseMode::ByPackageHash) || (ReuseMode == EMeshPartitionReuseMode::ValidateHashes))
	{
		// we need to ensure the asset registry is up to date before we start querying it
		// (otherwise we may fail to update things that were recently changed, but have not yet been updated in the asset registry...)
		AssetRegistry.WaitForCompletion();
	}

	bool AllTaskCompleted = false;
	int32 SectionsBuilt = 0;
	int32 SectionsReused = 0;
	bool bUpdateLog = false;

	auto GetTotalSectionNumber = [&]()
	{
		return TotalBuiltCompiledSections + TotalFailedCompiledSections + TotalReusedCompiledSections + SectionsBuilt + SectionsReused;
	};

	int NumTasksInFlight = 0;

	while (!AllTaskCompleted)
	{
		AllTaskCompleted = true;

		for (FModifierGroupTask& Task : ModifierGroupTasks)
		{
			switch (Task.State)
			{
				case ETaskState::Ready:
					if (NumTasksInFlight < MaxTasksInFlight && !FWorldPartitionHelpers::HasExceededMaxMemory())
					{
						Task.ReuseCompiledSectionActorDescGuids.Empty();

						// TODO [chris.tchou] : this DOES NOT include GUIDs from modifiers inside level instances...	so it doesn't detect adding a new instance inside a level instance.
						Task.BuildInfo.ModifierSetHash = Task.ModifierGroup.ComputeModifierSetHash();

						// try to reuse by package hashes (which doesn't require loading any modifiers, so we can early out if this works)
						if ((ReuseMode == EMeshPartitionReuseMode::ByPackageHash) || (ReuseMode == EMeshPartitionReuseMode::ValidateHashes))
						{
							Task.ReuseCompiledSectionActorDescGuids = FindMatchingCompiledSectionsFromPackageHashes(Task.OldCompiledSectionDescriptors, Task.BuildInfo, AssetRegistry);
						}

						// if we're not early out (or if we're in validate mode)
						if (Task.ReuseCompiledSectionActorDescGuids.IsEmpty() || (ReuseMode == EMeshPartitionReuseMode::ValidateHashes))
						{
							// load the modifiers, including all modifiers within level instances
							LoadActors(InMeshPartition, Task);
							Task.State = ETaskState::ActorLoaded;

							// and compute an up-to-date modifier hash
							Task.BuildInfo.ModifiersHash = Task.ModifierGroup.UpdateAndComputeModifierGroupHash();

							if (ReuseMode == EMeshPartitionReuseMode::ByModifierHash)
							{
								Task.ReuseCompiledSectionActorDescGuids = FindMatchingCompiledSectionsFromModifierHash(Task.OldCompiledSectionDescriptors, Task.BuildInfo);
							}
							else if (ReuseMode == EMeshPartitionReuseMode::ValidateHashes)
							{
								TArray<FGuid> ModifierHashReuseGuids = FindMatchingCompiledSectionsFromModifierHash(Task.OldCompiledSectionDescriptors, Task.BuildInfo);

								// ModifierHash may find a match when PackageHash does not, that is expected.
								// But we can check:

								// Sort both for set comparison (order may differ between search paths)
								Task.ReuseCompiledSectionActorDescGuids.Sort();
								ModifierHashReuseGuids.Sort();

								// if PackageHash finds matches, then modifier hash should agree
								if (!Task.ReuseCompiledSectionActorDescGuids.IsEmpty())
								{
									// (if not we have a bug in our hash/dependency calculations)
									check(Task.ReuseCompiledSectionActorDescGuids == ModifierHashReuseGuids);
								}

								// if ModifierHash finds NO match, then PackageHash should agree
								if (ModifierHashReuseGuids.IsEmpty())
								{
									// (if not we have a bug in our hash/dependency calculations)
									check(Task.ReuseCompiledSectionActorDescGuids.Num() == 0);
								}

								// TODO: if we record PackageHash and/or ModifierHash per Package/Modifier, then we can provide better info on which is causing the issue here
							}
						}

						// For grid-split groups, reject partial reuse -- all cells must match or none do.
						if (!Task.ReuseCompiledSectionActorDescGuids.IsEmpty() && Task.GridSettings.IsGridSplit())
						{
							int32 TotalStructuralMatches = 0;
							
							for (const FCompiledSectionDescriptor& Desc : Task.OldCompiledSectionDescriptors)
							{
								if (Desc.Info.TargetsSameCompiledSectionAs(Task.BuildInfo) &&
									Desc.Info.BuildVariantHash == Task.BuildInfo.BuildVariantHash &&
									Desc.Info.ModifierSetHash == Task.BuildInfo.ModifierSetHash)
								{
									TotalStructuralMatches++;
								}
							}
							
							if (Task.ReuseCompiledSectionActorDescGuids.Num() < TotalStructuralMatches)
							{
								UE_LOGF(LogMegaMeshEditor, Warning, "Grid-split reuse: only %d of %d cells have up-to-date hashes, forcing rebuild",
									Task.ReuseCompiledSectionActorDescGuids.Num(), TotalStructuralMatches);
								Task.ReuseCompiledSectionActorDescGuids.Empty();
							}
						}

						if (!Task.ReuseCompiledSectionActorDescGuids.IsEmpty())
						{
							Task.Result = ETaskResult::ReuseExisting;
							Task.State = ETaskState::Completed;
							SectionsReused++;
							bUpdateLog = true;

							WorldPartitionMeshPartitionBuilderLocals::LogBuildProgress(GetTotalSectionNumber(), TotalNumberOfSectionsToBuild);
						}
						else
						{
							// record package and class dependencies
							GatherDependenciesAndComputeHashes(Task, AssetRegistry);

							// Build a new compiled section.  Start by launching the async task to execute the modifiers
							Task.ModifierGroup.SetBuildType(MeshPartition::EBuildType::CompiledSection);
							Task.ModifierGroup.ProgressToState(MeshPartition::FModifierGroup::EState::BackgroundOpsCreated);

							Task.ModifierTaskGraph = MakeShared<MeshPartition::FModifierTaskGraph>();

							Task.State = ETaskState::LaunchedProcessModifierTask;
							// #todo: this whole processing task should be using the MeshPartition builder system instead of using its own bespoke path.
							Tasks::FTask ProcessModifiersTask = Tasks::Launch(TEXT("UWorldPartitionMeshPartitionBuilder_ProcessModifiersTask"),
								[&Task, Group = Task.ModifierGroup.CreateAsyncBuildGroup()]() mutable
								{
									TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionMeshPartitionBuilder::ProcessModifiersTask);

									Tasks::FTask ProcessComplete = BuildHelpers::ProcessModifierGroup(Task.BuilderSettings, Group, Task.ModifierTaskGraph);

									Tasks::AddNested(ProcessComplete);
								}
							);
						
							Tasks::TTask<MeshPartition::FMeshData> GenerateUVLayoutTask = Tasks::Launch(TEXT("UWorldPartitionMeshPartitionMeshBuilder_GenerateUVLayoutTask"),
								[&Task]
								{
									MeshPartition::FMeshData MeshData = MoveTemp(Task.ModifierTaskGraph->GetResultMesh());

									if (Task.BuilderSettings.TexcoordGenerationOptions.IsSet())
									{
										MeshPartition::FChannelTextureRenderer::CreateSectionUVLayout(MeshData, Task.BuilderSettings.TexcoordGenerationOptions.GetValue());
									}
									return MoveTemp(MeshData);
								},
								Tasks::Prerequisites(ProcessModifiersTask)
							);

							Task.ProcessModifierTask = Tasks::Launch(TEXT("UWorldPartitionMeshPartitionMeshBuilder_FinishProcessModifiersTask"),
								[&Task, GenerateUVLayoutTask]() mutable
								{
									MeshPartition::FMeshData MeshData = MoveTemp(GenerateUVLayoutTask.GetResult());

									Task.State = ETaskState::ModifiersProcessed;
									return MoveTemp(MeshData);
								},
								Tasks::Prerequisites(GenerateUVLayoutTask)
							);

							++NumTasksInFlight;
						}
					}
					break;
								
				case ETaskState::ModifiersProcessed:
					{
						MeshPartition::FMeshData& FullMesh = Task.ProcessModifierTask.GetResult();
						FPrepareCompiledSectionsParams CompiledSectionsParams {.FullMesh = FullMesh, .GridSettings = Task.GridSettings};

						Task.CompiledSections = Task.MeshPartitionEditorComponent->PrepareCompiledSections(Task.BuildInfo, Task.BuildVariant, CompiledSectionsParams);

						if (Task.CompiledSections.IsEmpty())
						{
							UE_LOGF(LogMegaMeshEditor, Error, "PrepareCompiledSections produced no sections for modifier group. Skipping.");
							Task.Result = ETaskResult::BuildFail;
							Task.State = ETaskState::Completed;
							NumTasksInFlight--;
							break;
						}

						TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> ModifierPtrs = Task.ModifierGroup.AllResolvedModifierPtrs();
						TArray<MeshPartition::FTransformerUnit> InitialUnits;
						InitialUnits.Reserve(Task.CompiledSections.Num());

						for (int32 CompiledSectionIndex = 0; CompiledSectionIndex < Task.CompiledSections.Num(); ++CompiledSectionIndex)
						{
							MeshPartition::ACompiledSection* Section = Task.CompiledSections[CompiledSectionIndex];
							MeshPartition::FMeshData* CellMeshPtr = CompiledSectionsParams.OutCellMeshes.Find(Section->GetBuildInfo().GridCellCoord);
							MeshPartition::FMeshData& SectionMesh = CellMeshPtr ? *CellMeshPtr : FullMesh;

							Task.MeshPartitionEditorComponent->BuildMegaMeshCompiledSectionTextures(Section, Task.ModifierGroup, SectionMesh);
							Task.MeshPartitionEditorComponent->PostBuildSectionMesh(Section, SectionMesh, ModifierPtrs);

							InitialUnits.Add(MeshPartition::MakeTransformerUnit(Section, MakeShared<const MeshPartition::FMeshData>(MoveTemp(SectionMesh))));
						}

						MeshPartition::UMeshPartitionDefinition* Definition = Task.MeshPartitionEditorComponent->GetMegaMeshDefinition();
						Task.TransformerContext = Task.MeshPartitionEditorComponent->LaunchTransformers(MoveTemp(InitialUnits), Definition, Task.BuildVariant);
					}

					if (Task.TransformerContext.IsValid())
					{
						Task.State = ETaskState::TransformersLaunched;
					}
					else
					{
						Task.State = ETaskState::TransformersExecuted;
					}

					break;
					
				case ETaskState::TransformersLaunched:
					if (Task.TransformerContext->JoinTask.IsValid() && Task.TransformerContext->JoinTask.IsCompleted())
					{
						Task.State = ETaskState::TransformersExecuted;
					}

					break;

				case ETaskState::TransformersExecuted:
					{
						TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> ModifierPtrs = Task.ModifierGroup.AllResolvedModifierPtrs();

						for (MeshPartition::ACompiledSection* CellSection : Task.CompiledSections)
						{
							Task.ModifiedPackages.Emplace(CellSection->GetPackage());
							Task.MeshPartitionEditorComponent->PostProcessSection(CellSection, ModifierPtrs);
							AssignSectionToPlatformDataLayers(CellSection);
						}
					}

					// save and reclaim memory
					SaveAndCollectGarbage(InWorld, InSourceControlHelper, Task);
					Task.Result = ETaskResult::BuildSuccess;
					Task.State = ETaskState::Completed;
					SectionsBuilt++;
					bUpdateLog = true;
					NumTasksInFlight--;

					int32 TotalSectionNumber = TotalBuiltCompiledSections + TotalFailedCompiledSections + TotalReusedCompiledSections + SectionsBuilt + SectionsReused;
					WorldPartitionMeshPartitionBuilderLocals::LogBuildProgress(GetTotalSectionNumber(), TotalNumberOfSectionsToBuild);

					break;
			}
			
			if (Task.State != ETaskState::Completed)
			{
				AllTaskCompleted = false;
			}
		}
		
		if (bUpdateLog)
		{
			bUpdateLog = false;
			UE_LOGF(LogMegaMeshEditor, Log, "Sections: %d built, %d reused out of %d total", SectionsBuilt, SectionsReused, ModifierGroupTasks.Num());
		}
		
		// Tick the engine while we still have remaining tasks to progress engine systems such as PCG and Rendering, allowing them to do work on the GameThread if needed.
		FWorldPartitionHelpers::FakeEngineTick(InWorld);
	}
}

void UWorldPartitionMeshPartitionBuilder::LoadActors(const AMeshPartition* InMeshPartition, FModifierGroupTask& InModifierGroupTask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionMeshPartitionBuilder::LoadActors);

	if (InMeshPartition == nullptr)
	{
		return;
	}

	UMeshPartitionEditorComponent* MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(InMeshPartition->GetMeshPartitionComponent());
	if (MeshPartitionEditorComponent == nullptr)
	{
		return;
	}

	auto OnActorLoaded = [this, &InModifierGroupTask](AActor* Actor, bool bIsInLevelInstance)
		{
			InModifierGroupTask.ModifierOwners.Emplace(Actor);
		};


	auto OnModifierLoaded = [this, MeshPartitionEditorComponent, &InModifierGroupTask](MeshPartition::UModifierComponent* Modifier, bool bIsInLevelInstance)
		{
			MeshPartitionEditorComponent->OnModifierLoaded(Modifier);
		};

	WorldPartitionHelpers::LoadAllRelevantMegaMeshActors(InModifierGroupTask.ModifierGroup, InMeshPartition, InModifierGroupTask.ActorRefs, OnActorLoaded, OnModifierLoaded);
}

void UWorldPartitionMeshPartitionBuilder::AssignSectionToPlatformDataLayers(MeshPartition::ACompiledSection* InCompiledSection) const
{
	const UWorld* World = InCompiledSection->GetWorld();
	if (AMeshPartitionDataLayerContainer* Container =  AMeshPartitionDataLayerContainer::Get(World))
	{
		if (TObjectPtr<UDataLayerAsset> DataLayer = Container->FindVariantDataLayer(InCompiledSection->GetParentMegaMesh()->GetMeshPartitionDefinition(), InCompiledSection->GetBuildInfo().BuildVariantName); ensure(DataLayer))
		{
			if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(World); ensure(DataLayerManager))
			{
				if (const UDataLayerInstance* Instance = DataLayerManager->GetDataLayerInstance(DataLayer); ensure(Instance))
				{
					Instance->AddActor(InCompiledSection);
				}
			}
		}
	}
}

bool UWorldPartitionMeshPartitionBuilder::SavePackages(const TArray<UPackage*>& PackagesToSave, FSourceControlHelper& InSourceControlHelper)
{
	// FSourceControlHelper::Save handles checkout / save / add and auto-records each operation
	// into the FBuilderModifiedFiles it was constructed with, so SubmitToSourceControl can pick them up later.
	if (!InSourceControlHelper.Save(PackagesToSave))
	{
		TArray<FString> PackageNames;
		Algo::Transform(PackagesToSave, PackageNames, [](const UPackage* Package) { return Package->GetName(); });
		UE_LOGF(LogMegaMeshEditor, Error, "Error saving packages: %ls.", *FString::Join(PackageNames, TEXT(", ")));
		return false;
	}
	return true;
}

void UWorldPartitionMeshPartitionBuilder::SaveAndCollectGarbage(UWorld* InWorld, FSourceControlHelper& InSourceControlHelper, FModifierGroupTask& InModifierGroupTask)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UWorldPartitionMeshPartitionBuilder::SaveAndCollectGarbage);

	TArray<UObject*> ObjectsToFinish;

	for (MeshPartition::ACompiledSection* Section : InModifierGroupTask.CompiledSections)
	{
		ForEachObjectWithOuter(Section, [&ObjectsToFinish](UObject* Object)
		{
			if (const IInterface_AsyncCompilation* AsyncCompilationIF = Cast<IInterface_AsyncCompilation>(Object))
			{
				ObjectsToFinish.Emplace(Object);
			}

			return true;
		});
	}

	if (!ObjectsToFinish.IsEmpty())
	{
		FAssetCompilingManager::Get().FinishCompilationForObjects(ObjectsToFinish);
	}

	if (!InModifierGroupTask.ModifiedPackages.IsEmpty())
	{
		SavePackages(InModifierGroupTask.ModifiedPackages, InSourceControlHelper);
	}

	// Destroy all newly created compiled sections _after_ the save so they can be fully unloaded by the next GC.
	for (MeshPartition::ACompiledSection* Section : InModifierGroupTask.CompiledSections)
	{
		InWorld->DestroyActor(Section);
	}

	// clear out pointers, since the objects may be going away in the garbage collection
	InModifierGroupTask.CompiledSections.Reset();
	InModifierGroupTask.ModifiedPackages.Empty();
	InModifierGroupTask.ModifierOwners.Empty();
	InModifierGroupTask.ActorRefs.Empty();

	// Simulate an engine tick to make sure engine & render resources that are queued for deletion are processed and streaming levels are removed.
	FWorldPartitionHelpers::FakeEngineTick(InWorld);
	FWorldPartitionHelpers::DoCollectGarbage();
}

bool UWorldPartitionMeshPartitionBuilder::SubmitToSourceControl(UWorld* InWorld)
{
	// Wait for pending async file writes before submitting
	UPackage::WaitForAsyncFileWrites();

	const FString ChangeDescription = FString::Printf(TEXT("Rebuilt Mesh Partition Compiled Sections for %s"), *InWorld->GetPackage()->GetName());
	return OnFilesModified(ModifiedFiles.GetAllFiles(), ChangeDescription);
}

} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
