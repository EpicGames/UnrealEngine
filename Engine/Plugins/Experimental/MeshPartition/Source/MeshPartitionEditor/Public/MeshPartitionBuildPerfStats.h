// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionBuildPerfStats.generated.h"

namespace UE::MeshPartition
{
USTRUCT()
struct FPerModifierBuildPerfStats
{
	GENERATED_BODY()
public:
	void AddInstanceStat(double InInstanceExecTime)
	{
		InstanceCount += 1;
		TotalExecutionTime += InInstanceExecTime;
		MinInstanceExecutionTime = FMath::Min(MinInstanceExecutionTime, InInstanceExecTime);
		MaxInstanceExecutionTime = FMath::Max(MaxInstanceExecutionTime, InInstanceExecTime);
	}

	/** Total number of instances. */
	UPROPERTY(VisibleAnywhere, Category = MegaMesh, meta=(EditCondition="InstanceCount > 1", EditConditionHides))
	int32 InstanceCount = 0;

	/** Accumulated modifier timings across all Instances. */
	UPROPERTY(VisibleAnywhere, Category = MegaMesh, meta=(DisplayName="Total Execution Time (Seconds)"))
	double TotalExecutionTime = 0.;
	
	UPROPERTY(VisibleAnywhere, Category = MegaMesh, meta=(DisplayName = "Min Instance Exec Time (Seconds)", EditCondition="InstanceCount > 1", EditConditionHides))
	double MinInstanceExecutionTime = TNumericLimits<double>::Max();

	UPROPERTY(VisibleAnywhere, Category = MegaMesh, meta=(DisplayName = "Max Instance Exec Time (Seconds)", EditCondition="InstanceCount > 1", EditConditionHides))
	double MaxInstanceExecutionTime = 0.;
};

USTRUCT()
struct FBuildPerfStats
{
	GENERATED_BODY()
public:
	/** Wall clock time to process all modifiers. */
	UPROPERTY(VisibleAnywhere, Category = MegaMesh, meta=(DisplayName = "Wall Clock Time (Seconds)"))
	double WallTime = 0.;

	UPROPERTY(VisibleAnywhere, Category = MegaMesh, meta=(ForceInlineRow))
	TMap<FSoftObjectPath, MeshPartition::FPerModifierBuildPerfStats> PerModifierTimings;
};
} // namespace UE::MeshPartition