// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionUVLayoutMethod.h"
#include "Tasks/Task.h"

#define UE_API MESHPARTITIONEDITOR_API


////////////////////////////////////////////////////////////////////////////

class UTexture;
enum EPixelFormat : uint8;

namespace UE::MeshPartition
{
class ACompiledSection;
class UMeshPartitionDefinition;
class UMeshPartitionEditorComponent;
class UModifierComponent;
struct FChannelMap;
struct FModifierGroup;

// Quality tier requested for the UV unwrap. Currently only used by the VolumeEncoded method
enum class EChannelUVUnwrapQuality : uint8
{
	// Full-quality unwrap
	Full,
	// Lower-quality preset tuned for interactive preview iteration
	Preview
};

// Options specific to the Volume-Encoded UV algorithm
struct FVEUVLayoutOptions
{
	// Target sample count per square meter of section surface area. Drives FVEUVConfig::Sampling.TotalSamples.
	float SamplesPerSquareMeter = 0.025f;
	// Voxel grid dimensions used by the optimizer. Drives FVEUVConfig::VoxelCount.
	FIntVector VoxelCount = FIntVector(4, 4, 1);
	// Ignore VEUV result validation failures and always accept the output; useful for debugging failure cases.
	bool bAcceptOutputOnFailure = false;
};

// Options specific to the PlaneProject UV layout method
struct FPlaneProjectionLayoutOptions
{
	// Whether to project along the section's area-weighted average normal or a fixed direction.
	EPlaneProjectionNormalSource NormalSource = EPlaneProjectionNormalSource::FixedPlane;
	// Plane normal used when NormalSource is FixedPlane. Need not be unit length; normalized at use.
	FVector FixedNormal = FVector(0.0, 0.0, 1.0);
};

struct FChannelCollectionUVLayoutOptions
{
	MESHPARTITIONEDITOR_API static FChannelCollectionUVLayoutOptions GetDefaults();
	MESHPARTITIONEDITOR_API static FChannelCollectionUVLayoutOptions GetFromDefinition(const UMeshPartitionDefinition* InDefinition);

	float TexelSize;
	int32 MaxTextureResolution;
	// Desired (approx.) texels between UV islands
	float GutterTexelCount = 4.0f;
	EChannelCollectionUVLayoutMethod UVLayoutMethod;
	EChannelUVUnwrapQuality UVQuality = EChannelUVUnwrapQuality::Full;
	UE_EXPERIMENTAL(5.8, "VEUV layout options are experimental; the parameter set may change without deprecation as the algorithm evolves.")
	FVEUVLayoutOptions VEUV;
	FPlaneProjectionLayoutOptions PlaneProjection;
};

struct FSectionDomainMapping
{
	double	TexelSize3D = 100.0; // Size of one texel in position unit [u] assigned from definition
	int32	MaxImageResolution = 4092;
	int32	MinImageResolution = 0;
	int32	ImageResolution = 0;

	double	Area3D = 0;
	double  AreaUV = 0;
			
	double	Size3D = 0;
	double  SizeUV = 0;

	// Output-texel-space gap between UV islands
	float	GutterTexelCount = 4.0f;

	float GetGutterTexelCount() const { return GutterTexelCount; }
	float GetGutterUV() const { return GetGutterTexelCount() / (float) ImageResolution; }
};

struct FSectionChannels
{
	TArray<uint8> Table;
	FVector2f TexcoordMetrics = { 1.0f, 1.0f };
	TStrongObjectPtr<UTexture> Texture;
	TOptional<FSectionChannelGenerationMeshData> Mesh;
	FSectionChannelDomainData Domain;
};

/** Apply pre-built channel data (texture, texcoord metrics, table) to a compiled section. */
UE_API void ApplyChannels(ACompiledSection* InSection, const FSectionChannels& InChannels, const float InMaterialCacheTexelSize);

/**
* FMegaMeshChannelCollection contains functions and objects needed to build the rendering elements 
* representing the masks and channels for the sections of the mega mesh
* Each base is assigned an Id in the collection allowing to reference and access other resources associated with it
* such as individual textures stored in the Texture collection
*/
class FChannelTextureRenderer
{
public:
	struct FChannelRenderUVMeshTopology;
	
	static constexpr int32 NumChannelsPerPage = 1;
	static constexpr float DefaultTexelSize = 100.0f; // Default texel size (in world unit) driving the channel rasterization resolution
	static constexpr int32 DefaultMaxImageResolution = 4096; // Default maximum channel texture resolution generated per section

	/**
	* Builds the texture for the different bases contained in a modifier group and their associated modifiers. Will also update the MegaMesh's texture collection.
	* @param InSectionMesh The Section mesh generated by the MeshPartition pipeline containing the Channel information that needs to be rasterized in the channel texture
	* @param InOwner An optional owner for the texture generated (otherwise, it is the ChannelCollection). This is used for compiled section, where the texture generated is directly assigned to its section actor
	* @param bInDownloadToAsset whether to download the mip data from video memory into the UTexture asset. Must be used for serialized compiled sections.
	* @param InChannels Channel to attribute id mapping.
	* @param InTexelSize worldspace size of a texel in the output channel textures.
	*/
	static UE_API Tasks::TTask<FSectionChannels> BuildTextureForSection(const FMeshData& InSectionMesh, UObject* InOwner, bool bInDownloadToAsset, const MeshPartition::FChannelMap& InChannels, float InTexelResolution);

	static UE_API Tasks::TTask<FSectionChannels> BuildTextureForSectionWithCachedTopology(
		const Geometry::FDynamicMesh3& InSectionMesh,
		const FChannelRenderUVMeshTopology& InCachedTopology,
		UObject* InOwner,
		bool bInDownloadToAsset,
		const MeshPartition::FChannelMap& InChannels);

	static UE_API FChannelRenderUVMeshTopology GenerateUVMeshTopology(const Geometry::FDynamicMesh3& InSectionMesh, FVector InScale, float InTexelResolution); 

	/**
	* Section element
	* A section struct is created for every unique section encountered
	* The section is associated to its Texture slot allocation on the first generation
	* 
	* The section UVmesh data is regenerated everytime the section is dirtied because bases or modifiers have been edited
	* 
	* The section UVDomain data store the intermediate steps of the mapping algorithm used to unify the uvdomain of each bases into one uv domain for the section
	* 
	* The section generated data is cached for efficienty in editor mode.
	* The generated data is useful for debugging and inspection
	*/
	struct FSection
	{
		struct FUVMesh
		{
			TArray<FInt32Vector2>	BaseIndicesRanges;
			TArray<int32>			Indices;
			TArray<FVector2f>		UVs;
			TArray<FInt32Vector3>	Outlines;
			TArray<float>			Channels;
			TArray<int32>			ChannelActiveCounts;
			TArray<uint8>			ActiveChannelIndices;
			TArray<int32>			UVElementToVertexID;
			int32 					VertexCount;

			int32 NumIndices() const { return Indices.Num(); }
			int32 NumChannels() const { return (VertexCount ? Channels.Num() / VertexCount : 0); }
		};
		FUVMesh					UVmesh;

		TArray<FBox2f>			Bases_UVBox;
		TArray<TArray<int32>>	Islands_Bases;
		TArray<FBox2f>			Islands_UVBox;
		TArray<FBox2f>			SectionIslands_UVBox;
		TArray<FBox2f>			FreeRects;

		int32 NumActivePages() const { return FMath::DivideAndRoundUp(UVmesh.ActiveChannelIndices.Num(), NumChannelsPerPage); }

		TArray<uint8>			ChannelOffsetTable;

		FUint32Vector2			RasterResolution;
		FVector2f				TexcoordMetrics;

		FSectionDomainMapping	DomainMapping;
	};

	/**
	 * Cached UV mesh topology that remains constant during displacement and paint operations.
	 */
	struct FChannelRenderUVMeshTopology
	{
		TArray<int32>       Indices;
		TArray<FVector2f>   UVs;
		TArray<int32>       UVElementToVertexID;
		int32               VertexCount;

		FSectionDomainMapping DomainMapping;
		FUint32Vector2        RasterResolution;
		FVector2f             TexcoordMetrics;
	};
private:

	static UTexture* GetDefaultTexture();

	static UE_API void GenerateSectionMeshUVs(const FMeshData& InSectionMesh, FSection& InSection, float InTexelResolution);
	static UE_API void GenerateSectionMeshChannels(const FMeshData& InSectionMesh, FSection& InSection, const MeshPartition::FChannelMap& InChannels);
	static UE_API void GenerateSectionMeshChannels(const Geometry::FDynamicMesh3& InSectionMesh, FSection& InSection, const MeshPartition::FChannelMap& InChannels);

	static UE_API FSection BuildSectionFromCachedTopology(const FChannelRenderUVMeshTopology& InTopology);

	static UE_API UTexture* AllocateSectionTextureResource(int32 InSizeX, int32 InSizeY, int32 InArraySize, EPixelFormat InFormat, UObject* InOuter, const FName InName);
	static UE_API void RenderSectionChannels(UTexture* InOutSectionTextures, FSection InSection);
	static UE_API void DownloadSectionTextureToAsset(UTexture* InAllocatedTexture);

	static Tasks::TTask<TOptional<UTexture*>> LaunchAllocateTextureTask(Tasks::TTask<FSection> InBuildSectionTask, UObject* InOwner);
	static Tasks::TTask<FSectionChannels> LaunchRenderTextureTask(Tasks::TTask<FSection> InBuildSectionTask, Tasks::TTask<TOptional<UTexture*>> InAllocateTextureTask, bool bInDownloadToAsset, bool bInExtractUVSectionMeshData = false);

public:
	// @param OptionalWorldTransform World transform of the associated section, used for debug/visualization tooling
	static UE_API void CreateSectionUVLayout(FMeshData& InOutSectionMesh, const FChannelCollectionUVLayoutOptions& GenerationOptions, const FTransform& OptionalWorldTransform = FTransform::Identity);

};
} // namespace UE::MeshPartition

#undef UE_API
