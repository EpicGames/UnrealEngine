// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/CollisionProfile.h"
#include "MeshPartitionCollisionGeneration.h"
#include "MeshPartitionTransformer.h"

#include "MeshPartitionCollisionTransformer.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{

/**
 * Mesh Partition Transformer producing and attaching a collision component for each transformer unit present in the execution context.
 */
USTRUCT(MinimalAPI)
struct FCollisionTransformer : public MeshPartition::FTransformer
{
	GENERATED_BODY()

public:
	UE_API FCollisionTransformer() = default;

	UE_API virtual void Initialize(const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InVariant) override;

	UE_API virtual bool Execute(MeshPartition::FTransformerContext& InTransformerContext) const override;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const override;

private:
	void BuildCollisionData(const MeshPartition::FTransformerContext& InTransformerContext, const int32 InEntryIndex, TArray<TSharedPtr<FMeshPartitionCollisionData>>& OutCollisionData) const;

	void FinalizeCollisionData(const MeshPartition::FTransformerContext& InTransformerContext, const TArray<TSharedPtr<FMeshPartitionCollisionData>>& InCollisionData) const;

private:
	UPROPERTY()
	TArray<MeshPartition::FPhysicalMaterialChannel> PhysicalMaterialChannels;

	UPROPERTY()
	TObjectPtr<UPhysicalMaterial> DefaultPhysicalMaterial;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Simplification")
	MeshPartition::Collision::FCollisionSimplificationSettings CollisionSimplificationSettings;

	UPROPERTY(EditAnywhere, Category = "Collision")
	FCollisionProfileName CollisionProfile = UCollisionProfile::BlockAll_ProfileName;

	UPROPERTY(EditAnywhere, Category = "Collision")
	bool bCanEverAffectNavigation = true;

	UPROPERTY(EditAnywhere, Category = "Collision")
	bool bFastCook = false; //todo(luc.eygasier): should be true for preview sections  Or we could pass the execution type in the context and the transformer adapts.

	UPROPERTY(EditAnywhere, Category = "Collision")
	bool bDisableActiveEdgePrecompute = false; //todo(luc.eygasier):  should be true for preview sections  Or we could pass the execution type in the context and the transformer adapts.
};

} // namespace UE::MeshPartition

#undef UE_API
