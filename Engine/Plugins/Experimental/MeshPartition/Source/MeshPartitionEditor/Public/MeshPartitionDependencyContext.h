// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionDependencyInterface.h"
#include "Hash/Blake3.h"

class IAssetRegistry;
class UTexture;
class UDynamicMesh;
class USplineComponent;
class UStaticMesh;

namespace UE::MeshPartition
{
// Collects all package and class dependencies, but does not do any hashing.
class FDependencyContext : public IDependencyInterface
{
	// This is a dummy archive that makes any serialization calls into a no-op.
	class FNoOpArchive : public FArchive
	{
	public:
		FNoOpArchive()
		{
			FArchive::SetIsSaving(true);
		}

		virtual void Reset() override
		{
			FArchive::Reset();
			FArchive::SetIsSaving(true);
		}

		virtual void Serialize(void*, int64) override {}
		virtual void SerializeBool(bool&) override {}
		virtual void SerializeBits(void*, int64) override {}
		virtual bool SerializeBulkData(FBulkData&, const FBulkDataSerializationParams&) override { return true; }
		virtual void SerializeInt(uint32&, uint32) override {}
		virtual void SerializeIntPacked(uint32&) override {}
		virtual void SerializeIntPacked64(uint64&) override {}

		virtual FArchive& operator<<(FName&) override { return *this; }
		virtual FArchive& operator<<(FText&) override { return *this; }
		virtual FArchive& operator<<(UObject*&) override { return *this; }
		virtual FArchive& operator<<(FField*&) override { return *this; }
		virtual FArchive& operator<<(FLazyObjectPtr&) override { return *this; }
		virtual FArchive& operator<<(FObjectPtr&) override { return *this; }
		virtual FArchive& operator<<(FSoftObjectPtr&) override { return *this; }
		virtual FArchive& operator<<(FSoftObjectPath&) override { return *this; }
		virtual FArchive& operator<<(FWeakObjectPtr&) override { return *this; }
	};

	TMap<const UObject*, EDependencyFlags> ObjectDependencies;
	TSet<const UClass*> ClassDependencies;
	FNoOpArchive NoOpArchive;

protected:
	friend class UWorldPartitionMeshPartitionBuilder;

	TArray<FName> GetPackageDependencies(const IAssetRegistry& InAssetRegistry) const;
	TArray<const UClass*> GetClassDependencies() const;

public:
	FDependencyContext() = default;
	virtual ~FDependencyContext() override = default;

	virtual void AddPackageDependency(const UObject* InObject, EDependencyFlags InDependencyFlags = EDependencyFlags::None) override;
	virtual void AddPackageDependency(TArrayView<UObject*> InObjectArray) override;
	virtual void AddClassDependency(const UClass* InClass) override;
	virtual FArchive& GetDependentDataArchive() override { return NoOpArchive; }

	virtual void operator+=(const UDynamicMesh& InMesh) override;
	virtual void operator+=(const UStaticMesh& InMesh) override;
	virtual void operator+=(const UTexture& InTexture) override;
	virtual void operator+=(const UTexture2D& InTexture) override;
	virtual void operator+=(const UCurveFloat& InCurveFloat) override;
	virtual void operator+=(const USplineComponent& InSpline) override;
	virtual void operator+=(const UObject& InObject) override;
};

// Hashes all the data, but does not collect any class or packages dependencies. 
class FDependencyHash : public IDependencyInterface
{
public:
	using IDependencyInterface::operator+=;

	FDependencyHash() = default;
	virtual ~FDependencyHash() override = default;

	virtual void AddPackageDependency(const UObject* InObject, EDependencyFlags InDependencyFlags = EDependencyFlags::None) override {}
	virtual void AddPackageDependency(TArrayView<UObject*> InObjectArray) override {}
	virtual void AddClassDependency(const UClass* InClass) override {}
	virtual FArchive& GetDependentDataArchive() override { return HashArchive; }

	virtual void operator+=(const UDynamicMesh& InMesh) override;
	virtual void operator+=(const UStaticMesh& InMesh) override;
	virtual void operator+=(const UTexture& InTexture) override;
	virtual void operator+=(const UTexture2D& InTexture) override;
	virtual void operator+=(const UCurveFloat& InCurveFloat) override;
	virtual void operator+=(const USplineComponent& InSpline) override;
	virtual void operator+=(const UObject& InObject) override;

	// return the hash of all dependencies hashed up until this point
	FGuid GetDependentDataHash() const;

private:
	// Accumulates a hash for all serialized data.
	class FHashArchive : public FArchive
	{
	public:
		FHashArchive()
		{
			FArchive::SetIsSaving(true);
		}

		virtual void Reset() override
		{
			FArchive::Reset();
			FArchive::SetIsSaving(true);
			HashBuilder.Reset();
		}

		virtual FArchive& operator<<(UObject*& Value) override { return CheckObjectReferences(); }
		virtual FArchive& operator<<(FLazyObjectPtr& Value) override { return CheckObjectReferences(); }
		virtual FArchive& operator<<(FObjectPtr& Value) override { return CheckObjectReferences(); }
		virtual FArchive& operator<<(FSoftObjectPtr& Value) override { return CheckObjectReferences(); }
		virtual FArchive& operator<<(FSoftObjectPath& Value) override { return CheckObjectReferences(); }
		virtual FArchive& operator<<(FWeakObjectPtr& Value) override { return CheckObjectReferences(); }

		virtual void Serialize(void* Data, const int64 Num) override
		{
			HashBuilder.Update(Data, static_cast<uint64>(Num));
		}

		FBlake3Hash GetHash() const
		{
			return HashBuilder.Finalize();
		}

		// This executes the passed in serialization function, and explicitly ignores object references during its execution.
		void IgnoreObjectReferences(TUniqueFunction<void()>&& SerializeFunc)
		{
			bAllowIgnoringObjectReferences = true;
			SerializeFunc();
			bAllowIgnoringObjectReferences = false;
		}

	private:
		FArchive& CheckObjectReferences()
		{
			// Ignoring object references needs to be explicitly allowed.
			ensureMsgf(bAllowIgnoringObjectReferences, TEXT("Hashing of object references is not allowed."));
			return *this;
		}

		FBlake3 HashBuilder;
		bool bAllowIgnoringObjectReferences = false;
	};

	FHashArchive HashArchive;
};
} // namespace UE::MeshPartition
