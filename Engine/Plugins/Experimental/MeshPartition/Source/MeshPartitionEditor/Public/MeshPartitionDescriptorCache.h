// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/SoftObjectPtr.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartition.h"
#include "MeshPartitionDefinition.h"

class UWorldPartition;
class FWorldPartitionActorDescInstance;
class IAssetRegistry;

namespace UE::MeshPartition
{
struct FCompiledSectionDescriptor;
class UModifierComponent;
struct FCachedDescriptors;
class ACompiledSection;

struct FModifierGrouping
{
	// NOTE: this struct is copy-on-write so it is thread-safe after construction
	// (and in general won't be deallocated as long as you hold a TSharedPtr to the parent MeshPartition::FCachedDescriptors struct)

public:
	FModifierGrouping(const MeshPartition::FCompiledSectionBuildVariant& InBuildVariant, const UMeshPartitionDefinition& InDefinition, const MeshPartition::FModifierGroup& AllModifiersGroupSorted);

private:
	FName BuildVariantName;

	// section groups for this build variant
	TArray<MeshPartition::FModifierGroup> ModifierGroups;

private:

public:
	TConstArrayView<MeshPartition::FModifierGroup> GetModifierGroups() const { return ModifierGroups; }

	FName GetBuildVariantName() const { return BuildVariantName; }
};

// stores all of the descriptors relevant to a specific AMeshPartition actor, as well as the FModifierGrouping for each build variant
struct FCachedDescriptors
{
	// NOTE: this struct is copy-on-write so it is thread-safe after construction (as long as you hold a TSharedPtr to it)
	friend struct FWorldCachedDescriptors;
	friend struct FModifierDescriptorCache;

public:
	FCachedDescriptors(const AMeshPartition* InMeshPartition, TConstArrayView<MeshPartition::FModifierDesc> InAllModifiersSorted, TArray<FCompiledSectionDescriptor>&& InCompiledSections);

	// this constructor initializes the cached descriptors, but does not fill out the data.  Used in the construction by FWorldCachedDescriptors
	// TODO: this should be private
	FCachedDescriptors(const FGuid& InMeshPartitionGUID, const AMeshPartition* InMeshPartitionActor);

private:
	// actor GUID of the mesh partition actor
	FGuid MeshPartitionGUID;

	// object path of the mesh partition actor
	TSoftObjectPtr<const AMeshPartition> MeshPartitionActor;

	// group containing all existing modifiers (including both base and non-base modifiers)
	MeshPartition::FModifierGroup AllModifiersGroup;

	// all existing compiled section descriptors
	TArray<FCompiledSectionDescriptor> CompiledSections;

	// section groups for each build variant
	TMap<FName, MeshPartition::FModifierGrouping> BuildVariants;

public:
	MeshPartition::FModifierGroup BuildModifierGroupForBaseModifierPaths(TConstArrayView<FSoftObjectPath> BaseModifierPaths) const;

	TConstArrayView<FModifierDesc> GetAllModifiers() const
	{
		return AllModifiersGroup.AllModifierDescs();
	}

	TConstArrayView<FModifierDesc> GetAllBaseModifiers() const
	{
		return AllModifiersGroup.BaseDescs();
	}

	TConstArrayView<FCompiledSectionDescriptor> GetCompiledSections() const
	{
		return CompiledSections;
	}

	const MeshPartition::FModifierGrouping* GetCachedBuildVariant(const FName& InBuildVariantName) const
	{
		if (const MeshPartition::FModifierGrouping* CachedBuildVariant = BuildVariants.Find(InBuildVariantName))
		{
			return CachedBuildVariant;
		}
		return nullptr;
	}

	TConstArrayView<MeshPartition::FModifierGroup> GetModifierGroups(const MeshPartition::FCompiledSectionBuildVariant& BuildVariant) const
	{
		if (const MeshPartition::FModifierGrouping* CachedBuildVariant = BuildVariants.Find(BuildVariant.Name))
		{
			return CachedBuildVariant->GetModifierGroups();
		}
		// no entry, return empty array
		return TConstArrayView<MeshPartition::FModifierGroup>();
	}

	const FSoftObjectPath& GetMeshPartitionPath() const
	{
		return MeshPartitionActor.GetUniqueID();
	}

	const FGuid& GetMeshPartitionGUID() const
	{
		return MeshPartitionGUID;
	}

	const AMeshPartition* GetMeshPartitionActor() const
	{
		return MeshPartitionActor.Get();
	}

	void ForAllBuildVariants(TFunctionRef<bool(const MeshPartition::FModifierGrouping&)> InFunc) const;
};

// stores the FCachedDescriptors for all AMeshPartition actors in a world
struct FWorldCachedDescriptors
{
	friend struct FModifierDescriptorCache;

private:
	TSoftObjectPtr<const UWorld> World;

	int32 TotalCompiledSections = 0;
	int32 TotalModifiers = 0;
	int32 TotalBuildVariants = 0;
	int32 TotalSectionGroups = 0;

	TArray<TSharedPtr<MeshPartition::FCachedDescriptors>> AllMeshPartitionsCachedDescriptors;

	// Mapping storing the cached descriptors for each mega mesh actor GUID.
	TMap<FGuid, TSharedPtr<MeshPartition::FCachedDescriptors>> CachedDescriptorsByGUID;

	// Mapping storing the cached descriptors for each mega mesh actor path.  We only cache one entry per unique path
	// Bt there can be two paths pointing to the same FCachedDescriptors, if they both share a GUID
	TMap<FSoftObjectPath, TSharedPtr<MeshPartition::FCachedDescriptors>> CachedDescriptorsByPath;

	TSharedPtr<MeshPartition::FCachedDescriptors> FindOrAddByActor(const AMeshPartition& MeshPartitionActor);
	TSharedPtr<MeshPartition::FCachedDescriptors> FindOrAddByGUID(const FGuid& MeshPartitionGUID);

public:
	TSharedPtr<MeshPartition::FCachedDescriptors> GetCachedDescriptors(const AMeshPartition& MeshPartitionActor) const;
	TSharedPtr<MeshPartition::FCachedDescriptors> GetCachedDescriptorsByPath(const FSoftObjectPath& MeshPartitionPath) const;
	TSharedPtr<MeshPartition::FCachedDescriptors> GetCachedDescriptorsByGuid(const FGuid& MeshPartitionGUID) const;
	TConstArrayView<TSharedPtr<MeshPartition::FCachedDescriptors>> GetAllMeshPartitionsCachedDescriptors() const { return AllMeshPartitionsCachedDescriptors; }
};

struct FModifierDescriptorCache
{
public:
	// returns an array of the mesh partition guids that exist in the world
	void UpdateCacheForAllMeshPartitionsInWorld(UWorld* World, TArray<FGuid>* OutAllMeshPartitionGuidsInWorld = nullptr);

	TSharedPtr<MeshPartition::FCachedDescriptors> GetCachedDescriptors(const AMeshPartition* MeshPartition);
	TSharedPtr<MeshPartition::FCachedDescriptors> GetCachedDescriptorsByPath(FSoftObjectPath MeshPartitionPath);
	TSharedPtr<MeshPartition::FCachedDescriptors> GetCachedDescriptorsByGuid(FGuid MeshPartitionGUID);

	void ForAllCachedDescriptors(TFunctionRef<bool(const FSoftObjectPath&, const TSharedPtr<MeshPartition::FCachedDescriptors>&)> InFunc) const;

private:
	void AddMeshPartitionToGuidMap(const AMeshPartition* MeshPartition);

private:
	// Mapping from MeshPartitionGuid (ActorGuid of AMeshPartition actor) to the soft object (path) pointer to the AMeshPartition.  Populated by cached mesh partitions.
	// This can have multiple entries -- for example when a world is duplicated, like when PIE, the same GUID maps to a new object path.
	TMultiMap<FGuid, FSoftObjectPath> GuidToMeshPartitionPath;

	// mapping storing the cached descriptors for each mega mesh actor
	TMap<FSoftObjectPath, TSharedPtr<MeshPartition::FCachedDescriptors>> CachedDescriptors;

	// returns the set of MeshPartition-relevant descriptors in the input World
	TSharedPtr<FWorldCachedDescriptors> GetDescriptorsForAllMeshPartitionsInWorld(UWorld* World);
};
} // namespace UE::MeshPartition

