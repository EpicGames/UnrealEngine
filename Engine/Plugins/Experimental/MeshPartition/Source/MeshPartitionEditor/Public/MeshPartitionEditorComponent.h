// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/ActorComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionComponent.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionGroupRegistry.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionModifierDescriptors.h"
#include "MeshPartitionModifierGraphCache.h"
#include "MeshPartitionCollisionComponent.h"
#include "MeshPartitionTransformer.h"
#include "Tasks/Task.h"

#include "DynamicMesh/DynamicMesh3.h"
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

#include "MeshPartitionEditorComponent.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UStaticMesh;
class AActor;
struct FStaticMeshSourceModel;

namespace UE::MeshPartition
{
class AMeshPartition;
class ACompiledSection;
class AInteractiveSection;
class APreviewSection;
class UMeshPartitionDefinition;
class UModifierComponent;
struct FSectionChannels;

extern MESHPARTITIONEDITOR_API TAutoConsoleVariable<bool> CVarMegaMeshEnablePreviewSimplification;
extern MESHPARTITIONEDITOR_API TAutoConsoleVariable<float> CVarMegaMeshPreviewSimplificationEdgeLength;
extern MESHPARTITIONEDITOR_API TAutoConsoleVariable<int32> CVarMegaMeshPreviewSimplificationMinVertexNumber;

// Struct used to manage async preview section builds.
USTRUCT()
struct FPreviewSectionBuildContext
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TArray<TObjectPtr<MeshPartition::APreviewSection>> IntersectingPreviewSections;

	UPROPERTY()
	TSet<TObjectPtr<MeshPartition::UModifierComponent>> OutdatedBases;

	TArray<MeshPartition::FBuildTaskHandle> BuildTasks;
};

/** Parameters for PrepareCompiledSections. */
struct FPrepareCompiledSectionsParams
{
	// Full mesh for this modifier group.
	const MeshPartition::FMeshData& FullMesh;

	// Grid configuration (CellSize=0 = no grid splitting).
	// Set by the builder from ResolveGridFromPipeline().
	MeshPartition::FGridSettings GridSettings;

	// Receives per-cell sub-meshes after the call, keyed by absolute grid coordinate.
	TMap<FIntVector, MeshPartition::FMeshData> OutCellMeshes;

	// Existing section actor to reuse instead of spawning a new one (single-section only).
	MeshPartition::ACompiledSection* ReuseSection = nullptr;

	// Mobility for spawned actors (applies to single-section path only; grid always uses static).
	bool bUseStaticMobility = true;
};

UCLASS(MinimalAPI, ClassGroup=(MeshPartition))
class UMeshPartitionEditorComponent : public UMeshPartitionComponent
{
	GENERATED_BODY()

public:	
	UE_API UMeshPartitionEditorComponent();
	UE_API ~UMeshPartitionEditorComponent();

	// Begin ActorComponent Implementation
	UE_API virtual void OnUnregister() override;
	virtual void CheckForErrors() override;
	// End ActorComponent Implementation

	// Begin UMeshPartitionComponent Implementation
	UE_API virtual void PostRegisterMegaMeshComponents() override;
	UE_API virtual void PostUnregisterMegaMeshComponents() override;
	// End UMeshPartitionComponent Implementation

	/** Updates the UMeshPartitionEditorComponent flushing pending async tasks and rebuilding pending modifications if needed. */
	UE_API void Update();

	/**
	* Assigns a filter function to control which modifiers are included in the build process
	* @param InFunc the filter function to use
	*/
	UE_API void SetBuildModifierFilterFunction(FModifierFilterFunc InFunc);

	/**
	* Clears any assigned filter function to control which modifiers are included in the build process
	*/
	UE_API void ClearBuildModifierFilterFunction();

	/**
	* Force rebuild the entire megamesh 
	* @param InChangeType The type of change to trigger
	*/
	UE_API void ForceRebuildAllSections(const EChangeType InChangeType);

	/**
	* Builds one or multiple APreviewSections encompassing the provided bounds. MeshPartition::APreviewSection is a EditorOnly class.
	* @param InBounds The bounds to use to build the compiled section.
	*/
	UE_API void BuildMegaMeshPreviewSections(TConstArrayView<const FBox> InBounds);

	/**
	 * Spawns one or more ACompiledSection actors for a modifier group.
	 * Dispatches to a grid-aligned split or a single section based on InBuildVariant.bSplitSectionsToMatchWorldPartitionRuntimeGrid and InParams.GridSettings.IsGridSplit():
	 *   - Grid path (bSplitSectionsToMatchWorldPartitionRuntimeGrid=true, InParams.GridSettings.IsGridSplit()):
	 *       Splits FullMesh by GridSettings (honoring bIs2D), spawns one section per intersecting cell,
	 *       populates OutCellMeshes with the per-cell sub-meshes.
	 *   - Single path (all other cases):
	 *       Spawns one section. Warns if bSplitSectionsToMatchWorldPartitionRuntimeGrid=true but GridSettings.IsGridSplit() is false.
	 * @return Array of spawned/reused sections; never empty on success.
	 */
	UE_API TArray<MeshPartition::ACompiledSection*> PrepareCompiledSections(
		const MeshPartition::FCompiledSectionBuildInfo& InBuildInfo,
		const MeshPartition::FCompiledSectionBuildVariant& InBuildVariant,
		MeshPartition::FPrepareCompiledSectionsParams& InParams) const;

	/**
     * Only used when building a preview section. Executes transformers on a preview mesh component.
     * This is a synchronous operation and should only be used to be able to render properly the preview mesh.
     * Any expensive operation should be delayed until building proper derived data.
     * @param InPreviewMeshComponent The preview component the transformer will be executed on.
     * @param InDefinition The definition in use.
     * @param InVariant The build variant in use.
     */
	UE_API void ExecuteTransformers(MeshPartition::UPreviewMeshComponent* InPreviewMeshComponent, const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InBuildVariant);

	/**
	 * Launches transformers associated with this build variant, seeding the pipeline with the provided initial units.
	 * @param InTransformerUnits The initial transformer units for the pipeline. The pipeline will take ownership of all units.
	 * @param InDefinition The definition used to launch the transformer pipeline.
	 * @param InBuildVariant The variant used to launch the transformer pipeline.
	 * @return If successful, launches the transformer pipeline and returns a new FTransformerContext.
	 */
	UE_API TUniquePtr<MeshPartition::FTransformerContext> LaunchTransformers(TArray<MeshPartition::FTransformerUnit>&& InTransformerUnits, const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InBuildVariant);

	/**
	* Builds the collection of textures containing the channel map information and binds it to the compiled section's material.
	* Modify the UVs of the MeshData for the Section
	*/
	UE_API void BuildMegaMeshCompiledSectionTextures(MeshPartition::ACompiledSection* InCompiledSection, const MeshPartition::FModifierGroup& InModifierGroup, FMeshData& InOutBuiltMesh);

	/**
	* Spawns a Base Modifier using the provided mesh and transform.
	* @param InMesh The dynamic mesh to setup
	* @param InMaterials The initial material used by the base.
	* @param InTransform The transform to use.
	* @return If successful, returns the new Base Modifier, nullptr otherwise.
	*/
	UE_API AActor* SpawnBaseModifier(FDynamicMesh3&& InMesh, const TArray<UMaterialInterface*>& InMaterials, const FTransform& InTransform, const bool bRegisterModifier = true);
	

	/**
	* Spawns a transient preview actor of the specified type into the persistent level of this component's world.
	* Forces the spawn into the persistent level of the editor component so these actors are never placed inside
	* the currently edited level instance or external data layer.
	*/
	template<class ActorType>
	ActorType* SpawnTransientActor(const FTransform& InActorTransform = FTransform::Identity)
	{
		return CastChecked<ActorType>(SpawnTransientActor(ActorType::StaticClass(), InActorTransform));
	}

	/**
	* Spawns a transient preview actor of the specified type into the persistent level of this component's world.
	* Forces the spawn into the persistent level of the editor component so these actors are never placed inside
	* the currently edited level instance or external data layer.
	*/
	UE_API AActor* SpawnTransientActor(TSubclassOf<AActor> InClass, const FTransform& InTransform = FTransform::Identity);

	/** Updates the currently used material resources (section material instance and channel textures). */	
	UE_API void UpdateMaterial();

	/**
	* Called after a MeshPartition::APreviewSection or MeshPartition::ACompiledSection FMeshData has been built.
	* Will call modifiers in need of performing additional processing.
	* @param InSection The section on which the processing can be performed.
	* @param InBuiltMesh The result of the build operation.
	* @param InModifiers The modifiers applied to this section.
	*/
	UE_API void PostBuildSectionMesh(AActor* InSection, const MeshPartition::FMeshData& InBuiltMesh, TConstArrayView<TWeakObjectPtr<MeshPartition::UModifierComponent>> InModifiers) const;

	/**
	 * Called after a MeshPartition::APreviewSection or MeshPartition::ACompiledSection has been built and their associate mesh finalized.
	 * Will call modifiers in need of performing additional processing.
	 * @param InSection The section on which the processing can be performed.
	 * @param InModifiers The modifiers applied to this section.
	 */
	UE_API void PostProcessSection(AActor* InSection, TConstArrayView<TWeakObjectPtr<MeshPartition::UModifierComponent>> InModifiers) const;

	/**
	 * Runs the full synchronous post-build pipeline on a compiled section:
	 * PrepareCompiledSections → ApplyChannels → PostBuildSectionMesh → Transformers (wait) → PostProcessSection.
	 * Resolves Definition and BuildVariant from the section's BuildInfo internally.
	 * Callers must ensure the section's BuildInfo is up-to-date before calling.
	 * @param InSection The section to finalize.
	 * @param InMesh The built mesh data (shared ownership for transformer pipeline).
	 * @param InChannels Pre-built channel data (texture, texcoord metrics, table).
	 * @param InModifierGroup The modifier group for this section.
	 */
	UE_API void ApplyBuiltMeshToSection(MeshPartition::ACompiledSection*				InSection,
										TSharedPtr<const MeshPartition::FMeshData>		InMesh,
										const MeshPartition::FSectionChannels&			InChannels,
										const MeshPartition::FModifierGroup&			InModifierGroup);

	/**
	* Called when a modifier contributing to the MegaMesh changed. Will invalidate and trigger a rebuilt of changed areas.
	* @param InChangedModifier The modifier which has changed.
	* @param InBounds The bounds impacted by the modifier.
	* @param InChangeType the type of this change event.
	*/
	UE_API void OnModifierChanged(MeshPartition::UModifierComponent* InChangedModifier, TConstArrayView<FBox> InBounds, const EChangeType InChangeType);
	UE_API void OnBoundsChanged(TConstArrayView<FBox> InBounds, const EChangeType InChangeType);

	/**
	* Called when a MeshPartition::UModifierComponent is assigned or unassigned to this MegaMesh. 
	*/
	UE_API void OnModifierAssigned();

	/*
	* Recreates list of relevant modifiers by looking through all the loaded modifiers.
	*/
	UE_API void UpdateModifierList();
	
	/**
	* Returns a list of all MegaMeshModifiers that affect this MegaMesh and  all match the passed filter.
	* @param InFilter If this function returns true for a given modifier, it will be added to the returned list.
	* @return The list of matching modifiers.
	*/
	UE_API TArray<MeshPartition::UModifierComponent*> GetModifiersFiltered(TFunctionRef<bool(const MeshPartition::UModifierComponent*)> InFilter) const;
	
	/**
	* Calls the provided function on all currently loaded MeshPartition::UModifierComponent belonging to the owner MegaMesh.
	* @param InFunc The function that will be called on MeshPartition::UModifierComponent. Should return true to continue iteration, false to stop.
	*/
	UE_API void ForAllCurrentModifiers(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc) const;

	/**
	* Calls the provided function on all APreviewSections belonging to the owner MegaMesh.
	* @param InFunc The function that will be called on MeshPartition::APreviewSection. Should return true to continue iteration, false to stop.
	*/
	UE_API void ForAllPreviewSections(TFunctionRef<bool(MeshPartition::APreviewSection*)> InFunc) const;

	/**
	* Retrieves all currently loaded modifiers to process, intersecting the provided bounds or belonging to the provided preview sections.
	* @param OutModifiers A set containing the MeshPartition::UModifierComponent to process.
	* @param InPreviewSections the preview sections used to retrieve base modifiers.
	* @param InBoundsToBuild The bounds used to filter the modifiers. 
	*/
	UE_API void GetModifiersToProcessForPreviews(TSet<MeshPartition::UModifierComponent*>& 		OutModifiers,
										const TArray<MeshPartition::APreviewSection*>& 	InPreviewSections,
										TConstArrayView<FBox> 					InBoundsToBuild) const;

	/**
	* Iterates through current preview section async builds.
	* Invalidating builds containing common bases with the most recent one.
	* Taking ownership of intersecting preview sections and modifiers contained in conflicting builds.
	* @param InOutModifiers An array containing the MeshPartition::UModifierComponent to process.
	* @param InPreviewSections the preview sections used to retrieve base modifiers.
	* @param InCurrentBuildContext The most recent build context.
	*/
	UE_API void InvalidateConflictingPreviewSectionBuildContexts(TSet<MeshPartition::UModifierComponent*>& 		InOutModifiers,
														const TArray<MeshPartition::APreviewSection*>& 	InPreviewSections,
														const MeshPartition::FPreviewSectionBuildContext& 		InCurrentBuildContext);

	/**
	* Retrieves all currently loaded modifiers which intersect with the passed bounding boxes.
	* @param OutModifiers An array containing the MeshPartition::UModifierComponent within the provided bounds.
	* @param InBoundsToBuild The bounds used to filter the modifiers. 
	*/
	UE_API void GetModifiersWithinBounds(TArray<MeshPartition::UModifierComponent*>& OutModifiers, TConstArrayView<const FBox> InBoundsToBuild) const;

	/**
     * Retrieves all currently loaded modifiers which affect the passed bounding boxes, taking into account chained modifications.
     * @param OutModifiers An array containing the MeshPartition::UModifierComponent affecting the provided bounds.
     * @param InBoundsToBuild The bounds used to filter the modifiers.
     */
	UE_API void GetModifiersAffectingBounds(TArray<MeshPartition::UModifierComponent*>& OutModifiers, TConstArrayView<const FBox> InBoundsToBuild) const;
	
	/**
     * Retrieves all currently loaded modifiers which affect the passed modifiers, taking into account chained modifications.
     * @param OutModifiers An array containing the MeshPartition::UModifierComponent affecting the provided modifiers.
     * @param InModifiers The modifiers used to filter the result.
     */
	UE_API void GetModifiersAffectingModifiers(TArray<MeshPartition::UModifierComponent*>& OutModifiers, const TArray<MeshPartition::UModifierComponent*>& InModifiers) const;

	UE_API const MeshPartition::FCommonBuildVariant& GetPreviewBuildVariant() const;

	UE_API void SetBaseModifiersHidden(bool bInHideBaseModifiers);
	bool ShouldHideBaseModifiers() const { return bHideBaseModifiers; }

	/**
	* Returns the effective layer stack ordering index for a modifier's layer name, depending on if a definition is available or not.
	* @param InModifierLayer The modifier's layer name
	* @return Index of the layer
	*/
	UE_API uint32 GetModifierLayerIndex(const FName& InModifierLayer) const;

	/**
	* Enables/Disables preview section building.
	* @param bInEnabled True if building preview section is enabled.
	*/
	void SetPreviewSectionBuildEnabled(const bool bInEnabled) { bBuildPreviewSection = bInEnabled; }
	bool IsPreviewSectionBuildEnabled() const {	return bBuildPreviewSection; }

	/**
	* Returns true if preview sections are currently visible. 
	* @return true if preview sections are currently visible.
	*/
	bool ArePreviewSectionsVisible() const { return bArePreviewSectionsVisible; }
	UE_API void SetPreviewSectionsVisibility(const bool bInArePreviewSectionsVisible);

	/**
	* Called when preview sections are targeted by tools. Is responsible to hide/unhide appropriate preview and interactive sections.
	* @param InPreviewSections The targtted preview sections.
	* @param bInVisible True if they should be visible, false otherwise.
	*/
	UE_API void SetToolTargetVisibility(const TArray<MeshPartition::APreviewSection*> InPreviewSections, const bool bInVisible);
	
	/**
	* @return The Owner's currently used UMeshPartitionDefinition. 
	*/
	UE_API UMeshPartitionDefinition* GetMegaMeshDefinition() const;

	/**
	* Callback called when the owner's currently used UMeshPartitionDefinition changed.
	*/
	UE_API virtual void OnDefinitionChanged(UMeshPartitionDefinition* InNewDefinition) override;

	/**
	* @return the Material described by the definition and generated for this MegaMesh instance
	*/
	UE_API UMaterialInterface* GetDefinitionMaterial() const;

	UMaterialInterface* GetEditorOverrideMaterial() const { return EditorMaterialOverride ? EditorMaterialOverride.Get() : GetDefinitionMaterial(); }

	/**
	* Called when a MeshPartition::UModifierComponent has been loaded via WorldPartition.
	* @param InModifier The loaded Modifier.
	*/
	UE_API void OnModifierLoaded(MeshPartition::UModifierComponent* InModifier);

	/**
	* Called when a MeshPartition::UModifierComponent has been unloaded via WorldPartition.
	* @param InModifier The unloaded Modifier.
	*/
	UE_API void OnModifierUnloaded(MeshPartition::UModifierComponent* InModifier);

	/**
	* @return true if the preview section build process is forced to synchronous.
	*/
	UE_API bool IsSynchronousPreviewSectionBuildForced() const;
	void SetForceSynchronousPreviewSectionBuild(const bool bInForced) { bForceSynchronousPreviewSectionBuild = bInForced; }

	/**
	* Pauses/Unpauses PreviewSections' transformers execution depending on the provided argument.
	* In case of pause, will also take care of notifying the user the build have been paused.
	* In case of unpause, will also launch the previously paused builds.
	* @param bInShouldPauseTransformerPipeline Should preview sections' transformer execution be paused.
	*/
	UE_API void SetPauseTransformerPipeline(const bool bInShouldPauseTransformerPipeline);

	/** Search all active preview section builds and check if a given modifier is participating in one or more builds. */
	UE_API bool IsModifierParticipatingInActivePreviewSectionBuild(const MeshPartition::UModifierComponent* InModifier) const;

	/** Check if any of the preview section builds are running. */
	UE_API bool IsAnyPreviewSectionBuildActive() const;

	/** Re-initializes the group registry with the provided modifier groups. */
	UE_API void ResetGroupRegistry(TConstArrayView<const MeshPartition::FModifierGroup> InModifierGroups);
	UE_API void ResetGroupRegistry();

	UE_API const MeshPartition::FModifierGroup* FindGroupInRegistry(const FGuid& InGroupKey) const;

	/** Will return true if preview section transformer pipeline is paused and the recompute tangents feature is enabled. */
	bool ShouldRecomputeTangents() const;

protected:
	/**
	* Called after loading or unloading actors in the current level.
	*/
	UE_API void OnLoadedActorAddedToLevel(const TArray<AActor*>&);
	UE_API void OnLoadedActorRemovedFromLevel(AActor&);

	/**
	* Called during actor deletion, giving us a chance to uninitialize modifiers
	*/
	UE_API void OnLevelActorDeleted(AActor* Actor);
	
	/**
	* Called when selection changed in the editor.
	*/
	UE_API void OnSelectionChanged(UObject* InObject);

	/**
	* Adds the modifier change information to a pending queue of modifier changes to be bulk processed together usually on the next tick.
	*/
	UE_API void QueuePendingModifierChange(MeshPartition::UModifierComponent* InChangedModifier, TConstArrayView<FBox> InBoundingBoxes, const EChangeType InChangeType);

	/**
	* Gets the PreviewSection intersecting the provided bounds in the provided array.
	* @param InBounds The bounds to rebuild.
	* @param OutIntersectingPreviewSections Output argument, returns the intersecting preview sections.
	* @return An adjusted array of FBox, encompassing the provided bounds plus intersecting preview sections.
	*/
	UE_API TArray<FBox> GetIntersectingPreviewSections(TConstArrayView<FBox> InBounds, TArray<TObjectPtr<MeshPartition::APreviewSection>>& OutIntersectingPreviewSections);

	/**
	* Destroys the provided preview sections. Nullifying any links.
	* @param InSectionsToDestroy The preview sections to be destroyed.
	*/
	UE_API void DestroyPreviewSections(TArray<TObjectPtr<MeshPartition::APreviewSection>>& InSectionsToDestroy);

	/**
	* Updates the relationship links between bases and preview sections.
	* @param InModifierGroup The modifier group used to build the preview section.
	* @param InPreviewSection The preview section resulting from the modifier group.
	*/
	static UE_API void UpdateLinks(const MeshPartition::FModifierGroup& InModifierGroup, MeshPartition::APreviewSection* InPreviewSection);

	/**
	* Callback called when properties are modified in the owner's currently used UMeshPartitionDefinition.
	* @param InMemberName The member name of the changed property.
	* @param InPropertyName The name of the changed property.
	*/
	UE_API void OnDefinitionModified(const FName& InMemberName, const FName& InPropertyName);

	/** Resets the state of the interactive preview section and all preview sections intersecting it. */
	UE_API void ResetInteractiveSection();

	/**
	* Setup interactive section, preview sections and the provided modifiers to be able to enter interactive mode.
	* @param The modifiers to setup. Can be an empty array to clear the previously setup interactive modifiers.
	*/
	UE_API void SetInteractiveModifiers(const TArray<MeshPartition::UModifierComponent*>& InModifiers);

	/**
	* @return True if interactive mode is currently enabled.
	*/
	UE_API bool IsInteractiveModeEnabled() const;
	
	/**
	* @return True if interactive mode is currently enabled, setup and ready to be used.
	*/
	UE_API bool IsInteractiveModeReady() const;

	/** Will invalidate any simplified preview section if the simplification is disabled.  */
	UE_API void OnPreviewSectionSimplificationEnabledChanged(IConsoleVariable* InConsoleVariable);

private:
	/**
	* Takes care of managing async tasks. Should be only called from gamethread.
	* @param bInCompleteAllTasks If true, will try to finalize all completed tasks. False will just complete one task.
	*/
	UE_API void FinalizeAsyncTasks(const bool bInCompleteAllTasks = false);

	/**
	* Takes care of managing preview section async tasks. Should be only called from gamethread.
	*/
	UE_API void FinalizePreviewSectionBuilds();

	/**
	* @return The current number of preview section being built.
	*/
	UE_API uint32 GetPreviewSectionBuildNumber() const;
private:
	UPROPERTY(Transient)
	TArray<TObjectPtr<MeshPartition::APreviewSection>> PreviewSections;

	UPROPERTY(Transient)
	TArray<TObjectPtr<MeshPartition::UModifierComponent>> CurrentModifiers;

	UPROPERTY(Transient)
	TObjectPtr<MeshPartition::AInteractiveSection> InteractiveSection;
	
	UPROPERTY(Transient)
	TArray<TObjectPtr<MeshPartition::APreviewSection>> InteractivePreviewSections;

	// As FTransformerContext can be passed to async thread and the array entry can be relocated, better allocate it.
	TArray<TUniquePtr<MeshPartition::FTransformerContext>> PreviewSectionTransformerContexts;

	TSet<TObjectKey<MeshPartition::APreviewSection>> PreviewSectionsToDestroy;

	UPROPERTY(Transient)
	TArray<MeshPartition::FPreviewSectionBuildContext> PreviewSectionBuildContexts;

	UPROPERTY(Transient)
	TWeakObjectPtr<AActor> LastActorSelected = nullptr;

	UPROPERTY(Transient)
	bool bHideBaseModifiers = true;

	UPROPERTY(Transient)
	bool bArePreviewSectionsVisible = true;

	UPROPERTY(Transient)
	bool bBuildPreviewSection = true;

	UPROPERTY(Transient)
	bool bIsTransformerPipelinePaused = false;

	UPROPERTY(Transient)
	bool bForceSynchronousPreviewSectionBuild = false;

	/** Optional material override to use in the editor when displaying the tool previews or pre-transformed preview meshes.
	 * If set to none, Definition material will be used. */
	UPROPERTY(EditAnywhere, Category = "Rendering")
	TObjectPtr<UMaterialInterface> EditorMaterialOverride;

	FModifierFilterFunc BuildModifierFilterFunc;

	MeshPartition::FModifierGroupRegistry GroupRegistry;

	// True when UpdateFromActorListChange needs to be called
	bool bPendingModifierListChange = false;
	// Set by OnLevelActorDeleted so we know that changes were made in the next UpdateModifierList
	bool bModifierListChanged = false;

	// Queued bounds updates to process on the next update.
	// These bounds invalidate all layers and are not associated with modifiers
	TMap<EChangeType, TSet<FBox>> PendingChangedBounds;
	// Queued bounds updates to process on the next update.
	// Bounds are associated per modifier and updates can be more granular with this knowledge.
	TMap<FSoftObjectPath, TArray<MeshPartition::FModifierChangeInfo>> PendingChangedModifiers;
};
} // namespace UE::MeshPartition

#undef UE_API
