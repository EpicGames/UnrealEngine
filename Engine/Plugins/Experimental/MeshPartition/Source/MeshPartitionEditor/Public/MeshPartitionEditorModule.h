// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

MESHPARTITIONEDITOR_API DECLARE_LOG_CATEGORY_EXTERN(LogMegaMeshEditor, Log, All);

struct FToolMenuContext;
class AStaticMeshActor;

namespace UE::MeshPartition
{
class FMeshPartitionEditorModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void OnPostEngineInit();
	void ModulesChangedCallback(FName InModuleName, EModuleChangeReason InChangeReason);

	void RegisterFoliageBaseIDClassIgnoreList();

	void OnMinimapWarmupTick(UWorld* World);

	TArray<FName> VisualizersToUnregisterOnShutdown;
};
} // namespace UE::MeshPartition