// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "EditorSubsystem.h"
#include "MeshPartitionModifierGraphCache.h"
#include "MeshPartitionMeshBuilder.h"
#include "Tasks/Task.h"
#include "TickableEditorObject.h"
#include "MeshPartitionDescriptorCache.h" 		// MeshPartition::FModifierDescriptorCache
#include "MeshPartitionSeparateWorldBuilder.h"	// MeshPartition::FSeparateWorldBuilder
#include "MeshPartitionCompilerInterface.h"
#include "MeshPartitionEditorUtils.h"			// MeshPartition::EditorUtils::FPIEPathFixer

#include "MeshPartitionEditorSubsystem.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class FAsyncCompilationNotification;

namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;
class UModifierComponent;
class APreviewSection;
struct FMeshPartitionWorldUpdater;

enum class EChangeType
{
	/**
	* Represents a change to the state of the megamesh/modifier.
	* For example: changing any serialized properties such as the position or bounds of a modifier.
	*/
	StateChange,

	/**
	* Represents a change to transient state.
	* Transient state would be restored to a default value when reloading a map.
	* For example: temporarily hiding a modifier via a flag which is not serialized.
	*/
	TransientStateChange,

	/** Represents a transient change such as loading or unloading a modifier, changing an editor visualization setting, etc. */
	TransientChange,
};

struct FModifierChangeInfo
{
	MeshPartition::FModifierDesc ModifierDesc;
	TArray<FBox> ChangedBounds;
	EChangeType ChangeType;
};

struct FOnChangedEventInfo
{
	const UMeshPartitionEditorComponent* ChangedEditorComponent;

	// Per modifier invalidated regions
	const TMap<FSoftObjectPath, TArray<MeshPartition::FModifierChangeInfo>>& ChangedModifiers;

	// Fully invalidated regions (modifier agnostic changes which invalidate all layers)
	const TMap<EChangeType, TSet<FBox>>& ChangedBounds;
};

UCLASS(MinimalAPI)
class UMeshPartitionEditorSubsystem : public UEditorSubsystem, public FTickableEditorObject, public MeshPartition::IMeshPartitionCompilerInterface
{
	GENERATED_BODY()
public:
	DECLARE_EVENT_OneParam(UMeshPartitionEditorSubsystem, FOnMegaMeshChanged, const FOnChangedEventInfo&);

	UE_API UMeshPartitionEditorSubsystem();

	// Begin FTickableEditorObject implementation
	UE_API virtual void Tick(float InDeltaSeconds) override;
	virtual bool IsTickable() const override { return bIsTickable; }
	virtual ETickableTickType GetTickableTickType() const override { return ETickableTickType::Conditional; }
	UE_API virtual TStatId GetStatId() const override;
	// End FTickableEditorObject implementation

	UE_API virtual void Initialize(FSubsystemCollectionBase& InCollection) override;
	UE_API virtual void Deinitialize() override;

	static UE_API UMeshPartitionEditorSubsystem* Get();

	static MeshPartition::FModifierGraphCache* GetGraphCache()
	{
		UMeshPartitionEditorSubsystem* Instance = UMeshPartitionEditorSubsystem::Get();
		return Instance ? &Instance->GraphCache : nullptr;
	}

	static MeshPartition::FMeshBuilder* GetMeshBuilder()
	{
		UMeshPartitionEditorSubsystem* Instance = UMeshPartitionEditorSubsystem::Get();
		return Instance ? &Instance->MeshBuilder : nullptr;
	}

	static MeshPartition::FModifierDescriptorCache* GetDescriptorCache()
	{
		UMeshPartitionEditorSubsystem* Instance = UMeshPartitionEditorSubsystem::Get();
		return Instance ? &Instance->DescriptorCache : nullptr;
	}
	
	static MeshPartition::EditorUtils::FPIEPathFixer& GetPIEPathFixer()
	{
		UMeshPartitionEditorSubsystem* Instance = UMeshPartitionEditorSubsystem::Get();
		check(Instance);
		return Instance->PIEPathFixer;
	}

	/**
	* Fill out a placeholder compiled section by building the mesh for the relevant megamesh modifiers
	*/
	UE_API virtual void BuildPlaceholderCompiledSection(MeshPartition::ACompiledSection* CompiledSection) override;

	/**
	* @return The current total number of preview sections being built for all MegaMeshes.
	*/
	UE_API uint32 GetTotalPreviewSectionBuildNumber() const;

	/**
	* @return The current total number of interactive sections being built for all MegaMeshes.
	*/
	UE_API uint32 GetTotalInteractiveSectionBuildNumber() const;

	/**
	* Sets the number of preview sections being built by a given MegaMeshEditorComponent.
	* @param InMegaMeshEditorComponent The editor component to update.
	* @param InPreviewSectionBuildNumber The new number of preview sections being built.
	*/
	UE_API void SetPreviewSectionBuildNumber(UMeshPartitionEditorComponent* InMegaMeshEditorComponent, uint32 InPreviewSectionBuildNumber);

	/**
	* Sets the state of the interactive section build owned by a MegaMeshEditorComponent.
	* @param InMegaMeshEditorComponent The editor component holding the interactive section to update.
	* @param bInIsBuilding True if the editor component is currently preparing the interactive section.
	*/
	UE_API void SetInteractiveSectionBuild(UMeshPartitionEditorComponent* InMegaMeshEditorComponent, const bool bInIsBuilding);

	UE_API FTextFormat GetPreviewSectionNameFormat() const;
	UE_API FTextFormat GetInteractiveSectionNameFormat() const;

	FOnMegaMeshChanged& OnMegaMeshChanged() { return MegaMeshChangedEvent; }

	UE_API TSharedPtr<FMeshPartitionWorldUpdater> GetPIEWorldUpdaterForWorld(UWorld* World);
	
	UE_API void OnStartPIE(const bool bIsSimulating);
	UE_API void OnPostPIEStarted(const bool bIsSimulating);

	UE_API void OnPrepareEditorGameWorldForPIE(UWorldPartition* WorldPartition);
	UE_API void OnShutdownEditorGameWorldForPIE(UWorldPartition* WorldPartition);

	UE_API void EndPIE(const bool bIsSimulating);
	UE_API void ShutdownPIE(const bool bIsSimulating);
	UE_API void CancelPIE();

	UE_API int32 CountMeshPartitions(UWorld* World);

	void TrackGarbagePreviewSection(MeshPartition::APreviewSection* InPreviewSectionToDestroy);
	void OnPostGC();

private:
	MeshPartition::FModifierDescriptorCache DescriptorCache;
	MeshPartition::FModifierGraphCache GraphCache;
	MeshPartition::FMeshBuilder MeshBuilder;

	UPROPERTY()
	MeshPartition::FSeparateWorldBuilder SeparateWorldBuilder;

	template <typename KeyType, typename ValueType>
	using TWeakObjectPtrKeyMap = TMap<KeyType, ValueType, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<KeyType, ValueType>>;

	TWeakObjectPtrKeyMap<TWeakObjectPtr<UMeshPartitionEditorComponent>, uint32> EditorComponentToPreviewSectionBuildNumber;
	TUniquePtr<FAsyncCompilationNotification> PreviewSectionBuildNotification;

	uint32 TotalPreviewSectionBuildNumber = 0;

	TSet<TObjectPtr<UMeshPartitionEditorComponent>> InteractiveEditorComponents;
	TUniquePtr<FAsyncCompilationNotification> InteractiveSectionBuildNotification;

	FOnMegaMeshChanged MegaMeshChangedEvent;

	bool bIsTickable = true;

	// cached world updater for current PIE session (if any) - this tracks what exists, up to date status, and what needs to be created / updated
	// this only applies to the world specified by PIEOriginalEditorWorldPath (which is the currently open world in editor at PIE startup)
	TSharedPtr<FMeshPartitionWorldUpdater> PIEWorldUpdater;
	FSoftObjectPath PIEOriginalEditorWorldPath;

	MeshPartition::EditorUtils::FPIEPathFixer PIEPathFixer;

	bool bCancelPIE = false;
	
	// Retain an estimate of the total memory occupied by preview sections which have not been GC'd.
	// The editor does not GC incrementally nor at memory pressure and garbage preview section mesh data
	// can represent a significant amount of memory over time.
	int64 EstimatedGarbagePreviewSectionTotalMB = 0.;
};
} // namespace UE::MeshPartition

#undef UE_API
