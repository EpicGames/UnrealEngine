// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/CollisionProfile.h"
#include "Engine/EngineTypes.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionMeshData.h"

class AActor;
struct FStaticMeshSourceModel;
class IAssetRegistry;
class UClass;
class UMaterialInterface;
class UMaterialInstanceDynamic;
class UStaticMesh;

namespace UE::MeshPartition
{
	struct FCommonBuildVariant;
	struct FCompiledSectionBuildInfo;
	struct FCompiledSectionDescriptor;
	struct FModifierGroup;
}
namespace UE::MeshPartition::EditorUtils
{

	/*
	 * Get the MegaMesh-specific version key of the specified class
	 * (NOTE: We use positive MegaMeshClassVersion numbers to indicate deterministic cache keys.)
	 */
	MESHPARTITIONEDITOR_API int32 GetMegaMeshClassVersionFromClass(const UClass* InClass);

	/**
	 * Compute the combined PackageHash for a set of packages
	 * @param AssetRegistry the asset registry used to query each package's file hash.  You should call AssetRegistry.WaitForCompletion() prior to calling this function.
	 * @param PackageNames the set of Packages to compute the hash of. Assumed to already be in a deterministic sorted order.
	 * @param OutPackageChecksums if non-null, a package checksum is appended for each input Package
	 */
	FGuid ComputePackageHash(IAssetRegistry& AssetRegistry, TArrayView<const FName> PackageNames, TArray<uint32>* OutPackageChecksums = nullptr);

	/**
	 * Compute the combined ClassHash for a set of classes (hash of their MegaMeshClassVersion metadata)
	 * This is used to detect when a modifier implementation has changed.
	 * @param Classes the set of classes to compute the hash of.
	 * @param OutClassChecksums if non-null, a class checksum is appended to the array for each input Class
	 */
	FGuid ComputeClassHash(TArrayView<const UClass*> Classes, TArray<uint32>* OutClassChecksums = nullptr);

	/* Check if the package and class hashes recorded on a compiled section are up to date
	*/
	bool PackageHashIsUpToDate(const MeshPartition::FCompiledSectionBuildInfo& BuildInfo, IAssetRegistry& AssetRegistry);
	bool ClassHashIsUpToDate(const MeshPartition::FCompiledSectionBuildInfo& BuildInfo);

	/**
	 * Find all pre-existing compiled sections that match BuildInfo by identity, BuildVariantHash, ModifierSetHash,
	 * PackageHash, and ClassHash. Returns GUIDs for all matching siblings (e.g. N grid-split sections from the same group).
	 * Empty result means no reuse is possible. Does not require loading modifiers.
	 */
	TArray<FGuid> FindMatchingCompiledSectionsFromPackageHashes(
		TArrayView<const FCompiledSectionDescriptor> CompiledSectionDescriptorsToSearch,
		const MeshPartition::FCompiledSectionBuildInfo& BuildInfo,
		IAssetRegistry& AssetRegistry);

	/**
	 * Find all pre-existing compiled sections that match BuildInfo by identity, BuildVariantHash, and ModifiersHash.
	 * Returns GUIDs for all matching siblings. Empty result means no reuse is possible.
	 */
	TArray<FGuid> FindMatchingCompiledSectionsFromModifierHash(
		TArrayView<const FCompiledSectionDescriptor> CompiledSectionDescriptorsToSearch,
		const MeshPartition::FCompiledSectionBuildInfo& BuildInfo);

	/**
	 * Check if a set of objects are unloaded, and if not, log the GC refs that are keeping them from being garbage collected
	 * @param a list of weak object pointers to actors that should have been unloaded in a call to GarbageCollect
	 * @return true if the objects were unloaded, false if any are still in memory
	 */
	bool ValidateObjectsAreUnloaded(TConstArrayView<TWeakObjectPtr<AActor>> InActorsToValidate);

	/**
	 * Creates a static mesh UObject.
	 * @param InOuter The outer UObject to use when creating the UStaticMesh.
	 * @param InStaticMeshName The name to use as a base to generate a unique name.
	 * @return The created UStaticMesh.
	 */
	UStaticMesh* CreateStaticMesh(UObject* InOuter, const FName InStaticMeshName);

	/**
	 * Builds a FStaticMeshSourceModel from the provided built MegaMesh Mesh.
	 * @param OutSourceModel The result of the conversion.
	 * @param InBuiltMesh The Mesh to use as source.
	 * @param bInRecomputeNormals Should normals be recomputed while building the static mesh
	 * @param bInRecomputeTangents Should tangents be recomputed while building the static mesh
	 */
	void BuildSourceModel(FStaticMeshSourceModel& OutSourceModel, const FMeshData& InBuiltMesh, bool bInRecomputeNormals = false, bool bInRecomputeTangents = true);

	/** Struct used to pass settings to FinalizeStaticMesh. */
	struct FFinalizeStaticMeshParams
	{
		UStaticMesh* StaticMesh = nullptr;
		UMaterialInterface* Material = nullptr;
		FCollisionProfileName CollisionProfile = UCollisionProfile::BlockAll_ProfileName;
		int32 NumLODs = 0;
		bool bCanEverAffectNavigation = false;
		bool bUseNanite = true;
		bool bSetupSections = true;
		ENaniteGenerateFallback NaniteFallbackMode = ENaniteGenerateFallback::Enabled;
		ENaniteFallbackTarget NaniteFallbackTarget = ENaniteFallbackTarget::PercentTriangles;
		float NaniteFallbackPercentTriangles = 0.2f;
		float NaniteFallbackRelativeError = 1.0f;
	};

	/** Finalizes the build of the provided static mesh given the settings contained in the passed struct. */
	void FinalizeStaticMesh(const FFinalizeStaticMeshParams& InParams);

	/** Removes any PIE path mangling from an FSoftObjectPath, if any is present, returning the non-PIE / original path. 
	  * Do not use across multiple PIE contexts without Resetting(), as the mapping is cached and may change
	  */
	struct FPIEPathFixer
	{
	private:
		TMap<FName, FName> PIEToNonPIEPackageName;
	public:
		void Reset();
		void FixInPlace(FSoftObjectPath& InOutPath);
	};

	/**
	 * Get the current preview platform, if any (null if no preview is set)
	 * @param OutPreviewPlatformName The name of the preview platform, only valid if ITargetPlatform is returned
	 */
	MESHPARTITIONEDITOR_API ITargetPlatform* GetPreviewPlatform(FName& OutPreviewPlatformName);

	/**
	 * Return the set of current target platforms.  When cooking, this will be the set of active target platforms.
	 * When preview mode is active, it will be the preview platform (as best we can determine).
	 * Otherwise it will be the current platform.
	 */
	MESHPARTITIONEDITOR_API TArray<ITargetPlatform*> GetTargetPlatforms();

	/** Creates or reuses a dynamic material instance. */
	MESHPARTITIONEDITOR_API UMaterialInstanceDynamic* GetOrCreateMaterialInstance(UMaterialInstanceDynamic* InMID, const UMaterialInterface* InBaseMegaMeshMaterial, UObject* InOuter, const FName& InName, EObjectFlags InAdditionalObjectFlags);

} // namespace UE::MeshPartition::EditorUtils
