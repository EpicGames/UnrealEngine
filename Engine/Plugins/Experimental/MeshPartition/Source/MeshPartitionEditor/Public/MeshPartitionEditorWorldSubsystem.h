// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Subsystems/WorldSubsystem.h"

#include "MeshPartitionEditorWorldSubsystem.generated.h"

namespace UE::MeshPartition
{
UCLASS(MinimalAPI, Transient)
class UMeshPartitionEditorWorldSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()
public:
	virtual bool IsTickable() const override { return true; }
	virtual bool IsTickableInEditor() const override { return true; }
	virtual bool DoesSupportWorldType(EWorldType::Type InWorldType) const override { return InWorldType == EWorldType::Editor; }
	virtual TStatId GetStatId() const override;

	virtual void Tick(float InDeltaTime) override;

	static void ReportSectionStatus();
};
} // namespace UE::MeshPartition
