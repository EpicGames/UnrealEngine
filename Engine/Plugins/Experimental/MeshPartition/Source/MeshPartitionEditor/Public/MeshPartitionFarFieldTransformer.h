// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionTransformer.h"

#include "MeshPartitionFarFieldTransformer.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UStaticMesh;

namespace UE::MeshPartition
{

/**
 * Mesh Partition Transformer producing and attaching a far field mesh for each transformer unit present in the execution context.
 * Should be placed relatively early in the pipeline to avoid being executed after a subsection transformer.
 */
USTRUCT(MinimalAPI)
struct FFarFieldTransformer : public MeshPartition::FTransformer
{
	GENERATED_BODY()

public:
	UE_API FFarFieldTransformer() = default;

	UE_API virtual bool Execute(MeshPartition::FTransformerContext& InTransformerContext) const override;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const override;

private:
	TArray<UStaticMesh*> CreateStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext) const;

	void BuildStaticMesh(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask, const int32 InTransformerUnitIndex) const;

	void FinalizeStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask) const;

private:
	UPROPERTY(EditAnywhere, Category="Far Field", meta=(ToolTip="Target edge length used by the simplifier when building the far field mesh."))
	float FarFieldMeshEdgeLength = 1000;
};

} // namespace UE::MeshPartition

#undef UE_API
