// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierDescriptors.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{

/**
 * Registry for managing collections of modifier groups within mesh partition workflows.
 * 
 * This class maintains both a committed set of groups and a staged set that allows
 * modifications to be previewed or accumulated before being finalized.
 *
 * Typical usage flow:
 *   - StageGroup() to prepare a new or modified group.
 *   - CommitStagedGroup() to move it into the persistent registry.
 *   - RevertStagedGroup() to discard staged changes.
 *   - RemoveGroup() to fully delete a committed group.
 *
 */
class FModifierGroupRegistry
{
public:
	using FGroupKeyToGroup = TMap<FGuid, MeshPartition::FModifierGroup>;

	/**
	 * Generates a unique registry key for the given group based on its contents.
	 *
	 * @param InGroup The group to evaluate.
	 * @return A unique FGuid representing the group.
	 */
	static FGuid GetGroupRegistryKey(const MeshPartition::FModifierGroup& InGroup);

    /**
	 * Adds a group to the staged collection, pending commit.
	 *
	 * @param InGroup The group to stage.
	 * @return A unique FGuid representing the staged group.
	 */
	FGuid StageGroup(const MeshPartition::FModifierGroup& InGroup);

	/**
	 * Moves a staged group into the committed collection.
	 *
	 * @param InGroupKey The key identifying the staged group to commit.
	 * @return true if the group have been successfully committed.
	 */
	bool CommitStagedGroup(const FGuid& InGroupKey);

	/**
	 * Discards a staged group without committing it.
	 *
	 * @param InGroupKey The key identifying the staged group to revert.
	 * @return true if the group have been successfully reverted.
	 */
	bool RevertStagedGroup(const FGuid& InGroupKey);

    /**
	 * Removes a committed group from the registry.
	 *
	 * @param InGroupKey The key identifying the group to remove.
	 */
	void RemoveGroup(const FGuid& InGroupKey);

	/**
	 * Clears both committed and staged groups.
	 */
	void Reset();

	/**
	 * Resets the modifier group registry with a predefined set of groups.
	 * 
	 * This clears any existing committed and staged groups, then populates the registry
	 * with the provided collection. Each group is keyed using its generated registry key.
	 * 
	 * @param InModifierGroups  Array of modifier groups to register as the initial state.
	 *                          All groups are considered committed after initialization.
	 */
	void Reset(TConstArrayView<const MeshPartition::FModifierGroup> InModifierGroups);

	/** @return The committed group map. */
	const FGroupKeyToGroup& GetGroups() const { return Groups; }

    /** @return The staged group map. */
	const FGroupKeyToGroup& GetStagedGroups() const { return StagedGroups; }

	/**
	 * Finds a committed group by its key.
	 *
	 * @param InKey The group identifier.
	 * @return Pointer to the group if found, otherwise nullptr.
	 */
	UE_API const MeshPartition::FModifierGroup* FindGroup(const FGuid& InKey) const;

    /**
	 * Finds a staged group by its key.
	 *
	 * @param InKey The group identifier.
	 * @return Pointer to the staged group if found, otherwise nullptr.
	 */
	UE_API const MeshPartition::FModifierGroup* FindStagedGroup(const FGuid& InKey) const;

	/**
	 * Searches for groups containing the given set of modifiers.
	 *
	 * @param InModifiers List of modifiers to search for.
	 * @return Array of groups that contain at least one of the specified modifiers.
	 */
	UE_API TArray<MeshPartition::FModifierGroup> FindGroupsContainingModifiers(TConstArrayView<MeshPartition::FModifierDesc> InModifiers) const;
	
	/**
	 * Searches for groups intersecting the provided bounds.
	 *
	 * @param InBounds List of bounds to check for.
	 * @return Array of groups that contain at least one of the specified bounds.
	 */
	UE_API TArray<MeshPartition::FModifierGroup> FindGroupsIntersectingBounds(TConstArrayView<const FBox> InBounds) const;

private:
	FGroupKeyToGroup Groups;
	FGroupKeyToGroup StagedGroups;
};

}

#undef UE_API