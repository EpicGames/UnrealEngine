// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Tasks/Task.h"

#define UE_API MESHPARTITIONEDITOR_API

class AActor;
class FPackageSourceControlHelper;

namespace UE::MeshPartition
{
class AMeshPartition;
class UMeshProviderModifier;

/**
* List of parameters for the MegaMesh Heightmap Importer.
*/
struct FHeightmapImportParams
{
	UWorld* World;

	/** Filename of the heightmap to use for import. */
	FString HeightmapFilename;

	/** Number of vertices in X/Y dimension for the generated mesh. */
	FInt32Point MeshResolution;

	/** World space size of the generated mesh. */
	FVector MeshSize;

	/** Number of sections in X/Y dimension. */	
	FInt32Point SectionsResolution;

	/** Save and unload sections as they are created. */
	bool bSaveAndUnload;

	/** Number of location volumes in X/Y dimension. */	
	FInt32Point LocationVolumesResolution;
};

/**
* Information about generated sections within an axis-aligned grid.
*/
struct FSectionInfo
{
	FInt32Point IndexXY;	// 2D index within the uniform grid
	FInt32Point Resolution;	// Mesh resolution within the section (as quads)
	FVector2d MinUV;		// Start of the section expressed as UV-style [0..1] values relative to the overall mesh size.
	FVector2d MaxUV;		// End of the section similar to MinUV.
};

/**
* Imports a heightmap texture file as a new MegaMesh actor which is automatically partitioned along a grid where each tile will have the specified maximum number of vertices.
*/
class FHeightmapImporter
{
public:
	UE_API FHeightmapImporter(const FHeightmapImportParams& InParams);

	/**
	* Execute the import operation. If an FPackageSourceControlHelper is provided, newly created packages will also be added to source control.
	* Returns true if the operation succeeded, false otherwise.
	* Note: On failure, a diagnostic error message can be retrieved with `GetErrorText`.
	*/
	UE_API bool Import(FPackageSourceControlHelper* InSourceControlHelper);

	/** Returns the reason for failure, if any occurred. */
	const FText& GetErrorText() const { return ErrorText; }

	/** Returns the newly created mega mesh actor */
	AMeshPartition* GetMegaMesh() const { return MegaMesh; }

	/** Returns the sections that need to be generated based on a given set of parameters. */
	static UE_API TArray<FSectionInfo> GetSectionInfos(const FHeightmapImportParams& Params);

private:
	UE_API bool BeginLoadHeightmapFile();
	UE_API void SaveActorPackages(const TArray<TWeakObjectPtr<AActor>>& InActorsToSave);

	/** Cancel and block until all outstanding tasks are finished executing. */
	UE_API void CancelAndWait();

	UE_API void UnloadActors(const TArray<TWeakObjectPtr<AActor>>& InActorsToUnload) const;

	FHeightmapImportParams Params;

	Tasks::TTask<TArray64<uint16>> LoadHeightmapTask;

	FInt32Point HeightmapResolution = {0};

	AMeshPartition* MegaMesh = nullptr;

	FText ErrorText;

	TArray<TPair<int, Tasks::FTask>> MeshTasks;
	TArray<MeshPartition::UMeshProviderModifier*> MegaMeshMeshProviders;

	// The importer operates on isolated FDynamicMeshes on parallel threads and are then moved onto the output mesh provider on the game thread.
	// This ensures the correct events/callbacks are fired for the DynamicMeshComponent to ensure bounds are computed, proxies recreated, etc.
	TArray<Geometry::FDynamicMesh3> Meshes;

	// Retain a list of all packages that were saved and unloaded in case the user cancels so they can be reverted.
	TArray<FString> SavedPackages;

	bool bIsCancelled = false;
};
} // namespace UE::MeshPartition

#undef UE_API
