// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GameFramework/Actor.h"
#include "MeshPartitionMeshBuilder.h"

#include "MeshPartitionInteractiveSection.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UMaterialInstanceDynamic;

namespace UE::MeshPartition
{
class AMeshPartition;
class UModifierComponent;
class UMeshPartitionEditorComponent;
class UPreviewMeshComponent;

enum class EInteractiveSectionState : uint8
{
	Initialized = 0,
	BuildingBase,
	PreparingBase,
	BasePrepared,
	BuildingPreviewMesh
};

/**
* This is a transient actor, expressing an interactive part of a MegaMesh.
* This means a region of the MegaMesh where processing is optimized for speed so a modifier
* can be interacted with and respond in a timeframe suited for interaction.
* It only exists in the editor.
*/
UCLASS(MinimalAPI, Transient)
class AInteractiveSection : public AActor
{
	GENERATED_BODY()
	
public:	
	UE_API AInteractiveSection();

	// AActor Implementation
	UE_API void Tick(float InDeltaSeconds) override;
	virtual bool ShouldTickIfViewportsOnly() const override { return true; }
	// End AActor Implementation

	UE_API void Update();
	UE_API void SetParent(AMeshPartition* InMegaMesh);

	UE_API UMeshPartitionEditorComponent* GetMegaMeshEditorComponent() const;

	/** @return The current Interactive Modifiers used by this interactive section. */
	const TArray<TObjectPtr<MeshPartition::UModifierComponent>>& GetInteractiveModifiers() const { return InteractiveModifiers; }

	/**
	* Setup the current interactive modifier to be used by this interactive section.
	* @param InModifiers The modifiers to setup.
	* @param InSettings The settings used to build the initial base on top of which to apply the interactive modifier.
	*/
	UE_API void SetInteractiveModifiers(const TArray<MeshPartition::UModifierComponent*>& InModifiers, MeshPartition::FBuilderSettings&& InSettings);
	
	/** Resets the current interactive modifiers for this interactive section. Cancelling all pending tasks if possible. */
	UE_API void ClearInteractiveModifiers();

	/** Called when the current interactive modifier changed to update the interactive section. */
	UE_API void OnModifierChanged();

	/**
	* Will filter any modifiers that should be applied after the provided modifiers. Will also filter all provided modifiers.
	* @param InModifiers The modifiers that will be applied on top of the base.
	* @param InTypePriorities The type priorities to use.
	* @return A FModifierFilterFunc to be used in the builder.
	*/
	static UE_API FModifierFilterFunc InteractiveSectionBaseFilter(const TArray<MeshPartition::UModifierComponent*>& InModifiers, TConstArrayView<FName> InTypePriorities);

	/**
	* Adds a modifier to the list of managed interactive modifiers.
	* @param InModifier The modifier to add.
	*/
	UE_API void AddModifier(MeshPartition::UModifierComponent* InModifier);

	/**
	* Removes a modifier to the list of managed interactive modifiers.
	* @param InModifier The modifier to remove.
	*/
	UE_API void RemoveModifier(MeshPartition::UModifierComponent* InModifier);

	/** @return True if the interactive section is being used. */
	bool IsInteractiveSectionActive() const { return State != EInteractiveSectionState::Initialized; }

	UMaterialInstanceDynamic* GetMaterialInstance() const { return MaterialInstance; }
	void SetMaterialInstance(UMaterialInstanceDynamic* InMaterialInstance) { MaterialInstance = InMaterialInstance; }

private:
	/** Will transition and launch the preparing base task if building the initial base task completed. */
	UE_API void UpdateBuildingBase();

	/** Will transition and apply the current interactive modifier if preparing the base task completed. */
	UE_API void UpdatePreparingBase();

	/** Will transition and setup the mesh component with the built result if the preview mesh is available and copies are done. */
	UE_API void UpdateBuildingPreviewMesh();

	/** @return The current base mesh used as a source by the mesh builder. */
	FMeshData& GetCurrentSourceBase() { return BaseMeshes[CurrentSourceBase]; }

	/** @return The current base mesh used as a copy source. */
	FMeshData& GetCurrentCopySource() { return BaseMeshes[(CurrentSourceBase + 1) % 3]; }

	/** @return The current base mesh used as a copy destination. */
	FMeshData& GetCurrentCopyDestination() { return BaseMeshes[(CurrentSourceBase + 2) % 3]; }

	/** Rotates the basemeshes source, copy source, copy destination. */
	void RotateBaseMeshes()	{ CurrentSourceBase = (CurrentSourceBase + 1) % 3; }

private:
	UPROPERTY(Transient)
	TObjectPtr<UPreviewMeshComponent> PreviewMeshComponent;

	UPROPERTY(Transient)
	TArray<TObjectPtr<MeshPartition::UModifierComponent>> InteractiveModifiers;

	UPROPERTY(Transient)
	TSoftObjectPtr<AMeshPartition> Parent;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> MaterialInstance;

	MeshPartition::FBuilderSettings BuildSettings;

	TArray<MeshPartition::FBuildTaskHandle> BaseBuildTasks;
	TArray<MeshPartition::FBuildTaskHandle> PreviewBuildTasks;
	Tasks::FTask BaseSetupTask;
	Tasks::FTask CopyTask;

	FMeshData BaseMeshes[3];
	uint32 CurrentSourceBase = 0;

	EInteractiveSectionState State;

	bool bPendingBuildIsOutdated = false;
};
} // namespace UE::MeshPartition

#undef UE_API
