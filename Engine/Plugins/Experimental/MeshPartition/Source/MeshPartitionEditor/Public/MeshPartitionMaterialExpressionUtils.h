// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/TextureCollection.h"
#include "Engine/Texture2DArray.h"
#include "Materials/MaterialExpression.h"
#include "MaterialValueType.h"
#include "MeshPartitionDefinition.h"

#include "MeshPartitionMaterialExpressionUtils.generated.h"

namespace UE::MeshPartition::MaterialExpression
{
#if WITH_EDITOR
	MESHPARTITIONEDITOR_API int32 CompilePickChannelTexture(class FMaterialCompiler* Compiler, int32 InTextureCollection_IDX, int32 InChannelPage_IDX);
	MESHPARTITIONEDITOR_API int32 CompileFetchChannel(class FMaterialCompiler* Compiler, int32 InTextureCollection_IDX, int32 InChannel_IDX);
#endif // WITH_EDITOR
}

namespace UE::MeshPartition
{
/**
 * Core MeshPartition material expression providing access to the channel(s) texture resource
 */
UCLASS(MinimalAPI)
class UMaterialExpressionMeshPartitionResource : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

protected:
	// The actual texture assigned by the MeshPartition using the Material
	UPROPERTY(Transient)
	TWeakObjectPtr<UTexture2DArray> TextureParameter;

#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif

	virtual FGuid& GetParameterExpressionId() override;
	virtual bool CanReferenceTexture() const override { return true; }
	virtual UObject* GetReferencedTexture() const override;

	virtual uint32 GetOutputType(int32 OutputIndex) override
	{
		return MCT_Texture2DArray;
	}
};

/**
* Core MeshPartition material expression providing access to the channel texcoord at the fragment
 */
UCLASS(MinimalAPI)
class UMaterialExpressionMeshPartitionTexcoord : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

protected:

#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
#endif

	virtual uint32 GetOutputType(int32 OutputIndex) override
	{
		return MCT_Float2;
	}
};

/** 
 * Core MeshPartition expression allowing to sample one of the channels at a texcoord location
 */
UCLASS(MinimalAPI)
class UMaterialExpressionMeshPartitionChannelSample : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

protected:
	//~ Begin UMaterialExpression Interface
	#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	#endif
	//~ End UMaterialExpression Interface
public:

	// Input: MeshPartition Resource from the shared parameter expression
	UPROPERTY()
	FExpressionInput ChannelTextureInput;

	// Retrieve the list of Channels available in the associated MeshPartitionDefinition
	UFUNCTION(CallInEditor)
	TArray<FName> GetDefinitionChannels() const;
	int32 FindChannelIndex(FName InChannel) const;

	// The name of the channel sample by this expression
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshPartition, meta = (GetOptions = "GetDefinitionChannels", NoResetToDefault))
	FName Channel;

	// The MeshPartition definition associated providing the list of channels 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = MeshPartition)
	TObjectPtr<UMeshPartitionDefinition> MeshPartitionDefinition;

};

/**
* Core MeshPartition expression allowing to sample one of the channels at a texcoord location
*/
UCLASS(MinimalAPI)
class UMaterialExpressionMeshPartitionChannelSampleIndex : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

protected:
	//~ Begin UMaterialExpression Interface
	#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	#endif
	//~ End UMaterialExpression Interface
public:
	// Input: MeshPartition Resource from the shared parameter expression
	UPROPERTY()
	FExpressionInput ChannelTextureInput;

	// The name of the channel sample by this expression
	UPROPERTY(meta = (RequiredInput = "false", ToolTip = "Defaults to 'ConstChannelIndex' if not specified"))
	FExpressionInput ChannelIndex;

	/** only used if A is not hooked up */
	UPROPERTY(EditAnywhere, Category = MeshPartition, meta = (OverridingInputProperty = "ChannelIndex"))
	int32 ConstChannelIndex = 0;
};


/**
* Helper material expression providing an easy channel inspector rendering 
*/
UCLASS(MinimalAPI)
class UMaterialExpressionMeshPartitionInspector : public UMaterialExpression
{
	GENERATED_UCLASS_BODY()

protected:
	//~ Begin UMaterialExpression Interface
	#if WITH_EDITOR
	virtual int32 Compile(class FMaterialCompiler* Compiler, int32 OutputIndex) override;
	virtual EMaterialValueType GetInputValueType(int32 InputIndex) override;
	virtual EMaterialValueType GetOutputValueType(int32 OutputIndex) override;
	virtual void GetCaption(TArray<FString>& OutCaptions) const override;
	#endif
	//~ End UMaterialExpression Interface

	// Input: MeshPartition Resource from the shared parameter expression
	UPROPERTY()
	FExpressionInput ChannelTextureInput;
};

} // namespace UE::MeshPartition