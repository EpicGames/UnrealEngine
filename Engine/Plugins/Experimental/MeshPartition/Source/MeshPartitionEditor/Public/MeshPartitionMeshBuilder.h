// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MeshPartitionMeshBuilderCommon.h"
#include "MeshPartitionModifierDescriptors.h"
#include "Containers/LruCache.h"
#include "Hash/Blake3.h"
#include "Tasks/Task.h"

namespace UE::MeshPartition
{
class UModifierComponent;
class UChannelCollection;
class FModifierTaskGraph;
struct FGridSettings;
}

namespace UE::MeshPartition::FilterHelpers
{
MESHPARTITIONEDITOR_API uint32 FindLayerPriorityIndexFromName(TConstArrayView<FName> InLayerStack, const FName& InLayerName);

/**
	* Filters the list of modifier descriptors to remove any modifiers which belong in layers higher than the passed layer.
	*/
MESHPARTITIONEDITOR_API FModifierFilterFunc FilterModifiersByLastLayerToBuild(const FName& InLastLayerToBuild, bool bInInclusive = true);
	
/**
	* Filters the list of modifier descriptors to remove any modifiers which belong in layers higher than the passed layer and sub priority.
	*/
MESHPARTITIONEDITOR_API FModifierFilterFunc FilterModifiersUntilSubpriorityWithinLayer(const FName& InLastLayerToBuild, double InLastSubPriorityToBuild, bool bInInclusive = true);

/**
* Filters the list of modifier descriptors to remove any modifiers which are sorted after the passed modifier.
*/
MESHPARTITIONEDITOR_API FModifierFilterFunc FilterModifiersByLastModifierToBuild(const MeshPartition::FModifierDesc& InModifierDescriptor, bool bInInclusive = true);

/**
* Filters the list of modifier descriptors to remove any modifiers which belong in layers higher than the passed layer index.
*/
MESHPARTITIONEDITOR_API FModifierFilterFunc FilterModifiersByIndexToBuild(const uint32& InIndexToBuild, bool bInInclusive);

/**
* Filters the list of modifier descriptors to keep only the bases.
*/
MESHPARTITIONEDITOR_API FModifierFilterFunc FilterOnlyBaseModifiers();
}

namespace UE::MeshPartition::Build
{
/** Starts the async mesh build tasks. */
MESHPARTITIONEDITOR_API TArray<MeshPartition::FBuildTaskHandle> LaunchBuilds(const MeshPartition::FBuilderSettings& InSettings);

/** Blocks the current thread until all async mesh build tasks are complete */
MESHPARTITIONEDITOR_API void Wait(TConstArrayView<MeshPartition::FBuildTaskHandle> InTasks);

/** Returns true if all InTasks are completed. */
MESHPARTITIONEDITOR_API bool AreAllTasksComplete(TConstArrayView<MeshPartition::FBuildTaskHandle> InTasks);
}

namespace UE::MeshPartition::GridHelpers
{
/** Result of grid dimension computation: snapped origin + per-axis counts + per-axis extent. */
struct FGridDimensions
{
	FVector SnappedMin;
	FIntVector OriginCoord;
	FIntVector CellNumber;
	FVector CellExtent;			// Per-axis size of one cell. (CellSize, CellSize, CellSize) in 3D; in 2D the Z component carries the full input Z extent.
	int32 TotalCells;
};

/**
 * Returns the snap anchor in translated by the passed argument.
 * Translation-only: LocalAnchor = InGridSettings.WorldOriginOffset - InLocalToWorld.GetTranslation().
 * @param InGridSettings Grid configuration carrying the world-space WP origin.
 * @param InLocalToWorld The transform to offset the grid. Pass FTransform::Identity for non world space aligned splits.
 * @return The snap anchor expressed in the same space as the mesh bounds (mesh-local).
 */
MESHPARTITIONEDITOR_API FVector ComputeLocalAnchor(const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld);

/**
 * Computes grid dimensions from bounds and grid configuration.
 * Snaps bounds min to grid-aligned coordinate (floor) anchored on ComputeLocalAnchor(InGridSettings, InLocalToWorld),
 * then divides the snapped extents. With WorldOriginOffset=Zero and InLocalToWorld=Identity, this reproduces the
 * legacy world-zero-anchored snap. Pass the actor's LocalToWorld to align the lattice with a WP runtime grid even
 * when the MeshPartition actor is translated.
 * This is the single source of truth for grid dimension logic — used by both BuildSections and EstimateGridCells.
 * @param InBounds The bounding box to subdivide (mesh-local space when InLocalToWorld is the actor transform).
 * @param InGridSettings Grid configuration. CellSize must be > 0; bIs2D collapses Z into a single column; WorldOriginOffset offsets the lattice.
 * @param InLocalToWorld Actor transform used to shift WorldOriginOffset into the same space as InBounds. Pass FTransform::Identity for non world space aligned splits.
 * @return Grid dimensions including snapped origin, per-axis cell counts, and per-axis cell extent.
 */
MESHPARTITIONEDITOR_API FGridDimensions ComputeGridDimensions(const FBox& InBounds, const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld);

/**
 * Splits a mesh into grid-aligned cells and returns a map keyed by absolute world-space grid coordinates.
 * Each coordinate component = floor((cellLocalMin - LocalAnchor) / CellSize), making it independent of the input bounds.
 * Empty cells are excluded from the result.
 * @param InMesh The mesh to split.
 * @param InGridSettings Grid configuration. CellSize must be > 0; bIs2D collapses Z (all cells share Z=0; cell mesh spans full source Z range).
 * @param InLocalToWorld Actor transform used by ComputeLocalAnchor to align cells with a WP runtime grid. Pass FTransform::Identity for non world space aligned splits.
 * @return Map from absolute grid coordinate to cell mesh. Only non-empty cells are included.
 */
MESHPARTITIONEDITOR_API TMap<FIntVector, MeshPartition::FMeshData> BuildGridCellMeshes(const MeshPartition::FMeshData& InMesh, const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld);
}

namespace UE::MeshPartition::BuildHelpers
{
/**
* Returns true if the base simplifications is enabled, taking into account the provided group and build settings.
* @param InGroup The group to check.
* @param InSettings The build settings associated with the provided group.
* @return True if the base simplifications is enabled.
*/
bool ShouldApplyBaseSimplificationForGroup(const MeshPartition::FModifierGroup& InGroup, const MeshPartition::FBuilderSettings& InSettings);

/**
* Returns the cache key for the bases of the provided group, taking into account the build settings.
* @param InGroup The group used to compute the cache key.
* @param InSettings The build settings associated with the provided group.
* @return An FGuid, representing the base cache key.
*/
FGuid ComputeBaseCacheKey(const MeshPartition::FModifierGroup& InGroup, const MeshPartition::FBuilderSettings& InSettings);

/**
* Collapses the modifier stack. Returning a modified FDynamicMesh3 provided as argument.
* @param InSettings The build settings associated with the provided group
* @param InModifierGroup The modifier group to process.
* @param InTaskGraph The task graph used for the processing.
* @return A processing task.
*/
UE::Tasks::FTask ProcessModifierGroup(
	const MeshPartition::FBuilderSettings& InSettings,
	MeshPartition::FModifierGroup InModifierGroup,
	TSharedPtr<MeshPartition::FModifierTaskGraph> InTaskGraph);

TArray<MeshPartition::FModifierDesc> InitializeModifierDescriptors(const MeshPartition::FBuilderSettings& InSettings);


/**
 * Splits the given mesh into multiple meshes according to pre-computed grid dimensions.
 * Callers compute FGridDimensions via GridHelpers::ComputeGridDimensions and pass it here -- this avoids
 * recomputing dimensions when the caller already needs them (e.g. BuildGridCellMeshes), and keeps BuildSections
 * agnostic of how the grid was resolved (FGridSettings, FTransform, future fields, ...).
 * @param InMesh The mesh to split.
 * @param InGridDimensions Pre-computed grid dimensions. Determines per-cell extent, count, and snapping origin.
 * @param bInFilterEmptyMeshes When true, empty cells are removed from the output; when false all cells are included.
 * @return An array containing the splitted mesh.
 */
TArray<MeshPartition::FMeshData> BuildSections(const MeshPartition::FMeshData& InMesh, const GridHelpers::FGridDimensions& InGridDimensions, const bool bInFilterEmptyMeshes);

/**
 * Produces a simplified version of the input mesh by collapsing edges whose lengths fall below the requested tolerance.
 * @param InSourceMesh          The source mesh to simplify.
 * @param InEdgeLength          Target edge-length tolerance in Unreal units.
 * @param bInTransferAttributes If true, per-element attributes are propagated to the simplified mesh.
 * @param bTransferNormals 		If true, normals will be propagated to the simplified mesh.
 * @return A new mesh containing the simplified geometry.
 */
MeshPartition::FMeshData SimplifyMesh(const MeshPartition::FMeshData& InSourceMesh, const float InEdgeLength, const bool bInTransferAttributes, const bool bTransferNormals);

}


namespace UE::MeshPartition
{
/**
* Class used to Build FDynamicMesh3 + optional FDynamicMeshAABBTree3 from a build settings and modifier group.
* 
* It has an optional internal local cache and can write/read to/from ddc based on the builder settings.
*/
class FMeshBuilder
{
protected:
	friend class UMeshPartitionEditorSubsystem;
	friend TArray<MeshPartition::FBuildTaskHandle> Build::LaunchBuilds(const MeshPartition::FBuilderSettings& InSettings);		// this should probably be a static member of MeshPartition::FMeshBuilder

	FMeshBuilder();

public:

	/* 
	* Start the async mesh build task or returns a result from its internal cache.
	*/
	MeshPartition::FBuildTaskHandle Build(const MeshPartition::FBuilderSettings& InSettings, const MeshPartition::FModifierGroup& InModifierGroup, bool bAllowDDC);

	/**
	* Cancel an in flight async mesh build task.
	*/
	void Cancel(const TSharedPtr<MeshPartition::FBuildTask>& InBuildTask);
	
	/**
	* Clears the internal cache.
	*/
	void ClearCache();

	/**
	* Returns the total size used by the cache (Mesh + Spatial)
	*/
	double GetCacheTotalMemoryUsageMB() const;

private:
	/**
	* Start the async mesh build task or returns a result from its internal cache.  This version can be called on any thread, and requires the modifier group to be prepared ahead of time.
	*/
	MeshPartition::FBuildTaskHandle Build_Internal(const MeshPartition::FBuilderSettings& InSettings, MeshPartition::FModifierGroup&& InModifierGroup, bool bAllowDDC);

	bool QueryCache(const MeshPartition::FBuilderSettings& InSettings, FBlake3Hash InKey, MeshPartition::FBuildTaskHandle& OutHandle);
	void EnforceMemoryBudget();

	mutable UE::FMutex CacheMutex;
	TLruCache<FBlake3Hash, TSharedPtr<MeshPartition::FBuildTask>> Cache;
	double LastEnforceMemoryBudget = 0.0;
};} // namespace UE::MeshPartition