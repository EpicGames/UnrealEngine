// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionModifierDescriptors.h"
#include "Hash/Blake3.h"
#include "Tasks/Task.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionBuildPerfStats.h"
#include "MeshPartitionChannelCollection.h"
#include "MeshPartitionChannel.h"
#include "OrientedBoxTypes.h"

namespace UE::MeshPartition
{
struct FBuilderSettings;
struct FBuildTask;
class FModifierTaskGraph;
class UModifierComponent;

using FModifierFilterFunc = TFunction<bool(const MeshPartition::FBuilderSettings&, const MeshPartition::FModifierDesc&)>;

enum class EFilterBoundsMode
{
	/** Include triangles inside filter bounds. */
	Inclusive,
	/** Exclude triangles inside filter bounds. */
	Exclusive,
};

struct FBuilderSettings
{
	/** The type of build requested. */
	MeshPartition::EBuildType BuildType = MeshPartition::EBuildType::Request;

	FTransform Transform;

	/** List of modifiers (including base modifiers) to process */
	TArray<MeshPartition::UModifierComponent*> ModifiersToProcess;

	/** Modifier Type priority stack. Modifiers are sorted according to what index their type appears in this array */
	TArray<FName> TypePriorities;

	/**
	* Optional bounds to filter the output mesh triangles.
	* Only triangles fully contained within the bounds will be kept.
	*/
	TArray<Geometry::FOrientedBox3d> FilterBounds;
	
	MeshPartition::EFilterBoundsMode FilterBoundsMode = EFilterBoundsMode::Inclusive;

	/** Optional filter function for the list of modifiers to process. */
	FModifierFilterFunc ModifierFilter;

	/** Optional if set, will be used instead of base modifiers. */
	TOptional<TNotNull<FMeshData*>> BaseMesh;

	/**
	* Maximum complexity value allowed when building meshes.
	* The system will aggregate modifiers into groups which attempt to satisfy this constraint.
	*/
	double MaxSectionComplexity;

	/** Should we recompute normals after processing modifiers. */
	bool bRecomputeNormals = false;

	/** Should we recompute tangents after processing modifiers. */
	bool bRecomputeTangents = false;

	struct FSimplifierOptions
	{
		float TargetEdgeLength;
		int32 MinVertexCount;
	};

	/** Optional settings to run a simplifier pass on the result of the processing pipeline. */
	TOptional<FSimplifierOptions> SimplifierOptions;

	/** Optional settings to run a texcoord generation pass on the result of the processing pipeline. */
	TOptional<FChannelCollectionUVLayoutOptions> TexcoordGenerationOptions;

	struct FChannelRenderSettings
	{
		MeshPartition::FChannelMap ChannelMap;
		float TexelSize;
	};

	/** Optional settings which control channel texture rendering. If unset, no channel texture will be produced*/
	TOptional<FChannelRenderSettings> ChannelRenderSettings;

	/** Cache result */
	bool bCacheResult = false;

	/** Build spatial query structure */
	bool bBuildSpatial = false;

	/** Should builds with these settings be allowed to use the DDC cache to avoid processing. */
	bool bAllowDDCRead = false;

	/** Should builds produced with these settings be allowed to store their results in DDC. */
	bool bAllowDDCWrite = false;
};

/**
* Handle on a TSharedPtr<MeshPartition::FBuildTask>.
* 
* This class is used so that multiple build requests can share the same Task.
*
* Handle can be cancelled if it is the only handle left.
* 
* Handles have a shared pointer on their build task so that the underlying cache can be cleared without breaking in-flight tasks.
*/
struct FBuildTaskHandle
{
public:
	FBuildTaskHandle() = default;
	MESHPARTITIONEDITOR_API ~FBuildTaskHandle();
	MESHPARTITIONEDITOR_API FBuildTaskHandle(MeshPartition::FBuildTaskHandle&& InHandle);
	MESHPARTITIONEDITOR_API FBuildTaskHandle(const MeshPartition::FBuildTaskHandle& InHandle);
	MESHPARTITIONEDITOR_API FBuildTaskHandle& operator=(const MeshPartition::FBuildTaskHandle& InHandle);
	MESHPARTITIONEDITOR_API FBuildTaskHandle& operator=(MeshPartition::FBuildTaskHandle&& InHandle);

	MESHPARTITIONEDITOR_API bool Wait() const;
	MESHPARTITIONEDITOR_API bool IsCompleted() const;
	MESHPARTITIONEDITOR_API void Cancel();
	
	bool IsCancelled() const { return bIsCancelled; }
	const TSharedPtr<MeshPartition::FBuildTask>& GetTask() const { return Task; }

private:
	friend struct FBuildTask;

	FBuildTaskHandle(const TSharedPtr<MeshPartition::FBuildTask>& InBuildTask);
	void Release(const TSharedPtr<MeshPartition::FBuildTask>& InTask, bool bCancel);

	TSharedPtr<MeshPartition::FBuildTask> Task;

	// This can be true without the underlying task being effectively cancelled if it has other existing handles.
	bool bIsCancelled = false;
};

/**
* Task / Result / Cache entry class for the MeshPartition::FMeshBuilder
*/
struct FBuildTask : public TSharedFromThis<MeshPartition::FBuildTask>
{
public:
	const FTransform& GetTransform() const { return Transform; }
	const MeshPartition::FModifierGroup& GetGroup() const { return Group; }
	const SIZE_T GetByteCount() const { return MeshByteCount + SpatialByteCount; }

	MESHPARTITIONEDITOR_API TSharedPtr<const FMeshData> GetMesh() const;
	MESHPARTITIONEDITOR_API TSharedPtr<FMeshData> GetMutableMesh();

	MESHPARTITIONEDITOR_API TSharedPtr<const FMeshABBTree3> GetSpatial() const;

	MESHPARTITIONEDITOR_API TSharedPtr<const MeshPartition::FSectionChannels> GetSectionChannels() const;

	MESHPARTITIONEDITOR_API MeshPartition::FBuildTaskHandle CreateHandle();
	
	MESHPARTITIONEDITOR_API MeshPartition::FBuildPerfStats GetBuildPerfStats() const;
	
	bool Wait() const;

private:
	friend class FMeshBuilder;
	friend struct FBuildTaskHandle;

	/** Current user count for this task. */
	std::atomic<int32> Handles = 0;

	/** Will be true if this build has been invalidated. */
	bool bIsCancelled = false;

	/** If true, calling GetMutableMesh() will fail as we can't give up ownership of the DynamicMesh. */
	bool bIsCached = false;

	/** If true, the task isn't completed without its Spatial member being built. */
	bool bBuildSpatial = false;
	
	/** Group representing all the modifiers which were aggregated and contributed to this mesh. */
	MeshPartition::FModifierGroup Group;

	FTransform Transform;

	TSharedPtr<MeshPartition::FModifierTaskGraph> TaskGraph;

	/** Async task for modifier processing. */
	Tasks::FTask BuildTask;

	FBlake3Hash Key;

	/** Results */
	SIZE_T MeshByteCount = 0;
	TSharedPtr<FMeshData> Mesh;

	SIZE_T SpatialByteCount = 0;
	TSharedPtr<FMeshABBTree3> Spatial;
	
	TSharedPtr<MeshPartition::FSectionChannels> SectionChannels;
};
} // namespace UE::MeshPartition
