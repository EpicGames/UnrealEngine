// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MeshPartitionMeshSkirt.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{

class FMeshData;

UENUM()
enum class EMeshSkirtDirectionMethod
{
	FixedDirection,
	VertexNormal
};

USTRUCT(MinimalAPI)
struct FMeshSkirtSettings
{
	GENERATED_BODY()

public:
	// Distance to offset the skirt vertices from the boundary, in the local plane of the boundary
	UPROPERTY(EditAnywhere, Category = "MeshSkirt", meta = (ClampMin = 0))
	float Width = 10.f;

	// Amount to push down the offset vertices
	UPROPERTY(EditAnywhere, Category = "MeshSkirt", meta = (ClampMin = 0))
	float PushDown = 10.f;

	// Method to choose the direction to push offset vertices
	UPROPERTY(EditAnywhere, Category = "MeshSkirt")
	EMeshSkirtDirectionMethod PushMethod = EMeshSkirtDirectionMethod::VertexNormal;

	// Direction to push the offset skirt vertices, if using a fixed direction Push Method
	UPROPERTY(EditAnywhere, Category = "MeshSkirt", meta = (EditCondition = "PushMethod==EMeshSkirtDirectionMethod::FixedDirection"))
	FVector PushDirection = FVector(0, 0, -1);

	// Consider vertices closer than this distance to be the same for purposes of boundary walking
	UPROPERTY(EditAnywhere, Category = "MeshSkirt", AdvancedDisplay, meta = (ClampMin = 0))
	float VertexSnapTolerance = UE_SMALL_NUMBER;

	// Do not add skirts to boundaries with perimeter smaller than this (to skip mesh boundaries that are not section boundaries)
	UPROPERTY(EditAnywhere, Category = "MeshSkirt", AdvancedDisplay, meta = (ClampMin = 0))
	float BoundaryMinPerimeter = 1000.f;
};

UE_API void AddMeshSkirt(FMeshData& InOutMeshData, const FMeshSkirtSettings& InSettings);

} // namespace UE::MeshPartition

#undef UE_API
