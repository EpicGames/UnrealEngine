// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "MeshPartitionMeshData.h"
#include "Misc/EnumClassFlags.h"

namespace UE::MeshPartition
{
enum class EMeshViewComponents : int32
{
	None 			= 0,

	/** Reads or writes the vertex positions of the internal mesh. */
	VertexPos 		= (1 << 0),

	/**
	* When reading, this provides the view an FDynamicSubmesh3 of the region internal mesh inside the view bounds.
	* When writing, this allows the view to inject new topology and replace existing triangles in the region with new ones.
	*
	* Writing a DynamicSubmesh is a stronger operation than VertexPos and will have a worse impact on performance.
	* Prefer VertexPos when possible.
	*/
	DynamicSubmesh 	= (1 << 1),

	/** Reads or writes the vertex attributes of the internal mesh. */
	VertexAttributeWeight 	= (1 << 2),

	/** Reads or writes to the vertex uv sets. */
	VertexUVs = (1 << 3)
};

ENUM_CLASS_FLAGS(EMeshViewComponents);

/**
* Subclass of FDynamicSubmesh3 which allows remapping of the base mesh pointer.
* This is a very dangerous operation which is only possible because of the invariants established
* through the MegaMesh system. Remapping only occurs when caching is deemed possible, which is
* guaranteed to produce an identical base FDynamicMesh3. Thus, all the mappings are consistent.
*/
struct FDynamicSubmesh
{
public:
	FDynamicSubmesh(const FMeshData& InSourceMesh, const TArray<int>& InTriangles, TSet<Geometry::FIndex2i>&& InEdgesOnSubmeshBoundary);

	FDynamicSubmesh() {}
	FDynamicSubmesh(const FDynamicSubmesh& InOther) = default;
	FDynamicSubmesh(FDynamicSubmesh&& InOther) = default;
	FDynamicSubmesh& operator=(const FDynamicSubmesh& InOther) = default;
	FDynamicSubmesh& operator=(FDynamicSubmesh&& InOther) = default;

	const Geometry::FDynamicMesh3& GetSubmesh() const { return Mesh; }
	Geometry::FDynamicMesh3& GetSubmesh() { return Mesh; }

	int MapVertexToSubmesh(int VertexID) const { return VertexMap.GetTo(VertexID); }

	bool VertexExistsInBaseMesh(int VertexID) const;
	int MapVertexToBaseMesh(int VertexID) const { return VertexMap.GetFrom(VertexID); }

	void RemapToParentMesh(const FMeshData& InNewSourceMesh);

	bool GetSubmeshInternalBoundaryEdges(TSet<int>& OutSubmeshInternalBoundaryEdges) const;

	SIZE_T GetByteCount() const
	{
		return Mesh.GetByteCount() 
			+ VertexMap.GetAllocatedSize()
			+ TriangleMap.GetAllocatedSize()
			+ EdgesOnSubmeshBoundary.GetAllocatedSize();
	}

	const FMeshData* SourceMesh = nullptr;

	TMap<FName, Geometry::FDynamicMeshWeightAttribute*> GetSubmeshWeightLayers();

	Geometry::FDynamicMesh3 Mesh;

	/** Maps vertices to/from the source mesh from/to vertices in the submesh */
	Geometry::FIndexMapi VertexMap;

	/** Maps source mesh triangles to submesh triangles */
	TMap<int, int> TriangleMap;

	/**
	* Remember the edges which used to be the boundary edges of the submesh. (VertexIDs are in the source mesh)
	* These are the edges which are shared with the source mesh and may cause cracks if they are modified.
	*/
	TSet<Geometry::FIndex2i> EdgesOnSubmeshBoundary;
};


/**
* A constrained view of an FDynamicMesh3 which is provided to MegaMeshModifierComponents when they apply modifications.
* The view is constructed to only provide data within a specified bounding region to ensure modifiers only modify the base mesh within their bounds.
*/
class FMeshView
{
public:
	FMeshView() = default;

	/** @param InBounds The bounds over which we colllect the view data. Expected to be in the local space of the input mesh. */
	FMeshView(FMeshData* InMesh, FBox InBounds, EMeshViewComponents InReadComponents, EMeshViewComponents InWriteComponents, const TArray<FName>& InUsedChannels);
	FMeshView(const FMeshView&) = default;
	FMeshView& operator=(const FMeshView&) = default;
	
	bool Compare(const FMeshView& InOther);

	/** Returns the vertex position of the base mesh for a given vertex index. */
	MESHPARTITIONEDITOR_API FVector3d GetVertexPos(int InVertexIndex) const;

	/** Returns the weight of the vertex attribute for the requested channel. */
	MESHPARTITIONEDITOR_API float GetVertexAttributeWeight(FName InChannelName, int InVertexIndex) const;

	MESHPARTITIONEDITOR_API FVector2f GetVertexUV(int InVertexIndex, int InUVChannelIndex) const;

	/** Sets the vertex position in the base mesh for a given vertex index. */
	MESHPARTITIONEDITOR_API void SetVertexPos(int InVertexIndex, FVector3d InNewVertexPos);

	/** Sets the weight of the vertex attribute for the requested channel. */
	MESHPARTITIONEDITOR_API void SetVertexAttributeWeight(FName InChannelName, int InVertexIndex, float InNewVertexWeight);

	MESHPARTITIONEDITOR_API void SetVertexUV(int InVertexIndex, FVector2f InNewVertexUV, int InUVChannelIndex);

	int VertexCount() const
	{
		ensureMsgf(EnumHasAnyFlags(ReadComponents, EMeshViewComponents::VertexPos | EMeshViewComponents::DynamicSubmesh), TEXT("Attempted to retrieve vertex ids but mesh view components doesn't contain vertex data"));
		return VertexIDs.Num();
	}

	/**
	* Get an immutable reference to the submesh provided by the view. This is only a valid operation
	* if the view requested read access to a DynamicSubmesh.
	*/
	const Geometry::FDynamicMesh3& GetSubmesh() const
	{
		ensureMsgf(EnumHasAnyFlags(ReadComponents, EMeshViewComponents::DynamicSubmesh), TEXT("Attempted to retrieve mesh from view but mesh view components doesn't contain dynamic mesh data"));
		return Submesh.GetSubmesh();
	}

	/**
	* Get a mutable reference to the submesh provided by the view. This is only a valid operation
	* if the view requested write access to a DynamicSubmesh.
	*/
	Geometry::FDynamicMesh3& GetSubmeshMutable()
	{
		ensureMsgf(EnumHasAnyFlags(WriteComponents, EMeshViewComponents::DynamicSubmesh), TEXT("Attempted to retrieve mesh from view but mesh view components doesn't contain dynamic mesh data"));
		return Submesh.GetSubmesh();
	}
	
	int GetSubmeshVID(int InVertexIndex) const
	{
		ensure(EnumHasAnyFlags(ReadComponents | WriteComponents, EMeshViewComponents::DynamicSubmesh));
		const int VertexID = VertexIDs[InVertexIndex];
		return Submesh.MapVertexToSubmesh(VertexID);
	}

	/**
	* Gets the edges in the submesh that have a neighboring triangle in the base mesh. If an edge in this set
	*  were to be split, it would create a crack when merging the submesh back, because the new vertex will not
	*  be in the base mesh version of the edge.
	* @return True if successul (we found all edge correspondences we expected)
	*/
	bool GetSubmeshInternalBoundaryEdges(TSet<int32>& OutSubmeshEIDs) const;

	/**
	* Returns the current memory usage of this mesh view in megabytes.
	*/
	double GetMemoryUsageMB() const;
private:
	friend class FModifierTaskGraph;

	/**
	* Builds the mesh view of a dynamic mesh, collecting all the attributes from the ReadComponents that are within the set bounds of the view.
	*/
	void Build();

	/**
	* Remaps all internal data structures which reference a specific base DynamicMesh to point to the new dynamic mesh.
	* This is used to retarged the mesh view output data so it can be applied to a new base mesh.
	*/
	void RemapParentMesh(FMeshData* InNewMeshInternal);
	
	/**
	* Applies any writes made to the view by a modifier back to the internal mesh.
	*/
	void Writeback();

	/**
	* Release any memory held by this mesh view.
	*/
	void Release();

private:
	FMeshData* MeshInternal;
	FBox Bounds;

	EMeshViewComponents ReadComponents;
	EMeshViewComponents WriteComponents;
	TArray<FName> UsedChannels;

	/** Mesh View Data **/

	/**
	* Submesh over the view region.
	* Only exists if read/write components require a DynamicSubmesh.
	*/
	MeshPartition::FDynamicSubmesh Submesh;

	/** View of vertex ids of the source mesh which overlap the view bounds */
	TArray<int> VertexIDs;

	/** Cache vertex positions for deferred writeback */
	TArray<FVector3d> VertexPositions;
	TArray<TArray<FVector2f>> VertexUVs;

	/** View of triangle ids of the source mesh which overlap the view bounds */
	TArray<int> TriangleIDs;

	// Retains the set of triangles which have only some of its vertices inside the bounds
	// This is the set of triangles which needs to be checked when looking to weld vertices
	TArray<int> TriangleIDsTouchingBounds;

	/** Cached vertex attribute weights for deferred writeback. */
	TMap<FName, TArray<float>> AttributeWeightChannels;
};
} // namespace UE::MeshPartition