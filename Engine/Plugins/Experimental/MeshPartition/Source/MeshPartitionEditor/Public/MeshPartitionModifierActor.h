// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "GameFramework/Actor.h"
#include "ActorFactories/ActorFactory.h"

#include "MeshPartitionModifierActor.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

struct FActorSpawnParameters;
class AActor;
struct FAssetData;

namespace UE::MeshPartition
{
class UModifierComponent;

UCLASS(MinimalAPI, HideCategories=(HLOD, Physics, Cooking, Activation, Replication, Collision, Networking, Input, Actor, Visualize, WorldPartition, LOD, AssetUserData, Navigation, LevelInstance, DataLayers, Tags))
class AModifierActor : public AActor
{
	GENERATED_BODY()

public:
	UE_API AModifierActor();

	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category="Mesh Partition")
	TObjectPtr<MeshPartition::UModifierComponent> Modifier;
	
protected:
	UPROPERTY(VisibleAnywhere, Category="Actor")
	TObjectPtr<USceneComponent> DefaultSceneRoot;
};

UCLASS(MinimalAPI)
class UModifierActorFactory : public UActorFactory
{
	GENERATED_UCLASS_BODY()

	//~ Begin UActorFactory Interface
	UE_API virtual AActor* SpawnActor(UObject* InAsset, ULevel* InLevel, const FTransform& InTransform, const FActorSpawnParameters& InSpawnParams) override;
	UE_API virtual bool CanCreateActorFrom(const FAssetData& AssetData, FText& OutErrorMsg) override;
	UE_API virtual FString GetDefaultActorLabel(UObject* Asset) const override;
	//~ End UActorFactory Interface
};
} // namespace UE::MeshPartition

#undef UE_API
