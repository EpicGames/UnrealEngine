// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Components/PrimitiveComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshPartitionChannel.h" // FNameWrapper
#include "MeshPartitionMeshView.h"
#include "MeshPartitionModifierBlueprintInterfaces.h"
#include "MeshPartitionModifierDescriptors.h"
#include "Algo/Accumulate.h"
#include "MeshPartitionDependencyContext.h"
#include "MeshPartitionEditorSubsystem.h"

#include "MeshPartitionModifierComponent.generated.h"

#define UE_API MESHPARTITIONEDITOR_API


PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

class FWorldPartitionActorDescInstance;

namespace UE::MeshPartition
{
class AMeshPartition;
class APreviewSection;
class UMeshPartitionDefinition;
class UMeshPartitionEditorComponent;
class FMeshView;
class UModifierComponent;

namespace MegaMeshModifierComponentLocals
{
	// for friending
	void SetSpriteSize(float Size);
	void SetSpritePath(const FString& Path);
}

/**
 * Version of FNameWrapper to use for priority layer names. Gets used with a detail customization that uses
 *  GetOptions metadata function to constrain options when a toggle is engaged.
 */
USTRUCT(MinimalAPI)
struct FPriorityLayerName : public FNameWrapper
{
	GENERATED_BODY()
public:
	FPriorityLayerName() = default;
	FPriorityLayerName(const FName& NameIn) : FNameWrapper(NameIn) {}
	using FNameWrapper::operator=;
};

/**
* An object created by a modifier to apply its modifications on background worker threads. Any data that the object uses must
*  be thread-safe to access.
*/
class IModifierBackgroundOp
{
public:
	using FInstanceInfo = UE::MeshPartition::FInstanceInfo;

	MESHPARTITIONEDITOR_API IModifierBackgroundOp(const FName& InOperationName);

	virtual ~IModifierBackgroundOp() {}

	/**
	* Given some bounds, give back the relevant instances of the modifier. For many modifiers there is just one instance,
	*  but some may have multiple instances in the world that share most settings but differ in location/bounds/etc.
	*/
	virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const = 0;

	/**
	* Modify the provided MeshView for the given instance.
	* @param InMeshView The view to the mega mesh that this component is allowed to modify. Modifiers can only affect the mesh within their defined bounds. The view enforces this constraint.
	* @param InTransform The World Transform of the modified mesh.
	*/
	virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceInfo) const = 0;

	UE::Tasks::FTask GetAsyncPrepareTask() const { return AsyncPrepareTask; };

	FName GetOperationName() const { return OperationName; }

	/**
	* Flag which controls whether builds including this modifier are allowed to be cached in DDC.
	*/
	virtual bool DisableDDCWrite() const = 0;


	/** BaseMesh Providers */

	virtual TSharedPtr<const FDynamicMesh3> GetMesh() const { return nullptr; };

	//~ Note: currently this is relative to section owning actor (i.e. GetRelativeTransform()).
	//~ Unclear if this is what we will want long term.
	virtual FTransform GetMeshTransform() const { return FTransform::Identity; };

protected:
	/** Helper that adds a single instance with ID 0 that edits vertex positions if the given bounds intersect the query bounds. */
	static bool AddDefaultInstanceIfIntersects(const FBox& ModifierBounds, const FBox& QueryBounds, TArray<FInstanceInfo>& InstanceInfosOut)
	{
		if (ModifierBounds.Intersect(QueryBounds))
		{
			FInstanceInfo Info;
			Info.Bounds = ModifierBounds;
			Info.InstanceID = 0;
			Info.ReadViewComponents = EMeshViewComponents::VertexPos;
			Info.WriteViewComponents = EMeshViewComponents::VertexPos;
			InstanceInfosOut.Add(Info);
			return true;
		}
		return false;
	}

protected:
	friend class MeshPartition::UModifierComponent;

	const FName OperationName;
	UE::Tasks::FTask AsyncPrepareTask;
};

/** An operation doing nothing. Can be returned by modifier wanting to be skipped depending on the execution context. */
class FPassthroughBackgroundOp : public MeshPartition::IModifierBackgroundOp
{
public:
	FPassthroughBackgroundOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {}

	virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const override
	{
		AddDefaultInstanceIfIntersects(GlobalBounds, InBounds, OutInstanceInfos);
	}

	virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceDesc) const override
	{
	}

	virtual bool DisableDDCWrite() const override { return true; }

	FBox GlobalBounds;
};

/**
* An object created by base sections to provide the base mesh when requested on background worker threads.
*  This is done through a function in case we someday want procedurally generated base sections.
*/
class IBaseMeshProviderOp : public MeshPartition::IModifierBackgroundOp
{
public:
	IBaseMeshProviderOp(const FName& InOperationName) : MeshPartition::IModifierBackgroundOp(InOperationName) {};

	virtual void GetInstancesInBounds(const FBox& InBounds, TArray<FInstanceInfo>& OutInstanceInfos) const {}

	/**
	* Modify the provided MeshView for the given instance.
	* @param InMeshView The view to the mega mesh that this component is allowed to modify. Modifiers can only affect the mesh within their defined bounds. The view enforces this constraint.
	* @param InTransform The World Transform of the modified mesh.
	*/
	virtual void ApplyModifications(MeshPartition::FMeshView& InMeshView, const FTransform3d& InTransform, const FInstanceInfo& InInstanceInfo) const {}

	/**
	* Flag which controls whether builds including this modifier are allowed to be cached in DDC.
	*/
	virtual bool DisableDDCWrite() const { return false; }
};
}

/**
* MegaMesh property keys that will be injected in ActorDescs.
*/
namespace MegaMeshModifierProperties
{
MESHPARTITIONEDITOR_API extern const FName PropertiesVersionNumberName;
MESHPARTITIONEDITOR_API extern const FName MegaMeshModifiersNum;
MESHPARTITIONEDITOR_API extern const FName MegaMeshModifierPath;
MESHPARTITIONEDITOR_API extern const FName MegaMeshGUID;
MESHPARTITIONEDITOR_API extern const FName Class;
MESHPARTITIONEDITOR_API extern const FName BaseGrowth;
MESHPARTITIONEDITOR_API extern const FName Type;
MESHPARTITIONEDITOR_API extern const FName Priority;
MESHPARTITIONEDITOR_API extern const FName Complexity;
MESHPARTITIONEDITOR_API extern const FName ComplexityMultiplier;
MESHPARTITIONEDITOR_API extern const FName IsContiguous;
MESHPARTITIONEDITOR_API extern const FName IsDisabled;

MESHPARTITIONEDITOR_API extern const uint32 PropertiesVersionNumber;
} // MegaMeshModifierProperties



namespace UE::MeshPartition
{

/**
* Identifies a MegaMesh modifier type. The type will impact the modifier role and processing order.
*/
namespace MegaMeshModifierType
{
	/**
	* The base type represent all meshes forming the first layer to be processed.
	*/
	MESHPARTITIONEDITOR_API extern const FName Base;
} // MegaMeshType

/**
* This is the base class for components modifying the MegaMesh.
* This is an EditorOnly Component. Their bounds represent the area they are affecting.
* They provide (overridable) interface to interact with the MegaMesh:
* * Logic to attach/detach to the AMeshPartition (to move/update the MegaMesh as a whole)
* * Invalidate bounds when updated (trigger a new compilation process for an area)
* * Perform modification as a step (by modifying a FDynamicMesh) (could evolve into adding/modifying a displacement render target in the future).
*
* The MegaMeshClassVersion meta data should be set to a positive number if you provide a deterministic cache behavior via GatherDependencies()
* (non-positive ModifierVersions will instead use random cache keys that are updated on change, and are less reliable in general)
* Whenever the implementation of the class is changed (such that it would generate a different result in Apply), the MegaMeshClassVersion should be bumped.
*/
UCLASS(MinimalAPI, Abstract, HideCategories = (HLOD, RayTracing, TextureStreaming, Physics, Collision, Lighting, Rendering, Mobile, Navigation, Activation, Cooking, LOD, AssetUserData), meta = (MegaMeshClassVersion = "0"))
class UModifierComponent : public UPrimitiveComponent,
	public MeshPartition::IModifierBlueprintInterface
{
	GENERATED_BODY()

public:
	UE_API UModifierComponent();
	
	// UObject Implementation
	UE_API virtual void Serialize(FArchive& Ar) override;
	virtual bool IsEditorOnly() const override { return true; }
	UE_API virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	using UObject::PreEditChange; // expose the FEditPropertyChain overload to be callable via Super from subclasses
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void PreEditUndo() override;
	UE_API virtual void PostEditUndo() override;
	// End UObject Implementation

	// UActorComponent Implementation
	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;
	UE_API virtual void OnComponentDestroyed(bool bInDestroyingHierarchy) override;
	UE_API virtual TUniquePtr<FWorldPartitionComponentDesc> CreateClassComponentDesc() const;
	UE_API virtual FBox GetStreamingBoundsEditor() const override;
	UE_API virtual TStructOnScope<FActorComponentInstanceData> GetComponentInstanceData() const override;
	// We need this for the SceneProxy to function properly.
	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	// End UActorComponent Implementation

	// USceneComponent Implementation
	UE_API virtual void PostEditComponentMove(bool bFinished) override;
	UE_API virtual void OnUpdateTransform(EUpdateTransformFlags InUpdateTransformFlags, ETeleportType InTeleport) override;
	// End USceneComponent Implementation

	// UPrimitiveComponent Implementation
#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;
#endif
	// End UPrimitiveComponent Implementation
	
	/**
	* Called after a modifier have been loaded and registered to the MegaMesh in case it needs initialization.
	*/
	virtual void InitializeModifier() {}

	/**
	* Called before a modifier will be unloaded and unregistered in case it need resource cleaning.
	*/
	virtual void UninitializeModifier() {}

	/** Ensures any async resources required by this modifier are prepared. */
	void PrepareResources();

	virtual UE::Tasks::FTask GetAsyncPrepareResourcesTask() const { return {}; }

	/**
	* Called by UMeshPartitionEditorComponent when gathering all of its modifiers to let the modifier know that
	*  it has been registered, so that it doesn't try to call OnModifierAssigned unnecessarily.
	*/
	UE_API virtual void MarkAsRegisteredWithMeshPartition(const AMeshPartition* InMeshPartition);

	/** 
	* @return The MeshPartition currently affected by this modifier. 
	*/
	UE_API AMeshPartition* GetAffectedMeshPartition() const;

	/**
	* A helper to retrieve the parent AMeshPartition's UMeshPartitionEditorComponent. 
	* @return the UMeshPartitionEditorComponent used by the parent AMeshPartition.
	*/
	UE_API UMeshPartitionEditorComponent* GetMeshPartitionEditorComponent() const;
	
	/**
	* Sets the MeshPartition to be affected by this modifier.
	* @param InMeshPartition to be affected by this modifier.
	*/
	UE_API void SetAffectedMeshPartition(AMeshPartition* InMeshPartition);

	//~ This version is called from blueprints, and it issues a notification to the MegaMesh.
	/**
	* Sets the MeshPartition to be affected by this modifier.
	* @param InMeshParitition to be affected by this modifier.
	*/
	UFUNCTION(BlueprintCallable, Category = "Mesh Partition Modifier", meta = (DisplayName = "Set Affected Mesh Partition"))
	UE_API void BP_SetAffectedMegaMesh(AMeshPartition* InMeshPartition) override;

	/**
	 * Binds the modifier to the nearest MeshPartition in the world
	 * @return Nearest attached actor of the bound mesh partition. Otherwise, nullptr if no valid mesh partition was found. 
	 */
	UE_API AActor* BindToNearestMeshPartition();

	/**
	 * Binds the modifier to the nearest MeshPartition.
	 * @return Nearest attached actor of the bound mesh partition. Otherwise, nullptr if no valid mesh partition was found.
	 */
	UFUNCTION(BlueprintCallable, Category = "Mesh Partition Modifier", meta=(DisplayName="Bind To Nearest Mesh Partition"))
	UE_API AActor* BP_BindToNearestMeshPartition();
	
	/**
	* @return If this component is a Base Modifier. Will return the preview section using this modifier as a base.
	*/
	MeshPartition::APreviewSection* GetPreviewSection() const { return PreviewSection.Get(); }

	/**
	* Sets the internal preview section. Meaning this base modifier is used to build the preview section.
	* @param InPreviewSection The preview section to set.
	*/
	UE_API virtual void SetPreviewSection(MeshPartition::APreviewSection* InPreviewSection);
	
	/**
	* @return True if this base modifier isn't used by any preview section. 
	*/
	bool IsFree() const { return !PreviewSection.IsValid(); }

	/**
	* @return The BaseGrowth structure, representing how a given modifier should grow the base bounds.
	*/
	const MeshPartition::FBaseGrowth& GetBaseGrowth() const { return BaseGrowth; }

	/**
	 * Sets the BaseGrowth structure, representing how a given modifier should grow the base bounds.
	 * @param InBaseGrowth FBaseGrowth structure to set.
	 */
	void SetBaseGrowth(const FBaseGrowth& InBaseGrowth);

	/**
	* @return The type of this modifier. Used by the processing stack. 
	*/
	const FName& GetType() const { return Type; }

	/**
	* @return The sub priority used for Modifiers belonging to the same type. 
	*/
	double GetPriority() const { return Priority; }
	void SetPriority(const double InPriority) { Priority = InPriority; }

	/**
	* @return The key used to determine the state of cached preview builds which depend on instances of this modifier.
	*/
	const FGuid& GetCacheKey() const { return CacheKey; }

	/**
	* Sets the type of this modifier. 
	* @param InType The type of this modifier to set.
	*/
	void SetType(const FName& InType) { Type = InType; }
	
	/**
	* Allows to modulate OnChanged behavior. 
	* @param bInIgnoreChanged If true, OnChanged will not forward the callback to UMeshPartitionEditorComponent.
	*/
	void SetIgnoreChanged(const bool bInIgnoreChanged) { bIgnoreChanged = bInIgnoreChanged; }

	/**
	* This function returns single bounding box representing the total area this modifier may affect.
	* This is used for things like world partition streaming bounds which cannot be made more granular than a singular bounds.
	*/
	virtual FBox ComputeCombinedBounds() const { return Algo::Accumulate(ComputeBounds(), FBox()); }

	/**
	* Computes the set of bounding boxes describing the regions this modifier is affecting.
	*
	* Modifiers overriding this function should try to be granular to avoid rebuilding more meshes than are required.
	*
	*/
	virtual TArray<FBox> ComputeBounds() const { return { GetOwnerBounds(false) }; }

	/**
	* Last applied bounds is updated when the modifier is applied to the mesh so changes to those bounds can be tracked and updates can be triggered.
	*/
	UE_API void UpdateLastAppliedBounds();

	/**
	* @return True if any of the input bounds intersects this modifiers bounds
	*/
	UE_API bool IntersectsAnyBounds(TConstArrayView<FBox> InBoundsToTest) const;

	/**
	* @return True if this modifier shouldn't be splitted across multiple sections.
	*/
	virtual bool IsContiguous() const { return false; }

	/**
	* Builds a PropertyKey to be used in ActorDesc properties.
	* @param InModifierIndex The index of this modifier in the actor descriptor's list of modifiers.
	* @param InKey The key of the property.
	* @return A FName ready to be used in ActorDesc properties.
	*/
	static UE_API FName BuildPropertyKey(const FString& InModifierIndex, const FName& InKey);

	/**
	* Called when the modifier changed and needs to be processed again. Will forward the call to the appropriate UMeshPartitionEditorComponent.
	* @param InBoundingBox The invalidated BoundingBox. Should encompass the whole actor.
	* @param bInChangeNondeterministicCacheKeys True will change the cache key (for modifiers with non-deterministic cache keys), invalidating any cached megamesh sections produced by the modifier.
	*/
	UE_API virtual void OnChanged(TConstArrayView<const FBox> InBoundingBoxes, EChangeType InChangeType);

	/**
	* Called after a class member property changes. Equivalent to PostEditChangeProperty, only separated out to allow the base class to handle housekeeping tasks related to Megamesh updates.
	* @param InPropertyChangedEvent The property event indicating what has been changed
	*/
	virtual void PropertyChanged(FPropertyChangedEvent& InPropertyChangedEvent) {};

	/**
	* Called to prepare for applying the modifier. The produced background operation will be called to
	*  apply the modifications on a worker thread. Ignored if it returns nullptr.
	* @param InBuildType The context for which this operation is generated for.
	*/
	virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const { return nullptr; }

	template <typename T, typename... InArgs>
	TSharedPtr<T> AllocateBackgroundOp(const FName& InOpName, InArgs&&... Args) const
	{
		auto Result = MakeShared<T>(InOpName, Forward<InArgs>(Args)...);
		Result->AsyncPrepareTask = GetAsyncPrepareResourcesTask();
		return Result;
	}

	/**
	* @return True if this modifier is contributing to the MegaMesh's texture.
	*/
	virtual bool CanRender() const { return false; }

	/**
    * Called after a MeshPartition::APreviewSection or MeshPartition::ACompiledSection FMeshData has been built in case the modifier
    * needs to perform additional processing.
    * @param InSection The built section on which to perform additional processing.
    * @param InBuiltMesh The result of the build operation.
    *
    */
	virtual void PostBuildSectionMesh(AActor* InSection, const MeshPartition::FMeshData& InBuiltMesh) {}
	
	/**
	* Called after a MeshPartition::APreviewSection or MeshPartition::ACompiledSection has been built and their associated mesh finalized
	* in case the modifier needs to perform additional processing.
	* @param InSection The built section on which to perform additional processing.
	*
	*/
	virtual void PostProcessSection(AActor* InSection) {}

	/**
	* Gets the added complexity when applying this modifier.
	* For example, a mesh adding its vertices should return the vertex number.
	* @return The complexity this modifier adds. 
	*/
	virtual double GetComplexity() const { return 0.f; }

	/**
	* Gets the complexity multiplier when applying this modifier.
	* For example, a modifier re-meshing a base by doubling the amount of detail should return 2.
	* @return The complexity multiplier.
	*/
	virtual float GetComplexityMultiplier() const {	return 1.f; }

	/**
	* Allows to determine if this modifier will be used as a base.
	* @return True if this modifier will be the first to be processed on the stack.
	*/
	virtual bool IsBase() const { return false; }

	/**
	* Called when the parent AMeshPartition's UMeshPartitionDefinition changed.
	* @param InDefinition The new UMeshPartitionDefinition.
	*/
	virtual void OnMegaMeshDefinitionChanged(const UMeshPartitionDefinition* InDefinition) {}

	/**
	* Called when a property from the parent AMeshPartition's UMeshPartitionDefinition changed.
	* @param InDefinition The currently used UMeshPartitionDefinition.
	* @param InPropertyName The property that changed.
	*/
	virtual void OnMegaMeshDefinitionModified(const UMeshPartitionDefinition* InDefinition, const FName& InPropertyName) {}

	/**
	* Called to show/hide components. Should be responsible of propagating the hidden state to owned/attached components.
	* @param bInIsHidden True if the current component should be hidden.
	*/
	virtual void SetIsTemporarilyHiddenInEditor(const bool bInIsHidden) {}

	static bool IsAffectedMeshPartitionPropertyName(const FName& InName) { return InName == GET_MEMBER_NAME_CHECKED(MeshPartition::UModifierComponent, AffectedMegaMesh); }

	/**
	* Called by the visualizer to give the modifier an oppertunity to render a debug visualization of itself when selected.
	* @param View the current SceneView
	* @param PDI a pointer to the drawing interface for the viewport
	*/
	virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const {};

	/**
	* Function collecting the current list of channels declared in the MegaMesh Definition
	*  This is used by the ui of any Modifier presenting a channel option. 
	*/
	UFUNCTION(CallInEditor)
	UE_API TArray<FName> GetMegaMeshDefinitionChannels() const;

	/**
	 * Function collecting the current list of priority layers declared in the Mesh Partition Definition.
	 */
	UFUNCTION()
	UE_API TArray<FName> GetDefinitionPriorityLayers() const;

	/**
	 * Function setting the priority layer for this modifier from the declared priority layers
	 * in the Mesh Partition Definition.
	 */
	UFUNCTION()
	UE_API void SetPriorityLayer(const FName PriorityLayer);

	/**
	 * Function getting the priority layer for this modifier from the declared priority layers
	 * in the Mesh Partition Definition.
	 */
	UFUNCTION()
	UE_API FName GetPriorityLayer() const;

	/*
	* Get the color of the modifier in it's unselected state for scene proxy drawing.
	*/
	UE_API virtual FColor EditorUnselectedModifierColor() const;	

	/*
	* Called by the SceneProxy to determine if the component should be visualized at the current moment.
	*/
	UE_API void SetDrawBounds(bool bEnabled);
	UE_API virtual bool ShouldDrawBoundingBox() const;

	/*
	* Updates the stored cache key.
	* Deterministic cache keys are always recomputed (by calling GatherDependencies).
	* Non-determinstic cache keys are only changed when bInChangeCacheKey is true.
	*    UpdateCacheKey(true) is called when changes are made to the properties.
	* (NOTE: We use positive MegaMeshClassVersion numbers to indicate deterministic cache keys.)
	*/
	UE_API FGuid UpdateCacheKey(bool bInChangeCacheKey = false);

	/*
	* Get the MegaMeshClassVersion of this modifier.
	*   Final, because this is just a helper function that calls GetMegaMeshClassVersionFromClass.
	*   It MUST return the same value as GetMegaMeshClassVersionFromClass().
	* (NOTE: We use positive MegaMeshClassVersion numbers to indicate we should use deterministic cache keys
	* calculated from the dependencies gathered via GatherDependencies())
	*/
	UE_API int32 GetMegaMeshClassVersion() const;

	virtual FGuid GetCodeVersionKey() const { return FGuid(); }

	/*
	* Returns true if the modifier is temporarily disabled from being processed in any MegaMesh builds.
	* Note: Builds containing temporarily disabled modifiers do not get written to DDC,
	* as this state is expected to be reset on next map load. Also does not affect compiled section builds
	*/
	virtual bool IsTemporarilyDisabledInEditor() const { return bIsTemporarilyDisabledInEditor; }

	/**
	* @return The raw value of the serialized bIsDisabled flag.
	*/
	bool GetIsDisabledFlag() const { return bIsDisabled; }

	/**
	* @param bInIsDisabled Sets disabled flag for the modifier.
	*/
	UE_API void SetIsDisabledFlag(const bool bInIsDisabled);

	/**
	* @return True if the modifier should be processed.
	*/
	bool IsDisabled() const { return GetIsDisabledFlag() || IsTemporarilyDisabledInEditor(); }

	/** Marks this modifier as currently interactive.
	* @param bInIsInteractive Set to true if this modifier is currently interactive.
	*/
	void SetIsInteractive(const bool bInIsInteractive) { bIsInteractive = bInIsInteractive; }
	
	/** @return True if this modifier is currently interactive */
	bool IsInteractive() const { return bIsInteractive; }

	/** @return An array of modifiers to use when building meshes for interactive mode. */
	virtual TArray<UModifierComponent*> GetInteractiveProxies() { return TArray<UModifierComponent*>(); }

protected:
	friend class UWorldPartitionMeshPartitionBuilder;

	/**
	* Gets the bound of the actor owning this component.
	* @param bShouldBeRegistered True if the method should also take into account unregistered components.
	* @return The bounding box of the owner.
	*/
	UE_API FBox GetOwnerBounds(const bool bShouldBeRegistered) const;


	/*
	* Gather Dependencies for this modifier, including asset and class dependencies
	* and other relevant data (for example local settings) that affect the modifier apply behavior.
	* 
	* The dependencies are used to detect changes to the modifier that would require rebuilding sections that use this modifier.
	* Both incremental build of compiled sections, as well as the in-editor build of preview sections track the declared dependencies.
	* 
	* PackageHash incremental build is a method to detect changes by that does not require loading the Modifiers into memory,
	* so is much faster than a ModifierHash check, and is usually the best method to use for incremental iterative local builds on large worlds.
	* 
	* PackageHash requires that all relevant assets and classes that affect the application of this modifier (in the current state) 
	* are declared in GatherDependencies(), so it can track whether the packages have changed on disk, or class implementations have changed.
	* 
	* You do not need to declare the modifier itself, it's parent actor, or the modifier's class as dependencies in GatherDependencies().
	* Those are automatically pulled in by the system.  Any additional assets or classes used should be declared (especially assets stored in different packages).
	* 
	* Note that the class implementations are tracked by their MegaMeshClassVersion metadata, so any class that is modified in a way that affects
	* modifier application behavior needs to change it's MegaMeshClassVersion metadata.
	* 
	* If a derived modifier class does NOT override this function, it should set its MegaMeshClassVersion to a non-positive number (N <= 0) to indicate that
	* it wishes to use non-deterministic cache keys -- stateful cache keys that are simply randomized whenever the editor makes a change to the modifier.
	* 
	* When a derived modifier DOES override this function, it should set its MegaMeshClassVersion to a positive number (N > 0) to indicate that
	* it is using a deterministic cache key, that can be recomputed at any time by calling this function.
	* 
	* Whenever a derived modifier changes the implementation of GatherDependencies(),
	* OR changes the implementation of the Apply in a way that modifies its behavior, the MegaMeshClassVersion should also be changed.
	* 
	* @param Dependencies The dependency context used to report dependencies.
	*/
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const;

	/**
	* A helper to retrieve the parent AMeshPartition's UMeshPartitionDefinition. 
	* @return the UMeshPartitionDefinition used by the parent AMeshPartition.
	*/
	UE_API UMeshPartitionDefinition* GetMegaMeshDefinition() const;

#if WITH_EDITOR
	/**
	 * Determines whether the editor-only sprite billboard is shown
	 * 
	 * Note: currently this only gets queried on component creation, so dynamic visibility control
	 *  will require the subclass to update the visibility itself.
	 */
	virtual bool ShouldShowSpriteComponent() const { return true; }
#endif // WITH_EDITOR
private:
	// If bNeedToRegisterWithMegaMesh is true and we have a MegaMesh, notifies it of
	//  our existence and sets bNeedToRegisterWithMegaMesh to false.
	UE_API void RegisterWithMegaMeshIfNeeded();

	// Used to filter out modifiers that are a template in the BP editor, or are in the BP preview viewport
	UE_API bool IsInRelevantWorld() const;

#if WITH_EDITOR
	UE_API void CreateModifierSpriteComponent();
#endif // WITH_EDITOR
private:
	UPROPERTY(Transient)
	TWeakObjectPtr<MeshPartition::APreviewSection> PreviewSection;

	/**
	* MegaMesh affected by the modifier. Its associated definition controls the ordering of priority layers,
	*  and channels available in the final result.
	*/
	UPROPERTY(EditInstanceOnly, Category = "Modifier", Meta = (DisplayName = "Affected Mesh Partition"))
	TSoftObjectPtr<AMeshPartition> AffectedMegaMesh;

	UPROPERTY(EditAnywhere, Category = "Modifier", AdvancedDisplay)
	MeshPartition::FBaseGrowth BaseGrowth;

	/**
	* Modifiers are grouped into priority layers whose relative application order is controlled by the mesh partition definition.
	*/
	UPROPERTY(EditAnywhere, Category = "Modifier", Meta = (DisplayName = "Priority Layer", GetOptions = "GetDefinitionPriorityLayers"))
	FPriorityLayerName Type;

	/**
	* The sub priority orders modifiers within the same priority layer (higher is applied later).
	*/
	UPROPERTY(EditAnywhere, Category = "Modifier", Meta = (DisplayName = "Sub-Priority"))
	double Priority = 0.;

	UPROPERTY(VisibleAnywhere, Category = "Modifier|Advanced", AdvancedDisplay, DuplicateTransient, TextExportTransient)
	FGuid CacheKey;

	// Transient Visualization Settings
	UPROPERTY(Transient)
	bool bDrawBounds;

	/** Disable processing this modifier completely. */
	UPROPERTY(EditInstanceOnly, Category = "Modifier|ModifierSettings", Meta = (DisplayName = "Disabled"))
	bool bIsDisabled = false;

	/**
	* Temporarily disable processing this modifier in any Mesh Partition builds (Preview/PCG Query/Modifier tool targets).
	* Note: Builds containing temporarily disabled modifiers do not get written to DDC,
	* as this state is expected to be reset on next map load. Also does not affect compiled section builds.
	*/
	UPROPERTY(EditInstanceOnly, Category = "Modifier|ModifierSettings", SkipSerialization, Transient, DuplicateTransient, TextExportTransient, Meta = (DisplayName = "Disabled only in Editor", EditCondition = "!bIsDisabled"))
	bool bIsTemporarilyDisabledInEditor = false;

	// The last bounds of the modifier that were integrated into the built mesh
	TArray<FBox> LastAppliedBounds;

	// The last bounds of the modifier that were submitted for evaluation, which
	//  may or may not have been applied yet. Used for invalidating in-flight section
	//  builds.
	TArray<FBox> LastSubmittedBounds;

	// Used to detect changes to our assigned megamesh across undo
	AMeshPartition* PreUndoMeshPartition = nullptr;

	// Transient property that starts out true to make it easy to make sure that the
	//  MegaMesh knows of our existence (particularly when copying modifiers).
	bool bNeedToRegisterWithMeshPartition = true;

	bool bIgnoreChanged = false;
	bool bIsInteractive = false;

	friend struct FModifierComponentInstanceData;
	friend void MegaMeshModifierComponentLocals::SetSpriteSize(float Size);
	friend void MegaMeshModifierComponentLocals::SetSpritePath(const FString& Path);
};

/** Used to store un-serialized modifier component data during RerunConstructionScripts */
USTRUCT()
struct FModifierComponentInstanceData : public FPrimitiveComponentInstanceData
{
	GENERATED_BODY()
public:
	FModifierComponentInstanceData()
		: bIsTemporarilyDisabledInEditor(false)
	{}
	explicit FModifierComponentInstanceData(const MeshPartition::UModifierComponent* SourceComponent)
		: Super(SourceComponent)
		, bIsTemporarilyDisabledInEditor(SourceComponent && SourceComponent->bIsTemporarilyDisabledInEditor)
	{}
	virtual ~FModifierComponentInstanceData() = default;

	virtual bool ContainsData() const override
	{
		return Super::ContainsData() || bIsTemporarilyDisabledInEditor;
	}

	virtual void ApplyToComponent(UActorComponent* Component, const ECacheApplyPhase CacheApplyPhase) override
	{
		Super::ApplyToComponent(Component, CacheApplyPhase);
		CastChecked<MeshPartition::UModifierComponent>(Component)->bIsTemporarilyDisabledInEditor = bIsTemporarilyDisabledInEditor;
	}

	UPROPERTY()
	bool bIsTemporarilyDisabledInEditor = false;
};
} // namespace UE::MeshPartition

// Needed to get SerializeFromMismatchedTag to be called to convert from serialized FName properties
template<>
struct TStructOpsTypeTraits<UE::MeshPartition::FPriorityLayerName> : public TStructOpsTypeTraitsBase2<UE::MeshPartition::FPriorityLayerName>
{
	enum
	{
		WithStructuredSerializeFromMismatchedTag = true
	};
};

#undef UE_API
