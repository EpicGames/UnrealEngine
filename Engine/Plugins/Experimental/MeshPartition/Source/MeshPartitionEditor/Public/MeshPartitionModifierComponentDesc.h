// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WorldPartition/WorldPartitionActorDesc.h"
#include "MeshPartitionModifierDescriptors.h"

namespace UE::MeshPartition
{

class FWorldPartitionModifierComponentDesc : public FWorldPartitionComponentDesc
{
public:
	virtual uint32 GetSizeOf() const override;
	virtual void Init(const UActorComponent* InComponent) override;
	virtual void Serialize(FArchive& Ar) override;

public:
	/** Component->World transform. Serialized in the descriptor so we can transform the LocalBounds back into worldspace after serializing. */
	UE_DEPRECATED(5.8, "ComponentTransform is no longer updated and is only present for backwards compatibity. Use ComponentToActorTransform instead")
	FTransform ComponentTransform;
	
	/**
	 * Component->Actor transform.
	 * Serialized in the descriptor so we can transform the LocalBounds back into worldspace after serializing after multiplying by the actor transform in the actor descriptor.
	 */
	FTransform ComponentToActorTransform;

	/** Path to the modifier component, used as a unique identifier for this modifier */
	FSoftObjectPath ModifierPath;

	/** Type of the modifier, affects grouping and sort order */
	FName Type;

	/** Relative priority of this modifier, affects sort order */
	double Priority;

	/** The world partition actor desc GUID for the parent mega mesh actor */
	FGuid MegaMeshGuid;

	/** Localspace bounds of the modifier. Must be serialized as local space bounds for delta serialization to work.*/
	FBox LocalBounds;

	/* Other values used to compute groups */
	MeshPartition::FBaseGrowth BaseGrowth;
	double Complexity;
	float ComplexityMultiplier;
	bool bIsContiguous;
	bool bIsDisabled;
	bool bIsBase;
	
	/**
	 * Indicates if this component has a ComponentToActor transform or not.
	 * This allows us to detect old modifiers which serialized the incorrect transform
	 * and still load them without breaking.
	 */
	bool bHasComponentToActorTransform;
};

} // namespace UE::MeshPartition