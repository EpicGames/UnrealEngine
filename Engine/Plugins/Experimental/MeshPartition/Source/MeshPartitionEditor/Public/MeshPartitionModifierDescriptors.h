// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/SoftObjectPath.h"
#include "WorldPartition/WorldPartitionHandle.h"
#include "MeshPartitionMeshView.h" // EMeshViewComponents

#include "MeshPartitionModifierDescriptors.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class FWorldPartitionActorDescInstance;
class UWorldPartition;
class ULevelStreaming;

namespace UE::MeshPartition
{
class AMeshPartition;
class UModifierComponent;
struct FBuilderSettings;
class IBaseMeshProviderOp;
class IModifierBackgroundOp;
struct FBuilderSettings;
class FWorldPartitionModifierComponentDesc;

/*
* Enum describing the type of usecase requesting a mesh build.
*/
enum class EBuildType : uint8
{
	/** A manual build request external to the pipeline. */
	Request = 0,

	/** The result will be used to create a preview section. */
	PreviewSection = 1 << 0,

	/** The result will be used to create a compiled section. */
	CompiledSection = 1 << 1,

	/** Request to build the frozen version of the modifiers stack. Used as a base to apply interactive modifiers onto. */
	InteractiveBase = 1 << 2,

	/** Request to apply the interactive modifiers on top of an already existing base. */
	InteractiveModifier = 1 << 3,

	/** Request to build a simplified preview section. */
	SimplifiedPreviewSection = 1 << 4
};

/*
* Struct describing how a modifier should grow the base so other modifiers can be taken into account when creating groups.
*/
USTRUCT()
struct FBaseGrowth
{
	GENERATED_BODY()

	const bool& operator[](const uint32 InIndex) const;
	FString ToString() const;
	bool InitFromString(const FString& InSourceString);

	friend FArchive& operator<<(FArchive& Ar, FBaseGrowth& BaseGrowth)
	{
		return Ar << BaseGrowth.X << BaseGrowth.Y << BaseGrowth.Z;
	}

	UPROPERTY(EditAnywhere, Category = "MeshPartition")
	bool X = false;

	UPROPERTY(EditAnywhere, Category = "MeshPartition")
	bool Y = false;

	UPROPERTY(EditAnywhere, Category = "MeshPartition")
	bool Z = true;

	bool operator==(const FBaseGrowth&) const = default;
};

/*
* Struct that can hold all of the ActorDesc properties for a mega mesh modifier
* (These are the properties that are available when the modifiers are not loaded into memory)
*/
struct FModifierDesc
{
	/** Path to the modifier component, used as a unique identifier for this modifier */
	FSoftObjectPath ModifierPath;

	/** Path to the modifier class, used to identify the class to check code versions */
	FSoftClassPath ClassPath;

	/** Type of the modifier, affects grouping and sort order */
	FName Type;

	/** Relative priority of this modifier, affects sort order */
	double Priority;

	/** World partition actor desc GUID for the actor containing this modifier component */
	FGuid OwnerGuid;

	/** The world partition actor desc GUID for the parent mega mesh actor */
	FGuid MegaMeshGuid;

	/* Other values used to compute groups */
	FBox Bounds;
	MeshPartition::FBaseGrowth BaseGrowth;
	double Complexity;
	float ComplexityMultiplier;
	bool bIsContiguous;
	bool bIsDisabled;
	bool bIsBase;

	// construct an empty / default descriptor
	FModifierDesc();

	// construct from a modifier
	MESHPARTITIONEDITOR_API FModifierDesc(const MeshPartition::UModifierComponent& InModifier);

	// construct from InActorDescInstance (which can have multiple modifiers, this builds the one indicated by InModifierIndex)
	// #todo: remove once all modifiers in older maps have been resaved.
	MESHPARTITIONEDITOR_API FModifierDesc(const FWorldPartitionActorDescInstance& InActorDescInstance, int32 InModifierIndex);
	MESHPARTITIONEDITOR_API FModifierDesc(const FWorldPartitionActorDescInstance& InActorDescInstance, const FWorldPartitionModifierComponentDesc& InComponentDesc);

	bool operator==(const FModifierDesc&) const = default;

	MESHPARTITIONEDITOR_API bool IsValid() const;
	bool IsBase() const { return bIsBase; }
};

/** Typesafe wrapper for identifying and indexing modifier component data inside Modifier Groups */
struct FModifierIndex
{
	using IndexType = int32;

	FModifierIndex() : Index(INDEX_NONE) {}
	explicit FModifierIndex(IndexType InIndex) : Index(InIndex) {}

	IndexType Index;

	// Implicit conversion operator to int to allow using this directly as an array index for convenience.
	operator IndexType() const { return Index; }
};

/** Typesafe wrapper for identifying and indexing modifier instances inside Modifier Groups */
struct FInstanceIndex
{
	using IndexType = int32;

	FInstanceIndex() : Index(INDEX_NONE) {}
	explicit FInstanceIndex(IndexType InIndex) : Index(InIndex) {}

	IndexType Index;

	// Implicit conversion operator to int to allow using this directly as an array index for convenience.
	operator IndexType() const { return Index; }
};

/**
* Per-Instance modifier data. Modifiers may declare 0 or more FInstanceInfos.
* Instances will be processed through the MegaMesh processing pipeline as individual entities whose dependencies
* are based on the instance's proper bounds rather than the encapsulating bounds of the owning modifier.
* 
* How modifiers choose to interpret the different instances and what kind of per-instance data they wish to store
* is modifier-specific. The simplest case is usually a separate transform/bounds per-instance but can be as complex
* as separate mesh data or separate textures.
*/
struct FInstanceInfo
{
	FBox Bounds;
	FModifierIndex ModifierIndex;
	int32 InstanceID;

	EMeshViewComponents ReadViewComponents; // #todo: move out of per instance data?
	EMeshViewComponents WriteViewComponents; // #todo: move out of per instance data?
	TArray<FName> UsedChannels; // #todo: move out of per instance data?
};

template <typename IndexType>
struct TIndexIterator
{
	inline bool operator==(const TIndexIterator<IndexType>& InOther) const
	{
		return Index == InOther.Index;
	}
	inline bool operator!=(const TIndexIterator<IndexType>& InOther) const
	{
		return Index != InOther.Index;
	}

	inline IndexType operator*() const
	{
		return IndexType(this->Index);
	}

	inline TIndexIterator<IndexType>& operator++() // prefix
	{
		this->Next();
		return *this;
	}
	inline TIndexIterator<IndexType> operator++(int32) // postfix
	{
		TIndexIterator copy(*this);
		this->Next();
		return copy;
	}
		
	TIndexIterator<IndexType>(int32 InIndex)
	: Index(InIndex)
	{
	}
private:
	void Next()
	{
		++Index;
	}
		
	int32 Index = 0;
};
	
template <typename IndexType>
struct TIndexRange
{
	/**
	* Creates a ranged-based for compatible range which returns all the indices from [Start, End) (inclusive start, exclusive end)
	* Eg: TIndexRange(1, 5) -> [1, 2, 3, 4]
	*/
	TIndexRange(int32 InStart, int32 InEnd)
	: Start(InStart)
	, End(InEnd)
	{}

	TIndexIterator<IndexType> begin()
	{
		return TIndexIterator<IndexType>(Start);
	}
	TIndexIterator<IndexType> end()
	{
		return TIndexIterator<IndexType>(End);
	}
		
	int32 Start = 0;
	int32 End = 0;
};

class IModifierResolver
{
public:
	virtual MeshPartition::UModifierComponent* ResolveModifier(const MeshPartition::FModifierDesc& ModifierDesc) = 0;
	virtual ~IModifierResolver() {};
};

struct FModifierGroup
{
public:
	enum class EState : uint32
	{
		Uninitialized 			= 0,
		/** Modifier group is now locked and further changes to the grouping is blocked. */
		DescriptorsFinalized 	= 1,
		/**
		* Resolved the MeshPartition::UModifierComponent pointers from the soft object paths in the descriptors.
		* Until this point only FModifierDescriptors are valid.
		*/
		ModifiersResolved 		= 2,
		/** Created the async background operators */
		BackgroundOpsCreated 	= 3,
		/** Initialized all modifier instances for processing on the async thread by the task graph. */
		InstancesReady 			= 4,
	};

	/** Returns true if the first passed modifier should be applied before the second according to the passed list of type priorities. */
	MESHPARTITIONEDITOR_API static bool ShouldApplyModifierBefore(TConstArrayView<FName> InLayerPriorities, const MeshPartition::FModifierDesc& InModifier, const MeshPartition::FModifierDesc& InOtherModifier);

	/** Returns true if the first passed modifier instance should be applied before the second according to the passed list of type priorities. */
	bool ShouldApplyInstanceBefore(TConstArrayView<FName> InLayerPriorities, const FInstanceInfo& InInstance, const FInstanceInfo& InOtherInstance) const;
	bool ShouldApplyInstanceBefore(TConstArrayView<FName> InLayerPriorities, FInstanceIndex InInstanceIndex, FInstanceIndex InOtherInstanceIndex) const
	{
		return ShouldApplyInstanceBefore(InLayerPriorities, GetInstanceInfo(InInstanceIndex), GetInstanceInfo(InOtherInstanceIndex));
	}

	void SetModifierResolver(TSharedPtr<MeshPartition::IModifierResolver> InModifierResolver) { this->ModifierResolver = InModifierResolver; }

	/** Returns true if the passed modifiers have a build dependency on each other. */
	static bool HasDependency(const MeshPartition::FModifierDesc& InModifier, const MeshPartition::FModifierDesc& InOtherModifier);
	bool HasDependency(const FInstanceInfo& InInstance, const FInstanceInfo& InOtherInstance) const;
	bool HasDependency(FInstanceIndex InInstanceIndex, FInstanceIndex InOtherInstanceIndex) const
	{
		return HasDependency(GetInstanceInfo(InInstanceIndex), GetInstanceInfo(InOtherInstanceIndex));
	}

	/** Returns true if the modifier descriptor list is correctly sorted by priority. Used for debug validation only. */
	bool ValidateIsSorted(TConstArrayView<FName> InLayerPriorities) const;

	FBox ComputeBaseBounds() const;
	double ComputeBaseComplexity() const;
	double ComputeTotalComplexity() const;

	void Add(const MeshPartition::FModifierDesc& InBaseDesc);
	void AddBase(const MeshPartition::FModifierDesc& InBaseDesc);
	void AddModifier(const MeshPartition::FModifierDesc& InModifierDesc);
	/** Inserts a new modifier to this group at the correct index considering the sort priority. */
	void AddModifierSorted(TConstArrayView<FName> InModifierTypePriorities, const MeshPartition::FModifierDesc& InModifierDesc);

	/** Sorts the modifiers in this group in sort priority. */
	void Sort(TConstArrayView<FName> InModifierTypePriorities);

	/** Removes all disabled modifiers from the group. */
	void RemoveDisabledModifiers();
	bool IsEmpty() const;

	const TOptional<MeshPartition::EBuildType>& GetBuildType() const { return BuildType; }
		
	void SetBuildType(MeshPartition::EBuildType InBuildType)
	{
		BuildType = InBuildType;
	}

	void ProgressToState(EState InTarget);

	/** Creates a copy of the build group for async processing. */
	MeshPartition::FModifierGroup CreateAsyncBuildGroup();

	const MeshPartition::FModifierDesc& GetModifierDesc(FInstanceIndex InInstanceIndex) const
	{
		return GetModifierDesc(GetInstanceInfo(InInstanceIndex));
	}

	const MeshPartition::FModifierDesc& GetModifierDesc(const FInstanceInfo& InInstance) const
	{
		return ModifierDescriptors[InInstance.ModifierIndex];
	}

	const MeshPartition::FModifierDesc& GetModifierDesc(FModifierIndex InIndex) const
	{
		return ModifierDescriptors[InIndex];
	}

	const FInstanceInfo& GetInstanceInfo(FInstanceIndex InIndex) const
	{
		checkf(bAsyncGroup, TEXT("Modifier instances are created on the async thread and only for the async copy of the group"));
		return InstanceInfos[InIndex];
	}

	FModifierIndex GetModifierIndex(FInstanceIndex InIndex) const
	{
		ensure(CurrentState >= EState::InstancesReady);
		checkf(bAsyncGroup, TEXT("Modifier instances are created on the async thread and only for the async copy of the group"));
		return GetInstanceInfo(InIndex).ModifierIndex;
	}
		
	TSharedPtr<const MeshPartition::IModifierBackgroundOp> GetModifierOp(FModifierIndex InIndex) const
	{
		ensure(CurrentState >= EState::BackgroundOpsCreated);
		return ModifierOps[InIndex];
	}

	TSharedPtr<const MeshPartition::IModifierBackgroundOp> GetModifierOp(FInstanceIndex InIndex) const
	{
		ensure(CurrentState >= EState::InstancesReady);
		return GetModifierOp(GetInstanceInfo(InIndex).ModifierIndex);
	}

	TWeakObjectPtr<MeshPartition::UModifierComponent> GetModifierPtr(FModifierIndex InIndex) const;

	FGuid GetModifierCacheKey(FModifierIndex InIndex) const
	{
		ensure(CurrentState >= EState::ModifiersResolved);
		ensure(ModifierCacheKeys[InIndex].IsValid());
		return ModifierCacheKeys[InIndex];
	}

	FGuid GetModifierCacheKey(FInstanceIndex InIndex) const
	{
		check(CurrentState >= EState::InstancesReady);
		return GetModifierCacheKey(GetInstanceInfo(InIndex).ModifierIndex);
	}
		
	FGuid ComputeBaseModifierSetHash() const;
	FGuid ComputeBaseCacheKey() const;
	FGuid ComputeModifierSetHash() const;
	FGuid UpdateAndComputeModifierGroupHash();

	FBlake3Hash ComputeGroupBuildHash() const;
		
	TIndexRange<FModifierIndex> AllModifierIndices() const
	{
		return TIndexRange<FModifierIndex>(0, ModifierDescriptors.Num());
	}
	TIndexRange<FModifierIndex> BaseIndices() const
	{
		return TIndexRange<FModifierIndex>(0, NumBases);
	}
	TIndexRange<FModifierIndex> ModifierIndices() const
	{
		return TIndexRange<FModifierIndex>(NumBases, ModifierDescriptors.Num());
	}
	TIndexRange<FInstanceIndex> InstanceIndices() const
	{
		check(bAsyncGroup);
		ensure(CurrentState >= EState::InstancesReady);
		return TIndexRange<FInstanceIndex>(0, InstanceInfos.Num());
	}

	TConstArrayView<MeshPartition::FModifierDesc> AllModifierDescs() const
	{
		return MakeConstArrayView(ModifierDescriptors);
	}
	TConstArrayView<MeshPartition::FModifierDesc> BaseDescs() const
	{
		return MakeConstArrayView(ModifierDescriptors.GetData(), NumBases);
	}
	TConstArrayView<MeshPartition::FModifierDesc> ModifierDescs() const
	{
		return MakeConstArrayView(ModifierDescriptors.GetData() + (NumBases), ModifierDescriptors.Num() - NumBases);
	}
	TConstArrayView<FInstanceInfo> Instances() const
	{
		check(bAsyncGroup);
		return MakeConstArrayView(InstanceInfos);
	}

	// @param bSkipInvalidModifiers If true, silently skip null modifiers. (Otherwise, we will still skip them but will trigger an ensure)
	void ForAllModifiers(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc, bool bSkipInvalidModifiers = false) const;

	void ForEachBase(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc) const;

	void ForEachModifier(TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc) const;

	UE_API TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> AllResolvedModifierPtrs() const;

private:
	void ForEachModifier_Internal(TConstArrayView<TSoftObjectPtr<MeshPartition::UModifierComponent>> InModifierPtrs, TFunctionRef<bool(MeshPartition::UModifierComponent*)> InFunc, bool bSkipInvalidModifiers = false) const;

	TConstArrayView<TSoftObjectPtr<MeshPartition::UModifierComponent>> AllModifierPtrs() const
	{
		check(IsInGameThread());
		ensure(CurrentState >= EState::ModifiersResolved);
		return MakeConstArrayView(ModifierPointers);
	}
	TConstArrayView<TSoftObjectPtr<MeshPartition::UModifierComponent>> BasePtrs() const
	{
		check(IsInGameThread());
		ensure(CurrentState >= EState::ModifiersResolved);
		return MakeConstArrayView(ModifierPointers.GetData(), NumBases);
	}
	TConstArrayView<TSoftObjectPtr<MeshPartition::UModifierComponent>> ModifierPtrs() const
	{
		check(IsInGameThread());
		ensure(CurrentState >= EState::ModifiersResolved);
		return MakeConstArrayView(ModifierPointers.GetData() + (NumBases), ModifierPointers.Num() - NumBases);
	}

	static int8 CompareModifierOrder(TConstArrayView<FName> InLayerPriorities, const MeshPartition::FModifierDesc& InModifier, const MeshPartition::FModifierDesc& InOtherModifier);

	/** Resolves the MeshPartition::UModifierComponent pointers from the soft object paths in the descriptors. */
	void ResolveModifierPtrs();
	/** Calls MeshPartition::UModifierComponent::PrepareResources on each modifier in the group. */
	void PrepareResources();
	/** Create the async background operators */
	void CreateBackgroundOps(MeshPartition::EBuildType InBuildType);
	/** Initializes all modifier instances for processing on the async thread by the task graph. */
	void InitInstances();

	/** Per Modifier Data Sets */
		
	/** Sorted lists of modifier data where bases appear first, followed by non-base modifiers */
	TArray<MeshPartition::FModifierDesc> ModifierDescriptors;
	/** List of soft object ptrs to loaded modifier objects. These are initialized after the group is finalized and map 1-1 with ModifierDescs. */
	TArray<TSoftObjectPtr<MeshPartition::UModifierComponent>> ModifierPointers;
	/** List of modifier cache keys. These are initialized after the group is finalized and map 1-1 with ModifierDescs. */
	TArray<FGuid> ModifierCacheKeys;
	/** List of modifier background operators. These are initialized after the group is finalized and map 1-1 with ModifierDescs. */
	TArray<TSharedPtr<const MeshPartition::IModifierBackgroundOp>> ModifierOps;

	/** Per Modifier Instance Data */
	TArray<FInstanceInfo> InstanceInfos;

	int32 NumBases = 0;

	/** Group is prepared for use on the async threads. It is no longer permitted to use it to reference GT objects. */
	bool bAsyncGroup = false;

	TSharedPtr<MeshPartition::IModifierResolver> ModifierResolver;

	EState CurrentState = EState::Uninitialized;
	TOptional<MeshPartition::EBuildType> BuildType;
};

UE_API void SortModifierDescriptors(TConstArrayView<const FName> InModifierTypePriorities, TArray<MeshPartition::FModifierDesc>& InOutModifierDescriptors);
TArray<MeshPartition::FModifierGroup> BuildModifierGroups(TConstArrayView<MeshPartition::FModifierDesc> InModifierDescriptors, const MeshPartition::FBuilderSettings& InSettings);
MeshPartition::FModifierGroup BuildModifierGroupForBase(TConstArrayView<MeshPartition::FModifierDesc> InBaseSet, TConstArrayView<MeshPartition::FModifierDesc> InAllNonBaseModifiersSorted);
} // namespace UE::MeshPartition

#undef UE_API