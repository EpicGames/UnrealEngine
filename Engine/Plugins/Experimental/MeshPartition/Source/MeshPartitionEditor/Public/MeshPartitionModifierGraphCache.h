// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionMeshData.h"
#include "Containers/LruCache.h"
#include "MeshPartitionMeshView.h"
#include "Async/Mutex.h"
#include "UObject/SoftObjectPath.h"

namespace UE::MeshPartition
{
class UModifierComponent;

class FModifierGraphCache
{
public:
	FModifierGraphCache();

	/**
	* Stores a new mesh result in the cache with the corresponding cache key.
	* @param InCacheKey The combined hash of all the base modifiers composing the merged base
	* @param InMesh The resulting intermediate mesh at the cached layer.
	*/
	void CacheMergedBaseMesh(const FGuid& InCacheKey, const FMeshData& InMesh);

	/**
	* Retrieves a copy of an entry of the cache, if it exists.
	* @param InCacheKey key created from hashing base modifier cache keys
	* @param OutCachedMesh If cache hit, result is copied to the out param.
	* @return True if a cache hit occured and cache entry was copied to the OutCachedMesh param.
	*/
	bool GetCachedMergedBaseMesh(const FGuid& InCacheKey, FMeshData& OutCachedMesh);

	/** Returns the current memory usage of the cache in MB. */
	double GetTotalMemoryUsageMB() const;

	/** Returns the current memory usage of the view cache in MB. */
	double GetViewCacheMemoryUsageMB() const;

	/** Returns the current memory usage of the view cache in MB. */
	double GetMergedBaseCacheMemoryUsageMB() const { return BaseMeshMemoryUsedMB; }

	/** Clears all cache entries and resets the cache. */
	void Clear();

	static bool IsCachingEnabled();

	struct FModifierCacheData
	{
		FGuid CacheKey;
	
		// Per instance local view hash.
		// Hash sequence of all the modifiers which are applied in the local region of this instance and before this instance.
		TArray<FGuid> LocalViewHashes;
	
		// Per instance cached mesh views
		TArray<MeshPartition::FMeshView> MeshViews;

		double GetMemoryUsageMB() const;
	};

	struct FBaseGroupViewCacheData
	{
		// Maps static modifier guids to their cache data
		TMap<FSoftObjectPath, FModifierCacheData> ModifierCacheData;
		
		FMutex Mutex;
	};

	/**
	* Copies all cache entries for the passed set of modifier guids in a given base set into the output cache data.
	* @param InBaseGroupCacheKey Hash of all the base modifier cache keys. Views are cached per modifier per base group, so the same
	*                            modifier may have a different view if it applies to multiple bases.
	* @param InModifierPaths The list of modifier paths whose cache entries should be retrieved.
	* @param OutbaseGroupCacheData Output parameter where the requested cache data will be stored.
	* @return Returns true if there is an existing cache entry for this base group.
	*/
	bool CopyBaseGroupViewCacheData(const FGuid& InBaseGroupKey, TConstArrayView<FSoftObjectPath> InModifierPaths, FBaseGroupViewCacheData& OutBaseGroupViewCacheData);

	/**
	* Writes the InBaseGroupCacheData into the persistent cache, moving the data out of the input parameter.
	* InBaseGroupCacheData may contain only a subset of all the modifiers which are affecting the cache.
	* In the case where local builders are executing, this may be the case.
	* @param InBaseGroupKey Hash of all base modifier guids.
	* @param InBaseGroupCacheData Set of modifier views to add to the cache for the base group. Data will be moved out of this parameter
	*                             so should no longer be referenced after calling this function.
	*/
	void CacheBaseGroupViewCacheData(const FGuid& InBaseGroupCacheKey, FBaseGroupViewCacheData&& InBaseGroupViewCacheData);

private:
	void CopyMesh(const FMeshData& InSourceMesh, FMeshData& OutMesh);

	void AddMergedBaseToCache_Unsafe(const FGuid& InCacheKey, const FMeshData& InMesh);

	/** Ensures that the view cache does not exceed the memory budget. Evicts LRU entries if memory budget is exceeded. */
	void EnsureViewMemoryBudget();

	double RemoveLeastRecentlyUsedMergedBase_Unsafe();
	double RemoveLeastRecentlyUsedViewCache_Unsafe();

	struct FMergedBaseCacheEntry
	{
		FMeshData MeshData;
	};

	TLruCache<FGuid, FMergedBaseCacheEntry> MergedBaseCache;

	/** Current memory used by the cache to store full merged base meshes. */
	double BaseMeshMemoryUsedMB = 0;

	/** Mutex to lock access to the LruCache */
	mutable FRWLock MergedBaseCacheMutex;

	/** Map of base groups to that group's cache data. */
	TMap<FGuid, TSharedPtr<FBaseGroupViewCacheData>> BaseViewCacheData;

	using FBaseGroupGuid = FGuid;

	/**
	* LruCache which stores the most recently used modifiers so we can evict modifiers which have not been used
	* across all base sections recently.
	*/
	TLruCache<FSoftObjectPath, TPair<FBaseGroupGuid, double>> LeastRecentlyUsedModifiers;

	/**
	* Locks the BaseViewCacheData and LRUModifier data. 
	* These two structures are almost always accessed together, so one mutex for both simplifies the code.
	*/
	mutable FMutex BaseViewCacheDataMutex;

	/** Mutex to lock access to the list of layers to cache. */
	mutable FRWLock LayersToCacheMutex;
};
} // namespace UE::MeshPartition