// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionEditorModule.h"
#include "Misc/ArchiveMD5.h"
#include "Tasks/Task.h"

struct FRichCurve;
class USplineComponent;
class UDynamicMesh;
class UStaticMesh;
class UTexture;
class UWorldPartition;
class FWorldPartitionActorDescInstance;

namespace UE::Geometry
{
	class FDynamicMesh3;
}
namespace UE::MeshPartition
{
	class UModifierComponent;
	struct FModifierDesc;
	struct FCompiledSectionDescriptor;
}

namespace UE::MeshPartition::Utils
{
	MESHPARTITIONEDITOR_API TArray<FBox> CollectBoundingBoxesForSpline(
		const USplineComponent* SplineComponent,
		TFunctionRef<float(float)> GetWidthForDistanceAlongSpline,
		TFunctionRef<float(float)> GetHeightForDistanceAlongSpline,
		float MaxSquaredDistance
	);

	enum class ERectangleFalloffMode
	{
		Linear,
		Smooth,
		CustomCurve
	};

	// Helper for storing data used in multiple GetRectangleFalloffAlpha calls.
	class FRectangleFalloffData
	{
	public:
		FRectangleFalloffData(
			const FVector2d& RectangleExtentIn, double CornerRadiusIn,
			double FalloffDistanceIn, ERectangleFalloffMode FalloffModeIn = ERectangleFalloffMode::Smooth,
			TSharedPtr<const FRichCurve> FalloffCurveIn = nullptr);

	private:
		FVector2d RectangleExtent;
		ERectangleFalloffMode FalloffMode = ERectangleFalloffMode::Smooth;
		double FalloffDistance = 0;
		TSharedPtr<const FRichCurve> FalloffCurve;
		double ClampedRadius = 0;
		double ClampedFalloff = 0;
		FVector2d CornerCenter = FVector2d::Zero();

		friend double GetRectangleFalloffAlpha(const FVector2d Local2DCoordinates, const FRectangleFalloffData& FalloffData);
	};
	double GetRectangleFalloffAlpha(const FVector2d Local2DCoordinates, const FRectangleFalloffData& FalloffData);

	// better deterministic hash combine, that mixes the data between the four DWORDs of the FGuid, instead of just hashing each independently
	FGuid BetterHashCombine(const FGuid& GuidA, const FGuid& GuidB);
	
	// Suggested hash method for computing modifier CacheKeys
	class FHashArchive : public FArchiveMD5
	{
	public:
		FHashArchive() {}

		void operator += (const UDynamicMesh& Mesh);
		void operator += (const UDynamicMesh* Mesh);

		void operator += (const UStaticMesh& Mesh);
		void operator += (const UStaticMesh* Mesh);

		void operator += (const UTexture& Texture);
		void operator += (const UTexture* Texture);

		// non-const pointers can just be serialized as const pointers
		// (without this definition, the const reference template below is invoked for const pointers)
		template<typename T>
		void operator += (T* Data)
		{
			*this += const_cast<const T*>(Data);
		}

		// use += operator to hash const types by making a non-const copy (basically a write-only serializer)
		template<typename T>
		void operator += (const T& Data)
		{
			T DataCopy = Data;
			*this << DataCopy;
		}
	};

	/**
	* Gathers descriptors for all of the mega mesh modifiers and compiled sections in the world partition
	* @param InWorldPartition The world partition to use to iterate actor descriptors.
	* @param PerModifierCallback the callback function that is invoked for each Modifier descriptor found.
	* @param PerSectionCallback the callback function that is invoked for each CompiledSection descriptor found.
	*/
	void ForEachMegaMeshDescInWorldPartition(UWorldPartition* InWorldPartition, TFunctionRef<void(MeshPartition::FModifierDesc& Descriptor)> PerModifierCallback, TFunctionRef<void(FCompiledSectionDescriptor& Descriptor)> PerSectionCallback);

	/**
	* Gathers descriptors for mega mesh modifiers from the ActorDescInstance
	* @param ActorDescInstance The world partition actor descriptor to gather modifier from.
	* @param PerModifierCallback the callback function that is invoked for each Modifier descriptor found.
	*/
	void ForEachModifierDescInActorDesc(const FWorldPartitionActorDescInstance* ActorDescInstance, TFunctionRef<void(MeshPartition::FModifierDesc& Descriptor)> PerModifierCallback);

	/**
	 * Returns true if the given index weight layer exists on the mesh, and has a nonzero value on at least one vertex.
	 */
	MESHPARTITIONEDITOR_API bool IsNonzeroWeightLayer(const UE::Geometry::FDynamicMesh3& Mesh, int32 WeightLayerIndex);

	/**
	 * Utility class to transform some Input data into an Output data asynchronously.
	 *
	 * TAsyncTransform will only hold on to the input data for as long as it is required to construct the output.
	 * Once the output is prepared, the input data will be released.
	 *
	 * TAsyncTransform and the input data can be destroyed without waiting for the operation to complete.
	 * 
	 * TAsyncTransform also provides utilities to chain multiple transforms together. Once the result of the prerequisite transform
	 * is ready it will chain into the subsequent. The subsequent output data will only be ready when both transforms are completed.
	 */
	template<typename OutputT>
	class TAsyncTransform
	{
	public:
		template<typename InputT, typename F>
		TAsyncTransform(TSharedRef<const InputT> InData, F&& AsyncInitFunc, UE::Tasks::FTask Prerequisite = {})
			: AsyncData(MakeShared<OutputT>())
			, AsyncInitTask(Tasks::Launch(TEXT("TAsyncInitData::AsyncInit"),
										[Data = MoveTemp(InData), AsyncData = AsyncData, AsyncInitFunc = MoveTemp(AsyncInitFunc)]() mutable
										{
											TUniqueFunction<OutputT(const InputT&)> AsyncInitFuncWrapper = MoveTemp(AsyncInitFunc);
											TRACE_CPUPROFILER_EVENT_SCOPE(TAsyncTransform::Init);
											*AsyncData = AsyncInitFuncWrapper(*Data);
										}, Prerequisite))
		{
		}

		template<typename InputT, typename F>
		TAsyncTransform(InputT InData, F&& AsyncInitFunc, UE::Tasks::FTask Prerequisite = {})
			: AsyncData(MakeShared<OutputT>())
			, AsyncInitTask(Tasks::Launch(TEXT("TAsyncInitData::AsyncInit"),
										[Data = MoveTemp(InData), AsyncData = AsyncData, AsyncInitFunc = MoveTemp(AsyncInitFunc)]() mutable
										{
											TUniqueFunction<OutputT(InputT)> AsyncInitFuncWrapper = MoveTemp(AsyncInitFunc);
											TRACE_CPUPROFILER_EVENT_SCOPE(TAsyncTransform::Init);
											*AsyncData = AsyncInitFuncWrapper(MoveTemp(Data));
										}, Prerequisite))
		{
		}

		const OutputT& GetResult() const
		{
			if (!AsyncInitTask.IsCompleted())
			{
				[[maybe_unused]] bool bIsCompleted = AsyncInitTask.Wait();
			}

			return *AsyncData;
		}
	
		bool IsCompleted() const
		{
			return AsyncInitTask.IsCompleted();
		}

		UE::Tasks::FTask GetAsyncInitTask() const
		{
			return AsyncInitTask;
		};

		template<typename ChainedOutputT>
		TAsyncTransform<ChainedOutputT> Chain(TUniqueFunction<ChainedOutputT(const OutputT&)>&& AsyncInitFunc) const
		{
			TSharedRef<const OutputT> ChainInput = AsyncData;
			return TAsyncTransform<ChainedOutputT>(ChainInput,
								MoveTemp(AsyncInitFunc),
								GetAsyncInitTask());
		}

	private:
		TSharedRef<OutputT> AsyncData;

		const Tasks::FTask AsyncInitTask;
	};

	/**
	 * TAsyncInitData expands on TAsyncTransform but also holds a persistent reference to the input data.
	 * Allowing it to be accessed even once the task has completed execution.
	 */
	template<typename ConstDataType, typename AsyncDataType>
	class TAsyncInitData : public TAsyncTransform<AsyncDataType>
	{
	public:
		using Super = TAsyncTransform<AsyncDataType>;
		using AsyncInitFuncType = TUniqueFunction<AsyncDataType(const ConstDataType&)>;

		TAsyncInitData(TSharedRef<const ConstDataType> InData, AsyncInitFuncType&& AsyncInitFunc, UE::Tasks::FTask Prerequisite = {})
			: Super(InData, MoveTemp(AsyncInitFunc), Prerequisite)
			, Data(InData)
		{
		}

		TAsyncInitData(TSharedRef<const ConstDataType> InData)
			: Super(InData)
			, Data(InData)
		{
		}

		const ConstDataType& GetImmutable() const
		{
			return *Data;
		}

		const AsyncDataType& GetAsyncData() const
		{
			return Super::GetResult();
		}

	private:
		TSharedRef<const ConstDataType> Data;
	};

	/** Utility which merges the join tasks of multiple separate init tasks into a single task event. */
	template<typename ...TasksTypes>
	UE::Tasks::FTask MakeJoinTask(TasksTypes&&... InitTasks)
	{
		TStaticArray<UE::Tasks::FTask, sizeof...(InitTasks)> Prereqs = {InitTasks.GetAsyncInitTask()...};
		return UE::Tasks::Launch(
			TEXT("TAsyncInitTask::Join"),
			[](){},
			Prereqs,
			UE::Tasks::ETaskPriority::Normal,
			UE::Tasks::EExtendedTaskPriority::TaskEvent
		);
	}
}