// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldPartition/WorldPartitionRuntimeCellTransformer.h"
#include "MeshPartitionPlatformCellTransformer.generated.h"

class UDataLayerAsset;

namespace UE::MeshPartition
{

class AMeshPartitionDataLayerContainer;

UCLASS(meta = (DisplayName = "Mesh Partition Platform Cell Transformer"))
class UPlatformCellTransformer : public UWorldPartitionRuntimeCellTransformer
{
	GENERATED_BODY()
public:
	virtual bool ShouldStripRuntimeCell(const UWorldPartitionRuntimeCell* InCell) const override;
	virtual void TransformRuntimeCell(UWorldPartitionRuntimeCell* InCell) override;
};
} // namespace UE::MeshPartition
