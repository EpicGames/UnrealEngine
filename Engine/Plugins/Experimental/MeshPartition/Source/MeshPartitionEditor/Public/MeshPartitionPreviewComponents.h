// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/MeshComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "Components/StaticMeshComponent.h"
#include "ComponentVisualizer.h"
#include "MeshPartitionStaticMeshComponent.h"
#include "MeshPartitionMeshData.h"

#include "MeshPartitionPreviewComponents.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{
class FMegaMeshCustomPreviewSceneProxy;

/**
* Subclass of static mesh component in order to customize the hit proxy, which needs to be custom to allow
*  our visualizer to customize click handling for preview components (to route selection to base sections).
*/
UCLASS(MinimalAPI)
class UStaticMeshPreviewComponent : public UMeshPartitionStaticMeshComponent
{
	GENERATED_BODY()

public:
	// Static mesh scene proxy calls into this method to set up hit proxies for different sections (note:
	//  there is still also a hitproxy that points to the preview component itself, but that doesn't seem
	//  to get in the way).
	virtual HHitProxy* CreateMeshHitProxy(int32 SectionIndex, int32 MaterialIndex) const override;
};


UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent), ClassGroup = Rendering)
class UPreviewMeshComponent : public UMeshComponent, public IInterface_CollisionDataProvider
{
	GENERATED_BODY()
public:

	UPreviewMeshComponent();
	
	virtual void GetResourceSizeEx(FResourceSizeEx& Size) override;

	UE_API void SetMeshData(TSharedRef<const MeshPartition::FMeshData> InMeshData);
	void SetMeshData(MeshPartition::FMeshData&& InMeshData);

	TSharedPtr<const FMeshData> GetMeshData() { return MeshData; }

	/** Returns the custom preview scene proxy if one has been created, otherwise nullptr. */
	UE_API FMegaMeshCustomPreviewSceneProxy* GetCustomSceneProxy() const;

	virtual int32 GetNumMaterials() const override { return 1; }
	virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const override;
	virtual UBodySetup* GetBodySetup() override;

	virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

	virtual void OnRegister() override;
	virtual void OnUnregister() override;

	//~ Begin Interface_CollisionDataProvider Interface
	virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	virtual void GetMeshId(FString& OutMeshId) override;
	//~ End Interface_CollisionDataProvider Interface

private:
	virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

	void RebuildPhysicsData();
	UBodySetup* CreateBodySetupHelper();

	void AbortPendingPhysicsCook();
	void FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup);

public:
	UPROPERTY(Transient, Experimental)
	TArray<TObjectPtr<class UMaterialCacheVirtualTexture>> MaterialCacheTextures;

private:
	UPROPERTY(Transient, Instanced)
	TObjectPtr<UBodySetup> MeshBodySetup;

	UPROPERTY(Transient, Instanced)
	TObjectPtr<UBodySetup> AsyncBodySetup;  //Body setup being cooked asynchronously

	TSharedPtr<const FMeshData> MeshData;
};

/**
* Simple custom hitproxy similar to HActor, but inheriting from HComponentVisProxy to allow
*  our visualizers to customize the click handling inside FComponentVisualizerManager::HandleProxyForComponentVis
*/
struct HPreviewProxy : public HComponentVisProxy
{
	DECLARE_HIT_PROXY();

	HPreviewProxy(const UActorComponent* InComponent);

	TWeakObjectPtr<const AActor> Actor;

	virtual FTypedElementHandle GetElementHandle() const override
	{
		// The base class seems to crash in the element acquisition if the component is destroyed,
		//  which seems to be possible.
		return Component.IsValid() ? HComponentVisProxy::GetElementHandle()
			: FTypedElementHandle();
	}
};

} // namespace UE::MeshPartition

#undef UE_API