// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "GameFramework/Actor.h"
#include "MeshPartitionMeshData.h"
#include "MeshPartitionBuildPerfStats.h"
#include "MeshPartitionCollisionComponent.h"
#include "MaterialCache/MaterialCacheVirtualTexture.h"

#include "MeshPartitionPreviewSection.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

class UStaticMeshComponent;
class UTexture;
class UMaterialInstanceDynamic;

namespace UE::MeshPartition
{
class AMeshPartition;
class UModifierComponent;
class UMeshPartitionEditorComponent;
class UMeshPartitionDefinition;
class UPreviewMeshComponent;
struct FCommonBuildVariant;
struct FStaticMeshDescriptor;

/**
* This is a transient actor, expressing a “compiled” part of a MegaMesh.
* This means a region of the MegaMesh where all processing has been performed to produce the final result.
* It only exists in the editor.
*/
UCLASS(MinimalAPI, Transient)
class APreviewSection : public AActor
{
	GENERATED_BODY()
	
public:	
	UE_API APreviewSection();

	// UObject Implementation
	UE_API virtual void BeginDestroy() override;
	// End UObject Implementation

	// AActor Implementation
	UE_API virtual void PostRegisterAllComponents() override;
	UE_API virtual void PostUnregisterAllComponents() override;
	// End AActor Implementation

	const TArray<TObjectPtr<UStaticMeshComponent>>& GetMeshComponents() const { return MeshComponents; }

	UE_API void AddMesh(UStaticMesh* InStaticMesh, const FStaticMeshDescriptor& InDescriptor);
	UE_API void SetFarFieldMesh(UStaticMesh* InStaticMesh);
	UE_API void AddCollisionComponent(UMeshPartitionCollisionComponent* InCollisionComponent);
	UE_API TArray<TObjectPtr<UStaticMesh>> GetMeshes() const;
	UE_API TSharedPtr<const MeshPartition::FMeshData> GetPreviewMesh() const;
	UPreviewMeshComponent* GetPreviewMeshComponent() const { return PreviewMeshComponent; }
	UE_API void SetPreviewMesh(TSharedRef<const MeshPartition::FMeshData> InMeshData, const FName& InCollisionProfileName, const bool bInCanEverAffectNavigation);

	UE_API AMeshPartition* GetParent() const;
	UE_API void SetParent(AMeshPartition* InMegaMesh);

	void SetMaterialInstance(UMaterialInstanceDynamic* InMaterialInstance);
	UMaterialInstanceDynamic* GetMaterialInstance() const { return MaterialInstance; }

	UE_API void SetChannelData(TConstArrayView<uint8> InChannelTable, const FVector2f& InTexcoordDesc);
	const TArray<uint8>& GetChannelTable() const { return ChannelTable; }
	UE_API void SetChannelTexture(UTexture* InChannelTexture);
	UTexture* GetChannelTexture() const { return ChannelTexture; }
	
	void SetChannelGenerationMesh(const MeshPartition::FSectionChannelGenerationMeshData& InSection) { SectionChannelMesh = InSection; }
	const MeshPartition::FSectionChannelGenerationMeshData& GetChannelGenerationMesh() const { return SectionChannelMesh; }

	UE_API void SetBuildPerfStats(MeshPartition::FBuildPerfStats&& InBuildPerfStats);
	UE_API const MeshPartition::FBuildPerfStats& GetBuildPerfStats() const;

	void SetIsTransformerPipelinePaused(const bool bInIsPaused) { bIsTransformerPipelinePaused = bInIsPaused; }
	bool IsTransformerPipelinedPaused() const { return bIsTransformerPipelinePaused;	}
	
	UE_API void OnMeshPartitionDefinitionChanged(const UMeshPartitionDefinition* InDefinition);
	UE_API void OnMeshPartitionDefinitionModified(const UMeshPartitionDefinition* InDefinition, const FName& InPropertyName);
	
	UE_API void SetMaterialCacheTileCount(const FIntPoint& TileCount);
	UE_API void RecreateMaterialCacheTextures();

	const TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>>& GetBaseModifiers() const { return BaseModifiers; }
	UE_API void AddBaseModifier(const TWeakObjectPtr<MeshPartition::UModifierComponent>& InBaseModifier);
	UE_API void RemoveBaseModifier(const TWeakObjectPtr<MeshPartition::UModifierComponent>& InBaseModifier);

	const TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>>& GetModifiers() const { return Modifiers; }
	UE_API void AddModifier(const TWeakObjectPtr<MeshPartition::UModifierComponent>& InModifier);

	FBox GetPreviewMeshBounds() const { return PreviewMeshBounds; }

	UE_API UMeshPartitionEditorComponent* GetMegaMeshEditorComponent() const;

	UE_API void SetRuntimeVirtualTextures(const TArray<TObjectPtr<URuntimeVirtualTexture>>& InRVTs);

	void SetGroupRegistryKey(const FGuid& InKey) { GroupRegistryKey = InKey; }
	FGuid GetGroupRegistryKey() const { return GroupRegistryKey; }

	UE_API void ForAllPrimitiveComponents(TFunctionRef<bool(UPrimitiveComponent*)> InFunc) const;
private:
	UE_API void OnStaticMeshBuild(UStaticMesh* InStaticMesh);
	UE_API void OnFarFieldStaticMeshBuild(UStaticMesh* InStaticMesh);
	UE_API void InvalidateRenderStates();

	UE_API UMaterialInterface* GetPreviewMaterial() const;

private:
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> BaseModifiers;

	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<MeshPartition::UModifierComponent>> Modifiers;

	UPROPERTY(Transient)
	TObjectPtr<UPreviewMeshComponent> PreviewMeshComponent;

	/** Fallback components for virtual texture rendering, this is a stop-gap solution, will be removed later */
	UPROPERTY(Transient)
	TArray<TObjectPtr<UStaticMeshComponent>> VirtualTextureFallbackMeshComponents;

	UPROPERTY(Transient)
	bool bIsTransformerPipelinePaused = false;
	
	UPROPERTY()
	TArray<TObjectPtr<UStaticMeshComponent>> MeshComponents;

	UPROPERTY()
	TArray<TObjectPtr<UMeshPartitionCollisionComponent>> CollisionComponents;

	// #todo: it would be better for the component to be visible so it shows up in the details panel.
	// However, currently the details panel will block the static mesh compilation in order to display the details.
	UPROPERTY()
	TObjectPtr<UStaticMeshComponent> FarFieldMeshComponent;

	UPROPERTY()
	TSoftObjectPtr<AMeshPartition> Parent;

	UPROPERTY(EditAnywhere, Category = "MeshPartition")
	TObjectPtr<UMaterialInstanceDynamic> MaterialInstance;

	UPROPERTY(EditAnywhere, Category = "MeshPartition")
	TObjectPtr<UTexture> ChannelTexture;

	UPROPERTY()
	FVector2f ChannelTexcoordDesc;
	
	UPROPERTY()
	TArray<uint8> ChannelTable;
	
	UPROPERTY()
	FIntPoint MaterialCacheTileCount = FIntPoint(1, 1);

	/** All allocated material cache textures */
	UPROPERTY(Transient, Experimental)
	TArray<TObjectPtr<UMaterialCacheVirtualTexture>> MaterialCacheTextures;

	/**
	* Cached preview mesh bounds.
	* When the static mesh build finishes, the DynamicMeshComponent no longer contains valid bounds data
	* and the component bounds will simply return a very small box (note: not empty/invalid) around the origin.
	* Conversely, while the static mesh build is processing, the StaticMeshComponent will return the same small bounds
	* around the origin. Rather than relying on either of them, we simply cache the bounds of the mesh when it is assigned to the preview section.
	*/
	UPROPERTY()
	FBox PreviewMeshBounds;

	UPROPERTY(VisibleAnywhere, Category = MeshPartition)
	MeshPartition::FBuildPerfStats BuildPerfStats;

	FGuid GroupRegistryKey;

	
	FSectionChannelGenerationMeshData SectionChannelMesh;

	friend class FMeshPreviewVisualizer;
};
} // namespace UE::MeshPartition

#undef UE_API
