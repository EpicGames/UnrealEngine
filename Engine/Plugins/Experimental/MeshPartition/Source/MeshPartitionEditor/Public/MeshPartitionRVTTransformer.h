// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MeshPartitionTransformer.h"

#include "MeshPartitionRVTTransformer.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class URuntimeVirtualTexture;

namespace UE::MeshPartition
{

/**
 * Mesh Partition Transformer binding the provided RVT to the sections present in the context.
 * Will only apply the transformation once per section.
 */
USTRUCT(MinimalAPI)
struct FRVTTransformer : public MeshPartition::FTransformer
{
	GENERATED_BODY()

public:
	UE_API virtual bool Execute(MeshPartition::UPreviewMeshComponent& InPreviewMeshComponent, const MeshPartition::UMeshPartitionDefinition* InDefinition, const MeshPartition::FCommonBuildVariant& InBuildVariant) const;

	UE_API virtual bool Execute(MeshPartition::FTransformerContext& InTransformerContext) const override;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const override;

private:
	void SetRuntimeVirtualTextures(const MeshPartition::FTransformerContext& InTransformerContext) const;

private:
	UPROPERTY(EditAnywhere, Category = "Material|RVT")
	TArray<TObjectPtr<URuntimeVirtualTexture>> RuntimeVirtualTextures;
};

} // namespace UE::MeshPartition

#undef UE_API
