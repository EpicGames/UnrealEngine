// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "Misc/SlowTask.h"
#include "Tasks/Task.h"

#define UE_API MESHPARTITIONEDITOR_API


class AActor;
class ALocationVolume;
class UWorldPartition;
class FPackageSourceControlHelper;

namespace UE::MeshPartition
{
class AMeshPartition;
class UMeshProviderModifier;
class UMeshPartitionEditorComponent;

struct FRectangleGeneratorUtils
{
	struct FSectionInfo
	{
		FInt32Point IndexXY;
		FInt32Point Resolution;
		FVector2d MinUV;
		FVector2d MaxUV;
	};

	/**
	* Compute the canonical section for the spatial decomposition of the given mesh resolution (in quads).
	* A resolution of zero will be returned if any of the inputs are .leq. to zero.
	* Note: when the mesh is decomposed into sections, it may require some smaller sections to complete the 
	* decomposition (see comment in ComputeSectionInfos )
	*/
	static UE_API FInt32Point ComputeSectionResolution(const FInt32Point MeshResolution, const int32 MaxTrianglesPerSection);

	/**
	* Compute the partitions of the the mesh based on the provided mesh resolution and optimal section resolution.
	* Note, in the case that the mesh resolution is not a multiple of the section resolution, the remainder sections will be smaller.
	* @return Array of FSectionInfo
	*/
	static UE_API TArray<FSectionInfo> ComputeSectionInfos(const FInt32Point MeshResolution, const FInt32Point SectionsResolution);

	/**
	* Spawn in the world (and optionally register) base modifiers associated with the MegaMesh by means of the provided MegaMeshEditorComponent
	* @return Array of pointers to the MegaMesh MeshProviders corresponding to the base modifiers.
	*/
	static UE_API TArray<MeshPartition::UMeshProviderModifier*> SpawnBaseModifiers(const TArray<FSectionInfo>& SectionInfos, const FVector& MeshSize, const FVector2d& MeshWorldOffset, const FTransform& ToWorld, const bool bRegisterModifers,
		UWorldPartition* WorldPartition, UMeshPartitionEditorComponent* MegaMeshEditorComponent, FSlowTask* SlowTask = nullptr);
	/**
	* Spawn location volumes in the world and attach them to the MegaMesh.  To be used by world partition.
	*/
	static UE_API void SpawnLocationVolumes(const FVector& MeshSize, const FVector2d& MeshWorldOffset, const FInt32Point& LocationVolumesResolution, UWorld* const World, AMeshPartition* const MegaMesh, FSlowTask* SlowTask = nullptr);

	static UE_API ALocationVolume* CreateLocationVolume(UWorld* const World, AMeshPartition* const MegaMesh, const FString& Label, const FBox& Bounds);

	/**
	* Create a height field rectangular mesh with the size and UVs specified by SectionInfo
	* If GetHeight references an unset TFunction, the mesh will be planar.
	*/
	static UE_API void GenerateSectionMesh(Geometry::FDynamicMesh3& Mesh, const FSectionInfo& SectionInfo, const FVector& MeshSize, TFunction<double(FVector2f) >& GetHeight);

	
};

// This base class exists to share functionality related to actor unloading and packages 
// when generating a multi-section Mega Mesh.   It is not intended to be used independently. 
class FMegaMeshGeneratorBase
{
public:
	FMegaMeshGeneratorBase() = default;


	const FText& GetErrorText() const { return ErrorText; }

	/** Returns the associated mega mesh actor */
	AMeshPartition* GetMegaMesh() const { return MegaMesh; }


protected:

	UE_API void SaveActorPackages(const TArray<TWeakObjectPtr<AActor>>& InActorsToSave);

	/** Cancel and block until all outstanding tasks are finished executing. */
	UE_API void CancelAndWait(UWorld* World);

	UE_API void UnloadActors(const TArray<TWeakObjectPtr<AActor>>& InActorsToUnload, UWorld* World) const;


	AMeshPartition* MegaMesh = nullptr;

	FText ErrorText;

	TArray<TPair<int, Tasks::FTask>> MeshTasks;
	TArray<MeshPartition::UMeshProviderModifier*> MegaMeshMeshProviders;

	// This class operates on isolated FDynamicMeshes on parallel threads and are then moved onto the output mesh provider on the GT
	// This ensures the correct events/callbacks are fired for the DynamicMeshComponent to ensure bounds are computed, proxies recreated, etc.
	TArray<Geometry::FDynamicMesh3> Meshes;

	// Retain a list of all packages that were saved and unloaded in case the user cancels so they can be reverted.
	TArray<FString> SavedPackages;

	bool bIsCancelled = false;
};


/**
* List of parameters for the MegaMesh Rectangle Generator.
*/
struct FMegaMeshRectangleGeneratorParams
{
	/** Number of vertices in X/Y dimension for the generated mesh. */
	FInt32Point MeshResolution;

	/** World space size of the generated mesh. */
	FVector2D MeshSize;

	/** Number of sections in X/Y dimension. */
	FInt32Point SectionsResolution;

	/** Save and unload sections as they are created. */
	bool bSaveAndUnload;

	/** Number of location volumes in X/Y dimension. */
	FInt32Point LocationVolumesResolution;
};


// Class that generates a rectangular MegaMesh that has been partitioned into multiple underlying DynamicMeshes
class FMegaMeshRectangleGenerator : public FMegaMeshGeneratorBase
{
public:
	FMegaMeshRectangleGenerator(const FMegaMeshRectangleGeneratorParams& InParams, UWorld& InWorld, FTransform InToWorld, AMeshPartition* InMegaMesh = nullptr)
	: World(&InWorld)
	, Params(InParams)
	, ToWorld(InToWorld)
	, ExistingMegaMesh(InMegaMesh)
	{}


	/**
	* Execute the Generate operation. If an FPackageSourceControlHelper is provided, newly created packages will also be added to source control.
	* Returns true if the operation succeeded, false otherwise.
	* Note: On failure, diagnostic error message can be retrieved with `GetErrorText`.
	*/
	UE_API bool Generate(FPackageSourceControlHelper* InSourceControlHelper);

protected:
	
	UE_API FMegaMeshRectangleGenerator();

	using FSectionInfo = MeshPartition::FRectangleGeneratorUtils::FSectionInfo;
	
	UE_API Tasks::FTask LaunchBuildSectionTask(const FSectionInfo& SectionInfo, Geometry::FDynamicMesh3& Mesh);

	UWorld* World = nullptr;

	FMegaMeshRectangleGeneratorParams Params;

	// controls the actual placement of the MegaMeshRetangle
	FTransform ToWorld;
	
	// if null, a new megamesh will be generated
	AMeshPartition* ExistingMegaMesh = nullptr;
};
} // namespace UE::MeshPartition

#undef UE_API
