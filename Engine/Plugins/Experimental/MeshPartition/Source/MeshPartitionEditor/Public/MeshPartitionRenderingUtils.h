// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "RenderCaptureInterface.h"

namespace UE::MeshPartition
{

class FRenderCaptureManager
{
	int NextCaptureCounter = 0;
	TUniquePtr<RenderCaptureInterface::FScopedCapture> CurrentCapture;
public:
	static FRenderCaptureManager& Get();

	static void CaptureNextBatch();
	static void	BeginCapture(FRDGBuilder& InGraphBuilder);
	static void EndCapture();
};

} // namespace UE::MeshPartition
