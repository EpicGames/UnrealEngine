// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "WorldPartition/WorldPartitionRuntimeCellTransformer.h"
#include "MeshPartitionRuntimeCellTransformer.generated.h"

class IAssetRegistry;

namespace UE::MeshPartition
{
struct FMeshPartitionWorldUpdater;

UCLASS()
class URuntimeCellTransformer : public UWorldPartitionRuntimeCellTransformer
{
	GENERATED_UCLASS_BODY()

#if WITH_EDITOR
public:
	virtual void PreTransform(ULevel* InLevel) override;
	virtual void Transform(ULevel* InLevel) override;
	virtual void PostTransform(ULevel* InLevel) override;
#endif

#if WITH_EDITORONLY_DATA
private:

	TSharedPtr<FMeshPartitionWorldUpdater> WorldUpdater;

	int32 CellsPreTransformed = 0;
	int32 CellsTransformed = 0;
	int32 CellsPostTransformed = 0;
#endif
};
} // namespace UE::MeshPartition