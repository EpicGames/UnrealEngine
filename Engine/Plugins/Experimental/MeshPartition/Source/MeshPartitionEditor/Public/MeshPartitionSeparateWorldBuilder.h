// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionMeshBuilderCommon.h"
#include "MeshPartitionSeparateWorldBuilder.generated.h"

namespace UE::MeshPartition
{
class ACompiledSection;
struct FModifierGroupBuildCache;

USTRUCT()
struct FSeparateWorldBuilder
{
	GENERATED_BODY()

	~FSeparateWorldBuilder();

	// Add a compiled section build request to the queue
	void RequestCompiledSectionBuild(MeshPartition::ACompiledSection* InCompiledSection);

	// Execute queued compiled section builds
	void Tick();

	// Clears all pending compiled section builds requests.
	void ClearRequests();

private:
	// RAII guard for the BuilderWorld lifecycle. Constructs the BuilderWorld + package
	// on entry; on exit, clears GroupBuildCache->Entries (cached modifier actor refs)
	// before tearing the world down so cache entries never outlive the world they
	// live in. Stack-only, used by Tick().
	class FBuilderWorldScope
	{
	public:
		FBuilderWorldScope(FSeparateWorldBuilder& InOwner, const FString& InWorldPackageName);
		~FBuilderWorldScope();

		UE_NONCOPYABLE(FBuilderWorldScope)

	private:
		FSeparateWorldBuilder& Owner;
	};

private:
	void BuildCompiledSectionMesh(MeshPartition::ACompiledSection* InCompiledSection, FName InBuildVariantName);

private:
	// Current package used as a container for the separate world
	UPROPERTY()
	TObjectPtr<UPackage> BuilderPackage;

	UPROPERTY()
	TObjectPtr<UWorld> BuilderWorld;

	UPROPERTY()
	TArray<TWeakObjectPtr<MeshPartition::ACompiledSection>> CompiledSectionsToBuild;

	// other state that doesn't hold GC refs
	TArray<MeshPartition::FBuildTaskHandle> TaskHandles;
	TArray<FGuid> AllMegaMeshGuidsInWorld;

	// Per-group build cache for grid-split sections (avoids N full mesh rebuilds per group).
	// Lifetime is enforced by FBuilderWorldScope.
	TSharedPtr<FModifierGroupBuildCache> GroupBuildCache;
};
} // namespace UE::MeshPartition