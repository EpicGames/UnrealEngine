// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/StaticMesh.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionMeshSkirt.h"
#include "MeshPartitionTransformer.h"
#include "Engine/CollisionProfile.h"

#include "MeshPartitionStaticMeshTransformer.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UStaticMesh;

namespace UE::MeshPartition
{

/**
 * Selects which simplification inputs the user authors directly and which are derived.
 * Driving values vs. derived values are paired so the UI can grey out the dependent fields.
 */
UENUM()
enum class EMeshPartitionSimplificationMode : uint8
{
	/** ErrorTolerance drives simplification; ScreenSize is derived from the achieved deviation and PixelError. */
	AutoScreenSizeFromError UMETA(DisplayName = "Derive ScreenSize from Error"),

	/** ScreenSize drives simplification; ErrorTolerance is derived from ScreenSize and PixelError. */
	AutoErrorToleranceFromScreenSize UMETA(DisplayName = "Derive Target ErrorTolerance from ScreenSize"),

	/** ScreenSize drives simplification via MaxTrianglesFraction = ScreenSize; ErrorTolerance is unused. */
	TriangleCountFromScreenSize UMETA(DisplayName = "TriangleCount from ScreenSize"),

	/** All fields authored manually; no field is derived. */
	Custom UMETA(DisplayName = "Custom"),
};

USTRUCT()
struct FMeshPartitionTransformerSimplificationSettings
{
	GENERATED_BODY()

	/**
	 * Selects which inputs are authored manually and which are derived. See EMeshPartitionSimplificationMode.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification")
	EMeshPartitionSimplificationMode Mode = EMeshPartitionSimplificationMode::Custom;

	/**
	 * If positive, acceptable geometric deviation from the input surface, otherwise unused.
	 * Will not decimate further than specified MinTrianglesFraction, and will always decimate
	 * at least to MaxTrianglesFraction (regardless of error tolerance). This is the recommended
	 * way of controlling detail.
	 *
	 * Derived in AutoErrorToleranceFromScreenSize mode. Unused in TriangleCountFromScreenSize mode.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification", meta = (EditCondition = "Mode == EMeshPartitionSimplificationMode::AutoScreenSizeFromError || Mode == EMeshPartitionSimplificationMode::Custom"))
	float ErrorTolerance = 0.f;

	/**
	 * If positive, maximum edge length to simplify up to. 
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification")
	float EdgeLength = 0.f;

	/**
	 * Keep at least this fraction of the input triangles. Only applies when using error tolerance criterion.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "ErrorTolerance > 0 || Mode == EMeshPartitionSimplificationMode::AutoErrorToleranceFromScreenSize"))
	float MinTrianglesFraction = 0.01f;

	/**
	 * Maximum fraction of triangles to retain. When ErrorTolerance > 0, this is a cap: the simplifier
	 * will always decimate at least to this fraction of the input, even if the error tolerance would
	 * otherwise prevent further collapses. When ErrorTolerance == 0, this becomes the target retention
	 * fraction (the simplifier reduces straight down to this triangle count). Default 1 = no reduction
	 * from this criterion.
	 *
	 * Derived in TriangleCountFromScreenSize mode (=ScreenSize).
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "Mode != EMeshPartitionSimplificationMode::TriangleCountFromScreenSize"))
	float MaxTrianglesFraction = 1.0f;

	/**
	 * Scaling term to re-balance the geometric and attribute layer influence of the simplifier.
	 * This value should reflect the relative scale of feature size of terrain vs standard assets.
	 * Increasing this value boosts the preservation of attributes (normals, tangents, UVs), but
	 * increasing it excessively will degrade geometric fidelity.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification")
	float ScaleCorrection = 100.0f;

	/**
	 * Regularization weight that can help to produce higher intrinsic quality in flat areas.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification")
	float MeshRegularization = 0.0005f;

	/**
	 * Screen size at which this LOD becomes active, expressed as the projected diameter of the
	 * mesh's bounding sphere relative to the viewport (0..1). The runtime selects the coarsest
	 * LOD whose ScreenSize still exceeds the current on-screen diameter, so this should
	 * monotonically decrease across LODs.
	 *
	 * Leave at 0 to fall back to the geometric default Pow(0.75, LODIndex) for that slot.
	 *
	 * Derived in AutoScreenSizeFromError mode.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification", meta = (ClampMin = "0.0", ClampMax = "1.0", UIMin = "0.0", UIMax = "1.0", EditCondition = "Mode != EMeshPartitionSimplificationMode::AutoScreenSizeFromError"))
	float ScreenSize = 0.0f;

	/**
	 * Pixel error budget used to convert between ScreenSize and geometric deviation: allow up to this
	 * many pixels of geometric deviation (at 1920x1080, 90-degree HFOV) for this LOD. Higher values
	 * let the LOD activate closer to the camera. Unused in TriangleCountFromScreenSize mode.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification", meta = (ClampMin = "0.0", UIMin = "0.0", EditCondition = "Mode != EMeshPartitionSimplificationMode::TriangleCountFromScreenSize"))
	float PixelError = 8.0f;

	/**
	 * Influence of per-corner normals on the attribute-aware quadric. Larger values preserve normal
	 * fidelity at the expense of geometry; 0 disables the normal term entirely.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification|Advanced", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float NormalAttributeWeight = 16.0f;

	/**
	 * Influence of tangents/bitangents on the attribute-aware quadric.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification|Advanced", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float TangentAttributeWeight = 0.1f;

	/**
	 * Influence of vertex colors on the attribute-aware quadric.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification|Advanced", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float ColorAttributeWeight = 0.1f;

	/**
	 * Influence of UVs on the attribute-aware quadric.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification|Advanced", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float TexCoordAttributeWeight = 0.5f;

	/**
	 * Influence of weight-map layers on the attribute-aware quadric. 0 disables weight layers
	 * (matching the library default); the historical transformer default of 0.2 enables them.
	 */
	UPROPERTY(EditAnywhere, Category = "Simplification|Advanced", meta = (ClampMin = "0.0", UIMin = "0.0"))
	float WeightLayerWeight = 0.2f;
};

USTRUCT()
struct FLODSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category = "LOD")
	bool bReduceMesh = false;

	UPROPERTY(EditAnywhere, Category = "LOD", meta = (EditCondition = "bReduceMesh"))
	FMeshPartitionTransformerSimplificationSettings SimplifierSettings;
};

/**
 * Mesh Partition Transformer producing and attaching a static mesh component for each transformer unit present in the execution context.
 */
USTRUCT(MinimalAPI)
struct FStaticMeshTransformer : public MeshPartition::FTransformer
{
	GENERATED_BODY()

public:
	UE_API virtual bool Execute(MeshPartition::FTransformerContext& InTransformerContext) const override;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const override;

private:
	TArray<UStaticMesh*> CreateStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext) const;

	void BuildStaticMesh(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask, const int32 InTransformerUnitIndex) const;

	void FinalizeStaticMeshes(MeshPartition::FTransformerContext& InTransformerContext, UE::Tasks::TTask<TArray<UStaticMesh*>>& InCreateStaticMeshesTask) const;

private:
	UPROPERTY(EditAnywhere, Category = "Collision")
	FCollisionProfileName CollisionProfile = UCollisionProfile::NoCollision_ProfileName;

	UPROPERTY(EditAnywhere, Category = "Static Mesh")
	TArray<MeshPartition::FLODSettings> LODs;

	UPROPERTY(EditAnywhere, Category = "Static Mesh")
	EStaticMeshLODStreaming LODStreaming = EStaticMeshLODStreaming::Default;

	UPROPERTY(EditAnywhere, Category = "Static Mesh")
	FPerPlatformInt NumStreamedLODs;

	UPROPERTY(EditAnywhere, Category = "Static Mesh")
	bool bCanEverAffectNavigation = false;

	UPROPERTY(EditAnywhere, Category = "Static Mesh|Skirt")
	bool bApplyMeshSkirt = false;

	UPROPERTY(EditAnywhere, Category = "Static Mesh|Skirt", meta = (EditCondition = "bApplyMeshSkirt"))
	FMeshSkirtSettings MeshSkirtSettings;

	UPROPERTY(EditAnywhere, Category = "Nanite")
	bool bUseNanite = true;

	UPROPERTY(EditAnywhere, Category = "Nanite", meta = (EditCondition = "bUseNanite", DisplayName = "Generate Fallback Mesh"))
	ENaniteGenerateFallback NaniteFallbackMode = ENaniteGenerateFallback::Enabled;

	UPROPERTY(EditAnywhere, Category = "Nanite", meta = (EditCondition = "bUseNanite"))
	ENaniteFallbackTarget NaniteFallbackTarget = ENaniteFallbackTarget::PercentTriangles;

	UPROPERTY(EditAnywhere, Category = "Nanite", meta = (EditCondition = "bUseNanite && NaniteFallbackTarget == ENaniteFallbackTarget::PercentTriangles", ClampMin = "0.0", ClampMax = "1.0"))
	float NaniteFallbackPercentTriangles = 0.2f;

	UPROPERTY(EditAnywhere, Category = "Nanite", meta = (EditCondition = "bUseNanite && NaniteFallbackTarget == ENaniteFallbackTarget::RelativeError", ClampMin = "0.0"))
	float NaniteFallbackRelativeError = 1.0f;
};

} // namespace UE::MeshPartition

#undef UE_API
