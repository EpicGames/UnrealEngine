// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MeshPartitionTransformer.h"

#include "MeshPartitionSubsectionTransformer.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{

/**
 * Mesh Partition Transformer producing multiple transformer units by splitting the meshdata contained in the pipeline transformer units.
 * Effectively acts as a demultiplexer.
 */
USTRUCT(MinimalAPI)
struct FSubsectionTransformer : public MeshPartition::FTransformer
{
	GENERATED_BODY()

public:
	UE_API virtual bool Execute(MeshPartition::FTransformerContext& InTransformerContext) const override;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const override;

private:
	UPROPERTY(EditAnywhere, Category="Subsection")
	uint32 SubSectionSize = 12800;
};

} // namespace UE::MeshPartition

#undef UE_API
