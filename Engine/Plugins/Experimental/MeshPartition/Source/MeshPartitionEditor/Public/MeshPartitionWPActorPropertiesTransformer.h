// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MeshPartitionTransformer.h"

#include "MeshPartitionWPActorPropertiesTransformer.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UDataLayerAsset;
class UHLODLayer;

namespace UE::MeshPartition
{

/**
 * Mesh Partition Transformer setting up all relevant WP properties for sections being produced.
 * Will only apply the transformation once per section.
 */
USTRUCT(MinimalAPI)
struct FWPActorPropertiesTransformer : public MeshPartition::FTransformer
{
	GENERATED_BODY()

public:
	UE_API virtual bool Execute(MeshPartition::FTransformerContext& InTransformerContext) const override;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& InDependencies) const override;

	FName GetRuntimeGrid() const { return RuntimeGrid; }

private:
	void SetActorProperties(const MeshPartition::FTransformerContext& InTransformerContext) const;

private:
	UPROPERTY(EditAnywhere, Category = "WorldPartition")
	bool bIsSpatiallyLoaded = true;
	
	UPROPERTY(EditAnywhere, Category = "WorldPartition")
	FName RuntimeGrid;

	UPROPERTY(EditAnywhere, Category = "DataLayers")
	TArray<TSoftObjectPtr<UDataLayerAsset>> DataLayerAssets;

	UPROPERTY(EditAnywhere, Category = "HLOD", meta = (DisplayName = "HLOD Layer"))
	TSoftObjectPtr<UHLODLayer> HLODLayer;

	UPROPERTY(EditAnywhere, Category = "HLOD", meta = (DisplayName = "Include Actor in HLOD"))
	bool bIncludeActorInHLOD = false;
};

} // namespace UE::MeshPartition

#undef UE_API
