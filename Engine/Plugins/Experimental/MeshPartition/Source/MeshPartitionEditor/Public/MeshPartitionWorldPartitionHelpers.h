// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorldPartition/WorldPartitionHandle.h"

class UWorld;
class UWorldPartition;
class ULevelStreaming;

namespace UE::MeshPartition
{
	class AMeshPartition;
	class UModifierComponent;
	struct FCompiledSectionBuildVariant;
	struct FGridSettings;
	struct FModifierGroup;
}

namespace UE::MeshPartition::WorldPartitionHelpers
{
	/**
	 * Resolves the WP grid for a build variant by scanning its transformer pipeline.
	 * Finds the last FWPActorPropertiesTransformer, reads its RuntimeGrid name, and resolves
	 * both the cell size and the 2D flag in a single pass.
	 * Fallback: if the named grid is not found, tries the main grid (NAME_None).
	 * Returns a default-constructed FGridSettings (CellSize=0, bIs2D=false) if no WP transformer is found or no grid resolves.
	 */
	MeshPartition::FGridSettings ResolveGridFromPipeline(const MeshPartition::FCompiledSectionBuildVariant& InBuildVariant, const UWorld* InWorld);

	/** Estimated grid cell with its absolute world-space coordinate and bounding box. */
	struct FGridCellEstimate
	{
		FIntVector GridCellCoord;
		FBox CellBounds;
	};

	/** Computes per-cell bounds from a bounding box and grid configuration, using the same snapping as BuildSections.
	 *  Returns all cells that the bounds overlap (some may contain no actual geometry).
	 *  When `InGridSettings.bIs2D` is true, all cells share `GridCellCoord.Z=0` and span the full Z extent of `InGroupBounds`.
	 *  Pass the MeshPartition actor's LocalToWorld so cell coords align with the WP runtime grid when the actor is translated;
	 *  pass FTransform::Identity for non world space aligned splits. */
	TArray<FGridCellEstimate> EstimateGridCells(const FBox& InGroupBounds, const MeshPartition::FGridSettings& InGridSettings, const FTransform& InLocalToWorld);

	TSet<ULevelStreaming*> LoadAllLevelInstances(UWorldPartition* WorldPartition, TSet<FWorldPartitionReference>& InOutActorRefs);

	void LoadAllActorsFromStreamingLevels(
		MeshPartition::FModifierGroup& InGroup,
		const AMeshPartition* InMegaMesh,
		UWorldPartition* WorldPartition,
		TSet<FWorldPartitionReference>& InOutActorRefs,
		TFunctionRef<void(AActor* Modifier, bool bIsInLevelInstance)> PerActorCallback,
		TFunctionRef<void(MeshPartition::UModifierComponent* Modifier, bool bIsInLevelInstance)> PerModifierCallback);

	void LoadGroupModifiersViaWorldPartitionRef(MeshPartition::FModifierGroup& InGroup, UWorldPartition* InWorldPartition, TSet<FWorldPartitionReference>& OutActorRefs);

	void LoadAllRelevantMegaMeshActors(MeshPartition::FModifierGroup& InGroup, const AMeshPartition* InMegaMesh, TSet<FWorldPartitionReference>& OutActorRefs, TFunctionRef<void(AActor* Modifier, bool bIsInLevelInstance)> PerActorCallback, TFunctionRef<void(MeshPartition::UModifierComponent* Modifier, bool bIsInLevelInstance)> PerModifierCallback);
}