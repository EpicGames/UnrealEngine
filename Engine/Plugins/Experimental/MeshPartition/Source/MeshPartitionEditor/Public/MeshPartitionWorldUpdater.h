// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionDescriptorCache.h"

namespace UE::MeshPartition
{
	class AMeshPartition;
	struct FMeshPartitionUpdater;
	struct FStatusSummary;

	// what is the state of the prebuilt compiled sections for a group
	struct FCompiledSectionStatus
	{
		bool bChecked : 1 = false;				// status has been determined
		bool bIsPlaceholder : 1 = false;		// is a PIE placeholder
		bool bNonTargetVariant : 1 = false;		// corresponds to a build variant that is not a target of this updater
		bool bMismatchedModifiers : 1 = false;	// modifier set hash does not match any target group (set hash is derived from the modifier paths)
		bool bBuildVariantHashMatches : 1 = false;	// build variant hash matches (definition/variant settings unchanged)
		bool bPackageHashMatches : 1 = false;	// package hash matches
		bool bClassHashMatches : 1 = false;  	// class hash matches
		bool bCannotFindFile : 1 = false;		// cannot find the compiled section file (asset registry does not know about it)
		bool bIsDuplicate : 1 = false;			// is a duplicate of another compiled section (and this one should not be used)
		bool bReuse : 1 = false;				// compiled section should be reused, because it is the best up-to-date compiled section for a target group

		FString ToString() const;
		FString GetDetailsString(int32 CompiledSectionIndex, const FMeshPartitionUpdater& MeshPartitionUpdater, IAssetRegistry& AssetRegistry) const;
	};

	// This structure reflects the status of compiled sections on disk (registered in the asset registry), with respect to the desired set of groups in a CachedBuildVariant.
	struct FBuildVariantUpdater
	{
		friend struct FMeshPartitionUpdater;

	public:
		FBuildVariantUpdater(const FGuid& InBuildKey, const AMeshPartition* InMeshPartition, const UMeshPartitionDefinition& InMeshPartitionDefinition, const MeshPartition::FModifierGrouping& InTargetModifierGrouping, TSharedPtr<MeshPartition::FCachedDescriptors> InAllMeshPartitionDescriptors, IAssetRegistry& InAssetRegistry, const UWorld* InWorld);

	private:
		// partial BuildInfo for this build variant (used to initialize PIE placeholders, and also stores info like BuildVariantName)
		FCompiledSectionBuildInfo BuildInfo;

		// this TSharedPtr keeps the CachedBuildVariant from being deallocated
		TSharedPtr<MeshPartition::FCachedDescriptors> AllMeshPartitionDescriptors;

		// the set of groups for a build variant
		const MeshPartition::FModifierGrouping& TargetGrouping;

		// mapping from modifier set hash to target group (index into TargetGrouping)
		TMap<FGuid, int32> ModifierSetHashToTargetGroupIndex;

		// Per-group reuse tracking: maps GridCellCoord → CompiledSectionIndex in the MeshPartitionUpdater's CompiledSections array.
		// For non-grid sections, the key is InvalidGridCellCoord. Empty map = no reusable sections for that group.
		TArray<TMap<FIntVector, int32>> ReusableCompiledSectionIndicesForGroup;

		// The set of base modifiers that are not currently represented within an up to date compiled section
		TMap<FSoftObjectPath, MeshPartition::FModifierDesc> BaseModifiersNeedingCreation;

		// Grid configuration for this build variant (CellSize=0 = not grid-aligned or resolution failed)
		MeshPartition::FGridSettings GridSettings;

	public:
		// Returns the target group index for the group with the specified modifier set hash.  Returns -1 if no matching target group is found.
		int32 GetTargetGroupIndexFromModifierSetHash(const FGuid& ModifierSetHash) const { return ModifierSetHashToTargetGroupIndex.FindRef(ModifierSetHash, -1); }
		TConstArrayView<MeshPartition::FModifierGroup> GetTargetModifierGroups() const { return TargetGrouping.GetModifierGroups(); }

		FName GetBuildVariantName() const { return BuildInfo.BuildVariantName; }
		const MeshPartition::FModifierGrouping& GetTargetGrouping() const { return TargetGrouping; }
		const FCompiledSectionBuildInfo& GetBuildInfo() const { return BuildInfo; }
		const FSoftObjectPath& GetMeshPartitionPath() const { return BuildInfo.MegaMeshPath; }
		const MeshPartition::FGridSettings& GetGridSettings() const { return GridSettings; }

		bool HasReusableCompiledSectionsForGroup(int32 InGroupIndex, const FMeshPartitionUpdater& InMeshPartitionUpdater) const;
		const TMap<FIntVector, int32>& GetReusableCompiledSectionIndicesForGroup(int32 InGroupIndex) const { return ReusableCompiledSectionIndicesForGroup[InGroupIndex]; }

		const TMap<FSoftObjectPath, MeshPartition::FModifierDesc>& GetBaseModifiersNeedingCreation() const { return BaseModifiersNeedingCreation; }
	};

	// captures the current state of a single mesh partition build variant, and manages what needs to be done to bring it up to date
	struct FMeshPartitionUpdater
	{
		// this shared ptr keeps the MeshPartition::FCachedDescriptors and all sub-structures (including the CompiledSections array) alive until we are done with our operation
		TSharedPtr<MeshPartition::FCachedDescriptors> AllCachedDescriptors;

		// the set of existing compiled sections for our mesh partition (across ALL build variants) - this points into the array inside AllCachedDescriptors
		TConstArrayView<FCompiledSectionDescriptor> CompiledSections;

		// Status of each compiled section (array is 1:1 with CompiledSections above)
		TArray<MeshPartition::FCompiledSectionStatus> CompiledSectionStatus;

		// Updater for each build variant, indexed by build variant name
		TMap<FName, FBuildVariantUpdater> BuildVariantUpdaters;

		// Maps a compiled section actor guid to its index in the CompiledSections array
		TMap<FGuid, int32> ActorGuidToCompiledSectionIndex;

		// MeshPartition actor that owns this updater. Held weakly so callers can resolve the live
		// actor transform on demand (e.g. for AMeshPartition::WorldToLocal/LocalToWorld during
		// placeholder cell estimation) -- avoids the stale-snapshot class of bug if the actor is
		// moved between updater construction and placeholder creation.
		TWeakObjectPtr<const AMeshPartition> MeshPartition;

		// stats for what was updated (NOTE that if this updater is used on multiple copies of the world, i.e. during multi-head PIE, then these stats are aggregated)
		mutable int32 RemovedOutOfDate = 0;						// compiled sections removed for being out of date
		mutable int32 RemovedNonTargetBuildVariant = 0;			// compiled sections removed for not being part of a target build variant
		mutable int32 ReusedCompiledSections = 0;				// existing compiled sections that were re-used
		mutable int32 AddedPlaceholderCompiledSections = 0;		// placeholder compiled sections created
		mutable int32 RemovedUnknown = 0;						// compiled sections removed because they did not correspond to any compiled section tracked in this updater (unexpected state)

	public:
		FMeshPartitionUpdater(const FGuid& InBuildKey, const AMeshPartition* InMeshPartition, const UMeshPartitionDefinition& InMeshPartitionDefinition, TSharedPtr<MeshPartition::FCachedDescriptors> InMeshPartitionAllCachedDescriptors, TConstArrayView<FName> InTargetBuildVariantNames, IAssetRegistry& InAssetRegistry, const UWorld* InWorld);
		const FBuildVariantUpdater* GetBuildVariantUpdater(const FName& BuildVariantName) const { return BuildVariantUpdaters.Find(BuildVariantName); }
		void ForAllBuildVariants(TFunctionRef<bool(const MeshPartition::FBuildVariantUpdater&)> InFunc) const;

		const FCompiledSectionDescriptor* GetCompiledSectionDescriptor(int32 CompiledSectionIndex) const { return CompiledSections.IsValidIndex(CompiledSectionIndex) ? &CompiledSections[CompiledSectionIndex] : nullptr; }
		const FCompiledSectionStatus* GetCompiledSectionStatus(int32 CompiledSectionIndex) const;
		const FCompiledSectionStatus* GetCompiledSectionStatus(const ACompiledSection* CompiledSection) const;

		void GetStatusSummary(FStatusSummary& InOutSummary) const;
	};

	// captures the current state of all mesh partitions in a world, and manages what needs to be done to bring the world up to date
	struct FMeshPartitionWorldUpdater
	{
		enum EUpdateMode
		{
			ForPIE,			// will query mesh partitions and build variants necessary for PIE
			ForCook,		// query all mesh partitions across all configured build variants - for cook, will issue warnings if compiled sections are not up to date
			ForCompile,		// query all mesh partitions across all configured build variants - for compile
		};

	private:
		EUpdateMode UpdateMode;
		FGuid BuildKey;

		// Updater for each mesh partition in the world, indexed by mesh partition guid
		TMap<FGuid, FMeshPartitionUpdater> MeshPartitionUpdaters;

		// record the world that placeholders were added to, so we can remove them afterwards
		TWeakObjectPtr<UWorld> PlaceholdersAddedToWorld;

	public:
		// construct world updater for the specified world
		FMeshPartitionWorldUpdater(UWorld* InWorld, EUpdateMode InUpdateMode);

		EUpdateMode GetUpdateMode() const { return UpdateMode; }
		const FGuid& GetBuildKey() const { return BuildKey; }

		int32 CreatePlaceholderActors(UWorld* InTargetWorld);
		int32 RemovePlaceholderActors();
		UWorld* GetPlaceholderWorld() const { return PlaceholdersAddedToWorld.Get(); }

		const FMeshPartitionUpdater* GetMeshPartitionUpdaterFor(const ACompiledSection* CompiledSection) const;
		const FCompiledSectionStatus* GetCompiledSectionStatus(const ACompiledSection* CompiledSection) const;

		void ReportStats() const;
		void GetStatusSummary(FStatusSummary& InOutSummary) const;
		void ReportSectionStatus(UWorld* InWorld) const;
	};

	// summary of the status of compiled sections and groups
	struct FStatusSummary
	{
		// overall counts
		int32 MeshPartitionCount = 0;		// total mesh partitions (existing actors, does not include deleted actors)
		int32 TargetBuildVariantCount = 0;	// total number of target build variants (sum over all mesh partitions)
		int32 GroupCount = 0;				// total target modifier groups (sum over all mesh partitions, all target build variants)
		int32 CompiledSectionCount = 0;		// total existing compiled sections (including out of date sections)

		struct FCompiledSections
		{
			// the counts here are mutually exclusive, even if a compiled section meets multiple criteria, it will only count towards one
			// the exception is "Missing" which indicates compiled sections that do not exist (and can overlap with CannotFindFile)

			// these compiled sections should be removed (RuntimeCellTransformer actively deletes them)
			int32 Placeholders = 0;				// compiled sections that are PIE placeholder leftovers (not cleaned up from previous runs)
			int32 NonTargetVariant = 0;			// compiled sections that do not correspond to any target build variant
			int32 MismatchedModifiers = 0;		// compiled sections whose modifier sets do not correspond to any target group
			int32 Duplicates = 0;				// compiled sections that are redundant duplicates of the same build variant + modifier group
			int32 BuildVariantHashFails = 0;	// compiled sections whose build variant hash is out of date (definition/variant settings changed — section is fundamentally invalid)

			// these compiled sections need to be updated (content is stale but configuration matches)
			int32 PackageHashFails = 0;			// compiled sections whose package hashes are out of date and need to be rebuilt, but do correspond to a current group
			int32 ClassHashFails = 0;			// compiled sections whose class hashes are out of date and need to be rebuilt, but do correspond to a current group

			// these compiled sections can be reused
			int32 Reuse = 0;					// compiled sections that are up to date and do not need any changes

			// these compiled sections are missing and need to be created
			int32 Missing = 0;

			// compiled sections that WERE reuseable, but were recently deleted (they still exist in the actor descriptors, but are not in the asset registry) - we can safely ignore these, they will be added to Missing if needed
			int32 CannotFindFile = 0;

			// this should always be zero, it's counting compiled sections that have none of the above status flags set, which shouldn't happen
			int32 Unknown = 0;
		};
		FCompiledSections CompiledSections;

		// Compiled sections that are up to date and will be reused (grid-split groups contribute N each)
		int32 SectionsUpToDate() { return CompiledSections.Reuse; }

		// Compiled sections whose package or class hash is out of date
		int32 SectionsOutOfDate() { return CompiledSections.PackageHashFails + CompiledSections.ClassHashFails; }

		// Modifier groups that have no corresponding compiled section at all
		int32 GroupsMissing() { return CompiledSections.Missing; }

		// compiled sections that should be deleted
		int32 CompiledSectionsToRemove() { return CompiledSections.Placeholders + CompiledSections.NonTargetVariant + CompiledSections.MismatchedModifiers + CompiledSections.Duplicates + CompiledSections.BuildVariantHashFails; }
	};
} // namespace UE::MeshPartition