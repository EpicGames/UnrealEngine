// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"

#include "CodeReusableMeshPartitionModifierInterface.generated.h"

namespace UE::MeshPartition
{
UINTERFACE(MinimalAPI)
class UCodeReusableModifier :
	public UInterface
{
	GENERATED_BODY()
};

/**
* A modifier that implements this interface may be reused by code that manages modifiers.
* 
* Notably, it can be reused by PCG elements that make use of UE::MeshPartition::Utils::GetPCGManagedMegaMeshModifier.
*/
class ICodeReusableModifier
{
	GENERATED_BODY()

public:

	/**
	* Used by code to temporarily deactive the modifier. 
	*
	* For example, a PCG spawning node would use this function to temporarily deactivate the modifier at
	*  the start of graph execution so that it can sample an unmodified megamesh, but still have the 
	*  component around in case it is able to reuse it later.
	*
	* Note that higher-level MegaMesh systems won't know about this deactivation, so implementers are
	*  responsible for making sure that the modifier doesn't make any changes and its cache keys are 
	*  appropriately invalidated. 
	*/
	virtual void SetDisabledByCode(bool bDisabledIn) = 0;

	/**
	* Called to prep the modifier for reinitializing with entirely different data.
	*/
	virtual void ResetForReuse() = 0;

	/**
	* Can be called to determine whether this reusable modifier ended up actually being used, so that 
	*  it can be cleaned up by its managing system if not.
	*/
	virtual bool IsUsed() const = 0;
};
} // namespace UE::MeshPartition