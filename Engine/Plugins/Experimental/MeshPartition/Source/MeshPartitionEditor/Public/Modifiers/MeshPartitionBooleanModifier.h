// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionChannel.h" // MeshPartition::FChannelName
#include "Modifiers/MeshPartitionMeshBasedModifierBase.h"
#include "SceneView.h"
#include "PrimitiveDrawInterface.h"
#include "Templates/SharedPointer.h"
#include "Containers/Array.h"

#include "Components/DynamicMeshComponent.h"

#include "MeshPartitionBooleanModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API


namespace UE::MeshPartition
{
/** CSG operation types */
UENUM()
enum class EBooleanOperation
{
	/** Merges the Tool and Mesh Partition geometry into a locally-closed surface, removing internal surfaces where they overlap. */
	Union = 0 UMETA(DisplayName = "Union"),

	/** Subtracts the Tool mesh from the Mesh Partition geometry, removing overlapping geometry and capping the result into a locally-closed surface. */
	Subtract = 1 UMETA(DisplayName = "Subtract"),

	/** Cuts away Mesh Partition geometry inside the Tool mesh, leaving an open hole at the boundary. */
	Trim = 2 UMETA(DisplayName = "Trim")
};

/** Method to use for determining what is 'inside' a mesh */
UENUM()
enum class EBooleanInsideTestMethod
{
	/** Use the fast winding number to determine if a point is inside a mesh */
	WindingNumber = 0
};

/** Options to account for geometry outside the local region of the Boolean modifier tool mesh. */
UENUM()
enum class EBooleanToolMeshEmbedding
{
	// Assume the terrain locally passes through the Boolean operator bounds, and extends to a closed solid outside those bounds.
	// Use this as the default for most terrain Booleans.
	Intersecting,
	// Assume terrain outside the Boolean operator fully encloses the operator's bounds.
	// Use this if the Boolean is applied fully underground.
	Inside,
	// Assume no terrain from outside the operator bounds will affect the Boolean operation.
	// Use this if the Boolean is applied fully above ground.
	Outside
};

UENUM()
enum class EBooleanModifierChannelSourceMode : uint8
{
	VertexColor,
	VertexWeight,
	Constant
};

UENUM()
enum class EMegaMeshBooleanModifierPreviewVisOptions : uint8
{
	WhenTemporarilyDisabled,
	Never,
	Always
};

// Define weight channels to transfer from the boolean tool mesh
USTRUCT()
struct FBooleanModifierWeightEntry
{
	GENERATED_BODY()

	/** The destination attribute channel to write. */
	UPROPERTY(EditAnywhere, Category = WeightAttribute, meta = (GetOptions = "GetMegaMeshDefinitionChannels"))
	MeshPartition::FChannelName ChannelName;

	/** Determines where the values to write are acquired. */
	UPROPERTY(EditAnywhere, Category = Weight)
	MeshPartition::EBooleanModifierChannelSourceMode SourceMode = MeshPartition::EBooleanModifierChannelSourceMode::VertexColor;

	/** Which channel of a vertex color to use as source data (0 through 3 for RGBA) */
	UPROPERTY(EditAnywhere, Category = Weight, meta = (ClampMin = "0", ClampMax = "3",
		EditCondition = "SourceMode == EBooleanModifierChannelSourceMode::VertexColor", EditConditionHides))
	int32 VertexColorIndex = 0;

	/** Vertex weight channel name to use as source data */
	UPROPERTY(EditAnywhere, Category = Weight, meta = (
		EditCondition = "SourceMode == EBooleanModifierChannelSourceMode::VertexWeight", EditConditionHides))
	FName SourceWeightChannelName;

	UPROPERTY(EditAnywhere, Category = Weight, meta = (
		EditCondition = "SourceMode == EBooleanModifierChannelSourceMode::Constant", EditConditionHides))
	float WriteValue = 1.f;
};


UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Mesh", "Boolean", "WeightChannels"), meta = (BlueprintSpawnableComponent, MegaMeshClassVersion = "1"))
class UBooleanModifier : public MeshPartition::UMeshBasedModifierBase
{
	GENERATED_BODY()

public:
	UE_API UBooleanModifier();

	/**
	* Updates internal mesh data from given detail panel inputs
	*/
	UFUNCTION(CallInEditor, Category = Mesh)
	UE_API void UpdateFromMesh();

	// MeshPartition::UMeshBasedModifierBase
	UE_API virtual void ProcessMeshInstance(FDynamicMesh3& OutMesh);

	// MeshPartition::UModifierComponent
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	virtual bool IsContiguous() const { return BooleanOp != EBooleanOperation::Trim; }
	UE_API TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;
	UE_API virtual double GetComplexity() const;

	// MeshPartition::UBooleanModifier
	/**
	 * Sets the amount to expand the global Boolean operation bounds beyond the tool mesh bounds.
	 * @param InExpandOperatorBounds The expansion amount along each axis.
	 */
	void SetExpandOperatorBounds(const FVector3d& InExpandOperatorBounds) { ExpandOperatorBounds = InExpandOperatorBounds; }

	/**
	 * @return The amount to expand the global Boolean operation bounds beyond the tool mesh bounds.
	 */
	const FVector3d& GetExpandOperatorBounds() const { return ExpandOperatorBounds; }

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;

private:

	UE_API virtual void PostUpdateMeshInstance(const FDynamicMesh3& MeshInstanceData) override;

	UE_API FBox ComputeWorldExpandedBounds(FVector ExpandAmount) const;

	bool ShouldDrawSolidPreview() const
	{
		if (DrawSolidMesh == EMegaMeshBooleanModifierPreviewVisOptions::WhenTemporarilyDisabled)
		{
			return IsTemporarilyDisabledInEditor();
		}
		else
		{
			return DrawSolidMesh == EMegaMeshBooleanModifierPreviewVisOptions::Always;
		}
	}
	
	/**
	* Strength of simplification to apply to the op mesh before using it in the boolean operation. 
	*  Higher value indicates more simplification, 0 is none. Value corresponds to desired edge
	*  length in the simplified mesh.
	*/
	UPROPERTY(EditAnywhere, Category = Mesh, meta = (ClampMin = "0.0", UIMax = "200"))
	double PreOpSimplifierStrength { 0 };

	/**
	* When simplifying, whether to try to preserve boundary edge flow.
	*/
	UPROPERTY(EditAnywhere, Category = Mesh, AdvancedDisplay)
	bool bSimplifierConstrainBoundary = true;

	/**
	* When simplifying, whether to try to preserve various seam edge flows (UVs, normals, etc).
	*/
	UPROPERTY(EditAnywhere, Category = Mesh, AdvancedDisplay)
	bool bSimplifierConstrainSeams = false;

	/**
	* Weight Channels to transfer from the tool mesh to the target mesh
	*/
	UPROPERTY(EditAnywhere, Category = WeightChannels, Meta = (ShowOnlyInnerProperties))
	TArray<MeshPartition::FBooleanModifierWeightEntry> WeightChannels;

	UPROPERTY(EditAnywhere, Category = Boolean)
	MeshPartition::EBooleanOperation BooleanOp { MeshPartition::EBooleanOperation::Union };

	/**
	* What to assume about the terrain/target mesh beyond the local operator bounds of the modifier.
	* 
	* This is useful for e.g. using Boolean subtractions to create caverns underneath the surface terrain, 
	* without needing to extend the Boolean operator bounds all the way to the surface.
	*/
	UPROPERTY(EditAnywhere, Category = Boolean, meta = (EditCondition="BooleanOp != EBooleanOperation::Trim"))
	MeshPartition::EBooleanToolMeshEmbedding ToolMeshEmbedding = MeshPartition::EBooleanToolMeshEmbedding::Intersecting;

	/* Method to use for determining the inside of the target mesh. */
	//~ Note: this is not exposed via EditAnywhere because there is currently only one option; if it were exposed to users, it would use the commented out category and display options
	UPROPERTY(/*EditAnywhere, Category = Boolean, AdvancedDisplay*/)
	MeshPartition::EBooleanInsideTestMethod InsideTargetMeshTestMethod = MeshPartition::EBooleanInsideTestMethod::WindingNumber;

	/* Winding number threshold to determine inside of the Target mesh, if the InsideTargetMeshTestMethod is WindingNumber */
	UPROPERTY(EditAnywhere, Category = Boolean, AdvancedDisplay,
		meta = (EditCondition = "InsideTargetMeshTestMethod == EBooleanInsideTestMethod::WindingNumber", UIMin = "0.0", UIMax = "1.0"))
	double TargetWindingThreshold{ 0.5 };

	/* Winding number threshold to determine the inside of the tool mesh. */
	UPROPERTY(EditAnywhere, Category = Boolean, AdvancedDisplay, meta = (UIMin = "0.0", UIMax = "1.0"))
	double ToolWindingThreshold{ 0.5 };

	/* Apply simplification of newly introduced edges at mesh boundaries. */
	UPROPERTY(EditAnywhere, Category = Boolean)
	bool bSimplifyAlongNewEdges{ true };

	/* Whether to weld newly-created cut edges where the input meshes meet. */
	UPROPERTY(EditAnywhere, Category = Boolean, AdvancedDisplay)
	bool bWeldSharedEdges{ true };

	/**
	* Amount to expand the global bounds defining which sections the modifier may write to.
	* Use this to artificially extend the modifier bounds when it does not overlap any input sections,
	* to make sure that it will find a section to write to.
	*
	* Note: Expanding these bounds will not expand the amount of geometry the Boolean operator sees or affects.
	*/
	UPROPERTY(EditAnywhere, Category = Boolean, meta = (ClampMin = "0.0", UIMax = "100000.0"))
	FVector3d ExpandSectionInclusionBounds{ 100.,100.,100. };

	/** 
	* Amount to expand the global Boolean operation bounds beyond the tool mesh bounds. 
	* Use this to expand the amount of terrain geometry that can be directly used in the Boolean operation. 
	*/
	UPROPERTY(EditAnywhere, Category = Boolean, meta = (UIMin = "0.0", UIMax = "10000.0"))
	FVector3d ExpandOperatorBounds{ 100.,100.,100. };


	UPROPERTY(EditAnywhere, Category = "Modifier|Visualization")
	bool bDrawLocalBounds{ true };

	UPROPERTY(EditAnywhere, Category = "Modifier|Visualization")
	bool bDrawWorldBounds{ true };

	UPROPERTY(EditAnywhere, Category = "Modifier|Visualization")
	bool bDrawWireMesh{ true };

	UPROPERTY(EditAnywhere, Category = "Modifier|Visualization")
	EMegaMeshBooleanModifierPreviewVisOptions DrawSolidMesh = EMegaMeshBooleanModifierPreviewVisOptions::WhenTemporarilyDisabled;

	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> ToolPreviewComponent = nullptr;
};
} // namespace UE::MeshPartition

#undef UE_API
