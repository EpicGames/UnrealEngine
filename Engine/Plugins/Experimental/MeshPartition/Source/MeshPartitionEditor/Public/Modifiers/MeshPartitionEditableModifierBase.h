// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DynamicMesh/DynamicMesh3.h"
#include "Math/MathFwd.h"
#include "MeshPartitionModifierComponent.h"

#include "MeshPartitionEditableModifierBase.generated.h"

namespace UE::MeshPartition
{
/**
* A base class for modifiers that want to support direct editing via a tool target
*/
UCLASS(MinimalAPI, Abstract)
class UEditableModifierBase : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()

public:

	UEditableModifierBase() {}

	// Whether to allow tools to launch and edit this modifier
	virtual bool SupportsToolEditing() const
	{
		// Nothing to edit if modifier is not attached to a mesh
		return GetMeshPartitionEditorComponent() != nullptr;
	}

	// Update the modifier using the updated mesh
	virtual void ApplyEditWithMesh(const FDynamicMesh3& UpdatedMesh)
	{
	}

	// Apply any custom, modifier-specific initialization to the extracted mesh before launching tooling to edit it
	virtual void PrepareForEdit(FDynamicMesh3& EditMesh) const
	{
	}

	// Bounds to use for editing the modifier.
	// As opposed to ComputeBounds, these bounds are oriented to allow edited modifiers to more tightly fit their shape.
	// This is especially important for the tools to generate a nice preview.
	// Note: Is allowed to differ from ComputeBounds
	virtual TArray<Geometry::FOrientedBox3d> GetBoundsForEdit() const
	{
		TArray<FBox> ModifierBounds = ComputeBounds();
		TArray<Geometry::FOrientedBox3d> OrientedBounds;
		Algo::Transform(ModifierBounds, OrientedBounds, [](const FBox& Box)
			{
				Geometry::FOrientedBox3d OrientedBox(Box); // Boxes returned by ComputeBounds are in world space already and orienting them would cause them to be larger than they should be.
				return OrientedBox;
			});
		return OrientedBounds;
	}


};
} // namespace UE::MeshPartition
