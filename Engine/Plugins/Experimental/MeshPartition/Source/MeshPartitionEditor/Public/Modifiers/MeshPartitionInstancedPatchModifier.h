// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionPatchModifier.h"
#include "Modifiers/CodeReusableMeshPartitionModifierInterface.h"

#include "MeshPartitionInstancedPatchModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{
namespace MegaMeshInstancedPatchModifierLocals
{
	class FBackgroundOp;
}

/**
* Variant of the MeshPartition::UPatchModifier which supports multiple instances of the same parameter with different instance locations,
*  to be used with PCG.
* Instanced patches are approximately as fast to compute as non-instanced patches, but it is substantially faster to add/remove individual instances.
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Patch"), meta = (MegaMeshClassVersion = "1"))
class UInstancedPatchModifier : public MeshPartition::UPatchModifier
	, public ICodeReusableModifier
{
	GENERATED_BODY()
	
public:
	UE_API UInstancedPatchModifier();

	// ICodeReusableModifier
	UE_API virtual void SetDisabledByCode(bool bDisabledByCode) override;
	UE_API virtual void ResetForReuse() override;
	UE_API virtual bool IsUsed() const override;

	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual bool IsTemporarilyDisabledInEditor() const override;
	// End MeshPartition::UModifierComponent Implementation

	UE_API void AddInstances(const TArray<FVector>& InNewInstances);
	UE_API void ClearInstances();
	int32 NumInstances() const { return Instances.Num(); }

protected:
	friend class MegaMeshInstancedPatchModifierLocals::FBackgroundOp;

	/** @return Position of instance in worldspace */
	UE_API FVector GetInstanceWorldspaceLocation(int InInstanceID) const;
	/** @return Bounds of instance in worldspace */
	UE_API FBox GetInstanceWorldspaceBounds(int InInstanceID) const;
	
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

private:
	/** Instance positions in this component's local space */
	UPROPERTY()
	TArray<FVector> Instances;

	// Cached to avoid copying array on each CreateBackgroundOp when instances have not actually changed.
	mutable TSharedPtr<const TArray<FVector>> InstancesForBackgroundOp;

	// See ICodeReusableModifier::SetDisabledByCode
	bool bDisabledByCode = false;
};
} // namespace UE::MeshPartition

#undef UE_API
