// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierComponent.h"
#include "Modifiers/CodeReusableMeshPartitionModifierInterface.h"
#include "Modifiers/MeshPartitionMeshBasedModifierBase.h"
#include "Modifiers/MeshPartitionMeshProjectModifier.h" // MeshPartition::EProjectModifierBlendMode
#include "UDynamicMesh.h"

#include "MeshPartitionInstancedProjectionModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{
/**
* Data needed for an instance of the mesh projection modifier.
*/
USTRUCT(BlueprintType)
struct FInstancedProjectionModifierInstance
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Projection)
	TObjectPtr<const UDynamicMesh> Mesh = nullptr;

	UPROPERTY(VisibleAnywhere, Category = Projection)
	FTransform MeshToWorld = FTransform::Identity;
	
	/**
	* Determines the direction in which projection is done. The projection is done to a point
	*  raycasted in the negative Z direction of this transform, and only vertices inside
	*  ProjectionSpaceBounds (which is given in the projection space) are modified.
	*/
	UPROPERTY(VisibleAnywhere, Category = Projection)
	FTransform ProjectionToWorld = FTransform::Identity;
	UPROPERTY(VisibleAnywhere, Category = Projection)
	FBox ProjectionSpaceBounds;
};

/**
* Modifier that takes meshes and projects the megamesh to them, capable of holding multiple projection instances
*  at once so that a new component doesn't have to be created for each one in PCG.
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Settings"))
class UInstancedProjectionModifier : public MeshPartition::UModifierComponent
	, public ICodeReusableModifier
{
	GENERATED_BODY()

public:
	UE_API UInstancedProjectionModifier();

	// ICodeReusableModifier
	UE_API virtual void SetDisabledByCode(bool bDisabledByCode) override;
	UE_API virtual void ResetForReuse() override;
	UE_API virtual bool IsUsed() const override;
	
	// MeshPartition::UModifierComponent
	UE_API virtual void InitializeModifier() override;
	UE_API virtual void UninitializeModifier() override;
	[[nodiscard]] UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual bool IsTemporarilyDisabledInEditor() const;

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;
	using Super::PreEditChange;
	UE_API virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	/**
	* @param bUpdateCachedMesh If true, then any existing cached version of the provided mesh will be updated. This
	*   can be set to false when adding multiple instances in a row that share the same mesh, to avoid recreating
	*   the cached data each time.
	*/
	UE_API void AddInstance(const MeshPartition::FInstancedProjectionModifierInstance& NewInstance, bool bUpdateCachedMesh);
	UE_API void ClearInstances();
	int32 NumInstances() const { return Instances.Num(); }
	UE_API const MeshPartition::FInstancedProjectionModifierInstance* GetInstanceAtIndex(int32 Index) const;

private:
	UE_API bool CreateCachedData(const UDynamicMesh* MeshKey, bool bUpdateIfExisting);
	UE_API UDynamicMesh* GetOrCreateMeshCopy(const UDynamicMesh* MeshKey);

	UE_API void OnCurveChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType);
	UE_API void AttachCurveListeners();
	UE_API void DetachCurveListeners();

private:
	UPROPERTY(VisibleAnywhere, Category = Settings)
	TArray<MeshPartition::FInstancedProjectionModifierInstance> Instances;

	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bDrawAffectedBox = true;

	UPROPERTY(EditAnywhere, Category = Settings, AdvancedDisplay)
	bool bDrawGlobalBounds = true;

	UPROPERTY(EditAnywhere, Category = Settings)
	MeshPartition::EProjectModifierBlendMode BlendMode = MeshPartition::EProjectModifierBlendMode::Set;

	UPROPERTY(EditAnywhere, Category = Settings)
	MeshPartition::FProjectModifierFalloffSettings HeightFalloff;

	UPROPERTY(EditAnywhere, Category = Settings)
	TArray<FProjectModifierWeightEntry> WeightChannels;

	// Updated by CreateCachedData. Key is our own copy of a mesh.
	TMap<const UDynamicMesh*, TSharedPtr<const Geometry::FDynamicMesh3>> ThreadSafeMeshCopies;
	TMap<const UDynamicMesh*, TSharedPtr<const Geometry::FDynamicMeshAABBTree3>> MeshSpatials;
	// Updated by GetOrCreateMeshCopy. Key is a mesh provided to us in AddInstance. We make copies of any
	//  meshes passed to us, and this lets us avoid making duplicate copies.
	// Note: on load, this will no longer be initialized, so we would start making copies if we added the
	//  same meshes. It's unclear whether this is worth dealing with.
	TMap<const UDynamicMesh*, TWeakObjectPtr<UDynamicMesh>> SourceMeshToInstanceMesh;

	// See ICodeReusableModifier::SetDisabledByCode
	bool bDisabledByCode = false;

	friend class FPCGProjectionSpawnerElement;
};
} // namespace UE::MeshPartition

#undef UE_API
