// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Modifiers/CodeReusableMeshPartitionModifierInterface.h"

#include "MeshPartitionInstancedTexturePatchModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "HeightDisplacement", "WeightChannels", "AdaptiveTessellation", "Instance"), meta = (BlueprintSpawnableComponent, MegaMeshClassVersion = "1"))
class UInstancedTexturePatchModifier : public MeshPartition::UTexturePatchModifier
	, public ICodeReusableModifier
{
	GENERATED_BODY()
public:
	UE_API UInstancedTexturePatchModifier();
	
	// ICodeReusableModifier
	UE_API virtual void SetDisabledByCode(bool bDisabledByCode) override;
	UE_API virtual void ResetForReuse() override;
	UE_API virtual bool IsUsed() const override;

	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual bool IsTemporarilyDisabledInEditor() const override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	// End MeshPartition::UModifierComponent Implementation

	UE_API void AddInstances(const TArray<FVector>& InNewInstances);
	UE_API void ClearInstances();
	int32 NumInstances() const { return Instances.Num(); }

protected:
	UE_API FTransform GetInstanceWorldspaceTransform(int InInstanceID) const;
	/** @return Bounds of instance in worldspace */
	UE_API FBox GetInstanceWorldspaceBounds(int InInstanceID) const;
	
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

private:
	UPROPERTY(EditAnywhere, Category = Instance)
	TArray<FTransform> Instances;

	bool bDisabledByCode = false;
};
}

#undef UE_API