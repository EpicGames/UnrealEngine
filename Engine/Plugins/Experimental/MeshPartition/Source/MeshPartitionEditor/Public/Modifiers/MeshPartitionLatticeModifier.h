// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierComponent.h"
#include "Modifiers/MeshPartitionEditableModifierBase.h"
#include "LatticeManager.h"
#include "IntVectorTypes.h"

#include "MeshPartitionLatticeModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Lattice", "Interpolation"), meta = (BlueprintSpawnableComponent))
class ULatticeModifier : public MeshPartition::UEditableModifierBase, public ILatticeStateStorage
{
	GENERATED_BODY()

public:

	ULatticeModifier() = default;
	virtual ~ULatticeModifier() = default;

private:

	// ILatticeStateStorage
	UE_API virtual void StoreLatticePoints(const TArray<FVector3d>& LatticePoints) override;
	UE_API virtual void ReadLatticePoints(TArray<FVector3d>& LatticePoints) const override;
	UE_API virtual void StoreInterpolationType(ELatticeInterpolationType Type) override;
	UE_API virtual ELatticeInterpolationType ReadInterpolationType() const override;
	UE_API virtual Geometry::FAxisAlignedBox3d GetInitialBounds() const override;
	UE_API virtual Geometry::FTransformSRT3d GetTransform() const override;
	UE_API virtual Geometry::FVector3i GetResolution() const override;
	UE_API virtual void InteractiveToolStarted() override;
	UE_API virtual void InteractiveToolShutDown() override;

	// UObject
#if WITH_EDITOR
	UE_API virtual void PostInitProperties() override;
	UE_API virtual void BeginDestroy() override;
	UE_API virtual void PreEditChange(FProperty* PropertyThatWillChange) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	// UMegaMeshEditorActorComponent Implementation
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override; 
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;

	// MeshPartition::UEditableModifierBase
	UE_API virtual void PrepareForEdit(FDynamicMesh3& EditMesh) const override;

	UE_API void InitializeControlPoints();

	UE_API FBox ComputeLocalBounds() const;


	/** Lattice control points */
	UPROPERTY()
	TArray<FVector3d> LatticeControlPoints;

	/** Region affected by the lattice deformer */
	UPROPERTY(EditAnywhere, Category = Lattice, meta=(EditCondition = "!bToolIsRunning", HideEditConditionToggle))
	FVector3d Extents = FVector3d(1000, 1000, 1000);

	/** Number of lattice vertices along each axis */
	UPROPERTY(EditAnywhere, Category = Lattice, meta = (EditCondition = "!bToolIsRunning", HideEditConditionToggle))
	FIntVector LatticeResolution = FIntVector(5, 5, 5);

	/** Whether to use linear or cubic interpolation to get new mesh vertex positions from the lattice */
	UPROPERTY(EditAnywhere, Category = Interpolation, meta = (EditCondition = "!bToolIsRunning"))
	ELatticeInterpolationType InterpolationType = ELatticeInterpolationType::Linear;

	/** Whether to draw the bounding box of the undeformed lattice */
	UPROPERTY(EditAnywhere, Category = Lattice, AdvancedDisplay)
	bool bDrawLocalBounds = false;

	/** Whether to draw the lattice vertices */
	UPROPERTY(EditAnywhere, Category = Lattice, AdvancedDisplay)
	bool bDrawLatticePoints = false;

	// Cached value of the lattice resolution property
	Geometry::FVector3i PreviousLatticeResolution;
	FVector3d PreviousLatticeExtents;

	/** Delete all lattice changes and reset to initial configuration */
	UFUNCTION(CallInEditor, Category = Lattice)
	UE_API void ClearLattice();

	/** Start the interactice lattice deformer tool. NOTE: Modeling Mode must be active for this to do anything */
	UFUNCTION(CallInEditor, Category = Lattice)
	UE_API void StartLatticeTool();

	// Not user visible
	UPROPERTY(meta = (TransientToolProperty))
	bool bToolIsRunning = false;

	FDelegateHandle OnEditorModeChanged;
	FDelegateHandle OnToolStartedDelegateHandle;
	FDelegateHandle OnToolEndedDelegateHandle;
};
} // namespace UE::MeshPartition

#undef UE_API
