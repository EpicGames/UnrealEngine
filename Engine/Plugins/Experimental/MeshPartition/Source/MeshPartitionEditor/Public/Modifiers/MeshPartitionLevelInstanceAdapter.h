// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionLevelInstanceAdapter.generated.h"

class UDynamicMesh;
class ILevelInstanceInterface;


namespace UE::MeshPartition
{
/**
* MeshPartition::ULevelInstanceAdapter is a MeshPartition::UModifierComponent that can be attached to a LevelInstance
* actor in the world to allow all UModifierComponents inside of that level instance to affect 
* the same MegaMesh that the adapter is set to affect.
*/
UCLASS(meta=(BlueprintSpawnableComponent))
class ULevelInstanceAdapter : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()
	
public:
	ULevelInstanceAdapter();
	~ULevelInstanceAdapter();

	// UObject Implementation
	virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	// End UObject Implementation

	// UActorComponent Implementation
	virtual void OnRegister() override;
	virtual void OnComponentDestroyed(bool bDestroyingHierarchy) override;
	// End UActorComponent Implementation
	
	// Begin MeshPartition::UModifierComponent Implementation
	virtual TArray<FBox> ComputeBounds() const override;
	virtual TArray<MeshPartition::UModifierComponent*> GetInteractiveProxies() override;
	virtual void OnChanged(TConstArrayView<const FBox> InBoundingBoxes, EChangeType InChangeType);
	// End MeshPartition::UModifierComponent Implementation
protected:
	void FixupModifiersInLevelInstance();
	void FixupModifierInstance(MeshPartition::UModifierComponent* InModifier);
	
	/**
	* Called by the engine when a new level is added to the world.
	* The adapter will check if this new level is owned by the level instance and uses this to know
	* when the level instance is ready and streamed in.
	*/
	void OnLevelAdded(ULevel* InLevel, UWorld* InWorld);
	
	/**
	* In the case of a LevelInstance with streaming enabled, it is not sufficient to depend on LevelAdded.
	* Each actor is loaded into the level instance independently through a sub-worldpartition so the only
	* reliable way to fixup modifiers in those actors is individually as they stream in.
	*/
	void OnLoadedActorAddedToLevel(AActor& InActor);

	/** Returns the owning level instance actor, or null if this component was incorrectly attached to a non-levelinstance. */
	ILevelInstanceInterface* GetOwningLevelInstance() const;

	void ForEachModifierInLevelInstance(TFunctionRef<void(MeshPartition::UModifierComponent*)> InOp) const;

private:
	TSet<MeshPartition::UModifierComponent*> OwnedModifiers;
};
} // namespace UE::MeshPartition