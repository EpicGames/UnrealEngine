// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Engine/EngineTypes.h" // FComponentReference
#include "Math/MathFwd.h" // FVector2D
#include "MeshPartitionModifierComponent.h"
#include "TargetInterfaces/MeshTargetInterfaceTypes.h" // EMeshLODIdentifier
#include "MeshPartitionModifierUtils.h" // TAsyncInitData

#include "MeshPartitionMeshBasedModifierBase.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UDynamicMeshComponent;
namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE::MeshPartition
{
	
class FAsyncMeshInstanceData
{
public:
	FAsyncMeshInstanceData(FDynamicMesh3 InMesh)
		: Mesh(MakeShared<FDynamicMesh3>(MoveTemp(InMesh)))
		, AABBTree(Mesh, [](const FDynamicMesh3& InMesh)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMeshInstanceData::BuildAABBTree);
					return Geometry::FDynamicMeshAABBTree3(&InMesh);
				})
		, MeshHash(Mesh, [](const FDynamicMesh3& InMesh)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(FAsyncMeshInstanceData::ComputeMeshHash);

					MeshPartition::Utils::FHashArchive HashArchive;
					HashArchive += InMesh;
					return HashArchive.GetGuidFromHash();
				})
	{
	}

	UE::Geometry::FAxisAlignedBox3d GetBounds() const
	{
		if (AABBTree.IsCompleted())
		{
			return AABBTree.GetResult().GetBoundingBox();
		}
		return Mesh->GetBounds(/* bParallel = */ true);
	}
	
	UE::Tasks::FTask GetAsyncInitTask() const
	{
		return AABBTree.GetAsyncInitTask();
	}
	
	const FDynamicMesh3& GetMesh() const
	{
		return *Mesh;
	}

	const UE::Geometry::FDynamicMeshAABBTree3& GetAABBTree() const
	{
		return AABBTree.GetResult();
	}
	
	const FGuid& GetHash() const
	{
		return MeshHash.GetResult();
	}

private:
	TSharedRef<const FDynamicMesh3> Mesh;

	MeshPartition::Utils::TAsyncTransform<UE::Geometry::FDynamicMeshAABBTree3> AABBTree;
	MeshPartition::Utils::TAsyncTransform<FGuid> MeshHash;
};


UENUM(BlueprintType)
enum class EModifierMeshSourceMode : uint8
{
	DynamicMeshComponent,
	StaticMesh
};

/**
* A helper base class for modifiers that want to use a mesh that a user sets using the detail
*  panel. If a class doesn't use the detail panel to set the mesh, it should not need to use
*  this base class, but it could use MeshPartition::UMeshBasedModifierBase::FMeshInstanceData to hold
*  data about mesh instances.
*/
UCLASS(MinimalAPI, Abstract)
class UMeshBasedModifierBase : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()

public:
	// Encapsulates information one would want for a mesh-based modifier, so that a single modifier component
	//  could potentially manage multiple instances at once.
	struct FMeshInstanceData
	{
		TSharedPtr<Geometry::FDynamicMesh3> Mesh;
		TSharedPtr<Geometry::FDynamicMeshAABBTree3> Spatial;
		TSharedPtr<Geometry::FAxisAlignedBox3d> MeshBounds;
		FGuid MeshHash;
	};

	UMeshBasedModifierBase();

	// MeshPartition::UModifierComponent
	UE_API virtual void InitializeModifier() override;
	UE_API virtual UE::Tasks::FTask GetAsyncPrepareResourcesTask() const override { return MeshInstance->GetAsyncInitTask(); }

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;

	UE_API virtual bool SetMeshComponent(UDynamicMeshComponent* DynamicMeshComponent);

	UE_API virtual void SetMeshSourceMode(const MeshPartition::EModifierMeshSourceMode InModifierMeshSourceMode);
	UE_API virtual MeshPartition::EModifierMeshSourceMode GetMeshSourceMode() const;

	UE_API virtual void SetStaticMesh(UStaticMesh* InStaticMesh);
	UE_API virtual UStaticMesh* GetStaticMesh() const;

private:

	static UE_API bool CopyDynamicMeshComponent(const UDynamicMeshComponent* MeshComponent,
		bool bWantAttributes, FDynamicMesh3& OutMesh);
	static UE_API bool CopyStaticMesh(const UStaticMesh* StaticMesh, EMeshLODIdentifier DesiredLOD,
		bool bWantAttributes, FDynamicMesh3& OutMesh);

	UPROPERTY(EditAnywhere, Category = Mesh)
	MeshPartition::EModifierMeshSourceMode MeshSourceMode = MeshPartition::EModifierMeshSourceMode::DynamicMeshComponent;

	UPROPERTY(EditAnywhere, Category = Mesh, Meta = (
		EditCondition = "MeshSourceMode == EModifierMeshSourceMode::StaticMesh", EditConditionHides))
	TObjectPtr<UStaticMesh> StaticMesh = nullptr;

	UFUNCTION(BlueprintCallable, Category = "Mesh Partition Modifier", meta=(DisplayName="Set Static Mesh"))
	UE_API void BP_SetStaticMesh(UStaticMesh* Mesh);

	/**
	* Level of detail of the mesh to use. Currently only supports non-auto-generated LODs. Negative one
	*  indicates highest quality LOD (including HiRes, if available). If chosen LOD does not exist or is
	*  auto generated, next available LOD is used.
	*/
	UPROPERTY(EditAnywhere, Category = Mesh, AdvancedDisplay, Meta = (ClampMin = -1, ClampMax = 7,
		EditCondition = "MeshSourceMode == EModifierMeshSourceMode::StaticMesh", EditConditionHides))
	int32 DesiredLOD = -1;

	UPROPERTY(EditInstanceOnly, Category = Mesh, Meta = (UseComponentPicker, AllowAnyActor, 
		AllowedClasses = "/Script/GeometryFramework.DynamicMeshComponent",
		EditCondition = "MeshSourceMode == EModifierMeshSourceMode::DynamicMeshComponent", EditConditionHides))
	FComponentReference MeshComponent;

	// Weak pointer to the mesh component which has its update event linked to this modifier (if any)
	UPROPERTY()
	TWeakObjectPtr<UDynamicMeshComponent> UpdatingDynamicMeshComponent = nullptr;

	UE_API void OnDynamicMeshChanged(UDynamicMeshComponent*);

	UE_API EMeshLODIdentifier GetDesiredLOD() const;
protected:
	TSharedPtr<FAsyncMeshInstanceData> MeshInstance;

	UE_API void UpdateMeshInstance();

	bool bKeepInternalMeshAttributes = false;

	// Apply any post-processing and compute derived data (including Spatial and MeshBounds) after a mesh instance is created/updated
	UE_API virtual void ProcessMeshInstance(FDynamicMesh3& InstanceDataOut) {}

	// Called after Mesh Instance has been updated (changed or cleared)
	virtual void PostUpdateMeshInstance(const FDynamicMesh3&) {}

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& DependencyContext) const override;
};
} // namespace UE::MeshPartition

#undef UE_API
