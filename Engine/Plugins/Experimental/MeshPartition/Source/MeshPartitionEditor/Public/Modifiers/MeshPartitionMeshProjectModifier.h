// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "Math/MathFwd.h" // FRotator
#include "MeshPartitionModifierUtils.h" // ERectangleFalloffMode
#include "Modifiers/MeshPartitionMeshBasedModifierBase.h"
#include "Modifiers/MeshPartitionTexturePatchModifier.h" // MeshPartition::ETexturePatchFalloffMode
#include "Modifiers/MeshPartitionProjectModifierTypes.h"

#include "MeshPartitionMeshProjectModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Mesh", "Settings", "Visualize"), meta = (BlueprintSpawnableComponent, MegaMeshClassVersion = "1"))
class UMeshProjectModifier : public MeshPartition::UMeshBasedModifierBase
{
	GENERATED_BODY()

public:

	UE_API UMeshProjectModifier();

	/**
	* Updates internal mesh data from given detail panel inputs
	*/
	UFUNCTION(CallInEditor, Category = Settings)
	UE_API void UpdateFromMesh();

	// MeshPartition::UModifierComponent
	UE_API virtual void InitializeModifier() override;
	UE_API virtual void UninitializeModifier() override;
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const; 
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const;
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;

	// UObject
	using Super::PreEditChange;
	UE_API virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	struct FFalloffParams
	{
		void Initialize(const MeshPartition::FProjectModifierFalloffSettings& Settings);

		Utils::ERectangleFalloffMode Mode = Utils::ERectangleFalloffMode::Smooth;
		float Distance = 0;
		double CornerRadius = 0;
		TSharedPtr<const FRichCurve> FalloffCurve = nullptr;
	};

	struct FWeightEntryParams
	{
		void Initialize(const FProjectModifierWeightEntry& Entry);

		FName WeightChannelName;

		MeshPartition::EProjectModifierChannelSourceMode SourceMode = MeshPartition::EProjectModifierChannelSourceMode::VertexColor;
		FName SourceWeightChannelName;
		int32 VertexColorIndex = 0;

		MeshPartition::EProjectModifierBlendMode BlendMode;
		TOptional<FFalloffParams> FalloffOverrides;
		TOptional<double> HeightMinMaxBlendDistance;
	};

	struct FProjectOntoMeshOptionalParams
	{
		//~ Need default constructor to deal with compiler bug that complains about using nested classes with
		//~  default initializers and an implicit constructor as a default argument.
		FProjectOntoMeshOptionalParams(){}

		MeshPartition::EProjectModifierBlendMode BlendMode = MeshPartition::EProjectModifierBlendMode::Set;
		double BlendSoftnessInProjectionSpace = 0;
		FFalloffParams FalloffSettings;

		TArray<FWeightEntryParams> WeightEntries;
	};

	static UE_API void ProjectOntoMesh(MeshPartition::FMeshView& MeshView,
		const FTransform3d& MegameshTransform,
		const FDynamicMesh3& ProjectionMesh, const FTransform& ProjectionMeshToWorld,
		const Geometry::FDynamicMeshAABBTree3& ProjectionMeshSpatial,
		const FTransform& ProjectionTransform, const Geometry::FAxisAlignedBox3d& ProjectionSpaceBounds,
		const FProjectOntoMeshOptionalParams& OptionalParams = FProjectOntoMeshOptionalParams());
private:

	UE_API void GetProjectionTransformAndBounds(FTransform& ProjectionTransformOut,
		Geometry::FAxisAlignedBox3d& ProjectionSpaceBoundsOut) const;

	UE_API void OnCurveChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType);
	UE_API void AttachCurveListeners();
	UE_API void DetachCurveListeners();

	/**
	* How far up above the projection mesh the modifier affects megamesh vertices.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0"), AdvancedDisplay)
	float VerticalExtentUp = 10000.f;

	/**
	* How far down below the projection mesh the modifier affects megamesh vertices.
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0"), AdvancedDisplay)
	float VerticalExtentDown = 10000.f;

	/**
	 * When true, the projection direction is specified relative to the mesh (whose transform is the component transform).
	 * When false, the the projection direction is specified in world space, not tied to the mesh transform.
	 */
	UPROPERTY(EditAnywhere, Category = Settings, meta = (DisplayName = "Direction Follows Mesh"))
	bool bUseRelativeProjectionDirection = true;

	UPROPERTY(EditAnywhere, Category = Settings, Meta = (EditCondition = "!bUseRelativeProjectionDirection", EditConditionHides))
	FRotator AbsoluteProjectionDirection;

	UPROPERTY(EditAnywhere, Category = Settings, Meta = (EditCondition = "bUseRelativeProjectionDirection", EditConditionHides))
	FRotator RelativeProjectionDirection;

	UPROPERTY(EditAnywhere, Category = Settings)
	MeshPartition::EProjectModifierBlendMode BlendMode = MeshPartition::EProjectModifierBlendMode::Set;

	/**
	* When using Raise or Lower blend modes, smooths the displacement in areas where the existing mesh is close 
	*  to the projection mesh. Specifically, it does a smooth min/max in areas where the projection mesh is withing 
	*  BlendSoftness distance of the affected mesh. 
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (ClampMin = "0.0", UIMax = "1000", EditConditionHides,
		EditCondition = "BlendMode != EProjectModifierBlendMode::Set"))
	double BlendSoftness = 0.0;
	
	UPROPERTY(EditAnywhere, Category = Settings)
	MeshPartition::FProjectModifierFalloffSettings HeightFalloff;

	UPROPERTY(EditAnywhere, Category = Settings, Meta = (ShowOnlyInnerProperties))
	TArray<FProjectModifierWeightEntry> WeightChannels;

	UPROPERTY(EditAnywhere, Category = Visualize, AdvancedDisplay)
	bool bDrawLocalBounds = true;

	/**
	* Draws a rectangle cross section of the affected box at the component location. Note that the size is
	*  determined by the size of the mesh, so this is not shown if the mesh is not set.
	*/
	UPROPERTY(EditAnywhere, Category = Visualize, AdvancedDisplay)
	bool bDrawProjectionRectangle = true;
};
} // namespace UE::MeshPartition

#undef UE_API
