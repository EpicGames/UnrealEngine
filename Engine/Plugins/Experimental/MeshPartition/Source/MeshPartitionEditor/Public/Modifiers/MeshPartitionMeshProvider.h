// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "MeshPartitionModifierComponent.h"

#include "MeshPartitionMeshProvider.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UDynamicMeshComponent;
class UTexture;

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

namespace UE::MeshPartition
{
class AMeshPartition;

/**
* This is a Base modifier aimed to be used on a MegaMesh.
* MeshPartition::UMeshProviderModifier is holding a dynamic mesh which is intended to be used as a first layer.
*/
UCLASS(MinimalAPI, meta = (BlueprintSpawnableComponent, MegaMeshClassVersion = "1"))
class UMeshProviderModifier : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()

public:
	UE_API UMeshProviderModifier();

	//~ TODO: Uncertain whether this direct accessor should exist- DynamicMeshComponent is intended to be
	//~  accessed via ProcessMesh and EditMesh, but we could use a subclass that is fine with it.
	/** Gets the internal mesh. */
	UE_API const FDynamicMesh3* GetMesh() const;

	/**
	 * Sets the internal mesh.
	 * @param bEmitChange If true, an undo/redo transaction is issued for the operation.
	 */
	UE_API void SetMesh(FDynamicMesh3&& InDynamicMesh, bool bEmitChange = false);

	// Gives a delegate that is fired if the base section's preview section is reassigned (due to a
	//  rebuild). This is useful, for example, if you want to continue suppressing the visibility of
	//  a base section's output.
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnPreviewSectionReassignment, UMeshProviderModifier*, MeshPartition::APreviewSection* Section);
	FOnPreviewSectionReassignment& OnPreviewSectionReassignment() { return OnPreviewSectionReassignmentDelegate; }

	// UActorComponent Implementation
	UE_API virtual void OnRegister() override;
	// End UActorComponent Implementation

	// UObject, needed to fixup missing RF_Transactional flags
	UE_API virtual void PostLoad() override;

	// MeshPartition::UModifierComponent Implementation
	UE_API virtual void InitializeModifier() override;
	UE_API virtual void UninitializeModifier() override;
	UE_API virtual void SetPreviewSection(MeshPartition::APreviewSection* InPreviewSection) override;
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual double GetComplexity() const override;
	UE_API virtual void SetIsTemporarilyHiddenInEditor(const bool bInIsHidden) override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	virtual bool IsBase() const override { return true; }
	// End MeshPartition::UModifierComponent Implementation
	
	/** Used to forward SetMaterial to the internal dynamic mesh. */
	UE_API void SetMaterial(int32 InElementIndex, UMaterialInterface* InMaterial);

protected:
	/**	Called when the internal dynamic mesh changed. Will call MeshPartition::UModifierComponent::OnChanged. */
	UE_API void OnDynamicMeshChanged();
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

private:
	UPROPERTY()
	TObjectPtr<UDynamicMeshComponent> MeshComponent;

	mutable TSharedPtr<const FDynamicMesh3> MeshCopyForBackgroundOps;

	//todo(luc.eygasier): investigate why OnMeshChanged event is always fired twice.
	uint64 LastFrameChanged = 0;

	FOnPreviewSectionReassignment OnPreviewSectionReassignmentDelegate;

	// Currently the tool target gets created out of the internal MeshComponent, which we may not want to expose yet.
	friend class USectionToolTargetFactory;
};
} // namespace UE::MeshPartition

#undef UE_API
