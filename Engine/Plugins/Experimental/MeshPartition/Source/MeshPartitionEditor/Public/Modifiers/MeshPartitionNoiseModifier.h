// Copyright Epic Games, Inc. All Rights Reserved.
// 

#pragma once

#include "ProceduralNoise.h"
#include "CoreMinimal.h"
#include "MeshPartitionChannel.h" // MeshPartition::FChannelName
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionNoiseModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API


namespace UE::MeshPartition
{
UENUM()
enum class EFBMMode : uint8
{
	/**
	* Classic Perlin Noise
	*/
	Standard,

	/**
	* Turbulent Modifier, creating upward bumps.
	*/
	Turbulent,

	/**
	* Ridge Modifier, creating sharper creases and ridges.
	*/
	Ridge
};

UENUM()
enum class ENoiseModifierType : uint8
{
	/**
	* Fractal Brownian Motion based on Perlin Noise
	*/
	FBmNoise UMETA(DisplayName = "fBM Noise"),

	/**
	* Move vertices in spatial sine wave pattern
	*/
	SineWave UMETA(DisplayName = "Sine Wave")
};

UENUM()
enum class ENoiseParameterization : uint8
{
	/** Use World XY coordinates. 
	*/
	World UMETA(DisplayName = "World Space"),

	/** Use Patch XY coordinates. 
	*/
	Patch UMETA(DisplayName = "Patch Space")
};

/**
* A simple modifier that will deform a MegaMesh by pulling all vertices within `Radius` units to the same Z position of the patch component.
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Noise", "NoiseTransform", "Channels"), meta = (BlueprintSpawnableComponent))
class UNoiseModifier : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()

public:
	UE_API UNoiseModifier();

	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;

	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;

private:
	Geometry::FAxisAlignedBox3d GetLocalCoverage() const
	{
		return Geometry::FAxisAlignedBox3d(-UnscaledCoverage / 2., UnscaledCoverage / 2.);
	}

	// XY-Region in which the modifier applies noise, centered at modifier location. It will be transformed by the component transform.
	UPROPERTY(EditAnywhere, Category = NoiseTransform, meta = (UIMin = "0", ClampMin = "0", AllowPreserveRatio = true))
	FVector3d UnscaledCoverage = FVector3d(2000., 2000., 50000.);
	
	// Determines which coordinates to feed into the noise function. 
	UPROPERTY(EditAnywhere, Category = NoiseTransform, meta = (DisplayName = "Parametrization"))
	MeshPartition::ENoiseParameterization ParametrizationType = MeshPartition::ENoiseParameterization::World;

	// 2D Translation applied to noise coordinates.
	UPROPERTY(EditAnywhere, Category = NoiseTransform)
	FVector2D NoiseTranslate{ 0., 0. };

	// 2D Frequency multiplier applied to noise coordinates.
	UPROPERTY(EditAnywhere, Category = NoiseTransform, meta = (AllowPreserveRatio = true))
	FVector2D NoiseFrequency{ 0.001, 0.001 };

	// 2D Rotation applied to noise coordinates.
	UPROPERTY(EditAnywhere, Category = NoiseTransform, meta = (UIMin = "0.0", UIMax = "360.0"))
	double NoiseRotation{ 0. };

	// Displacement type 
	UPROPERTY(EditAnywhere, Category = Noise, meta = (DisplayName = "Noise Type"))
	MeshPartition::ENoiseModifierType DisplacementType = MeshPartition::ENoiseModifierType::FBmNoise;
	
	// Noise modification control.
	UPROPERTY(EditAnywhere, Category = Noise, meta = (EditConditionHides, EditCondition = "DisplacementType == ENoiseModifierType::FBmNoise"))
	MeshPartition::EFBMMode FBMMode { MeshPartition::EFBMMode::Standard };

	// Amplitude of the noise function.
	UPROPERTY(EditAnywhere, Category = Noise)
	double Intensity { 1. };

	// Controls the amount of smooth falloff towards the boundary of the control region.
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "0", UIMax = "1", ClampMin = "0", ClampMax = "1"))
	double Falloff { 0.5 };

	// FBM-specific properties

	// Number of octaves.
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "1", UIMax = "8", ClampMin = "1", ClampMax = "16",
		EditConditionHides, EditCondition = "DisplacementType == ENoiseModifierType::FBmNoise"))
	int FBMOctaves{ 4 };

	// Multplier applied to frequency when going to finer octave.
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "1.0", ClampMin="1.0", 
		EditConditionHides, EditCondition = "DisplacementType == ENoiseModifierType::FBmNoise"))
	double FBMLacunarity{ 2. };

	// Multplier applied to amplitude when going to finer octave.
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "0.1", UIMax = "1.0", ClampMin = "0.1", ClampMax = "1.0",
		EditConditionHides, EditCondition = "DisplacementType == ENoiseModifierType::FBmNoise"))
	double FBMGain{ 0.5 };

	// Smoothness control applied to Turbulent and Ridge modes.
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0",
		EditConditionHides, EditCondition = "DisplacementType == ENoiseModifierType::FBmNoise"))
	double FBMSmoothness{ 0.02 };

	// Gamma control applied to Turbulent and Ridge modes.
	UPROPERTY(EditAnywhere, Category = Noise, meta = (UIMin = "0.5", UIMax = "4.0", ClampMin = "0.2", ClampMax = "16.0",
		EditConditionHides, EditCondition = "DisplacementType == ENoiseModifierType::FBmNoise"))
	double FBMGamma{ 2.0 };

	UPROPERTY(EditAnywhere, Category=Channels)
	bool bWriteToWeightChannel = false;

	UPROPERTY(EditAnywhere, Category=Channels, meta = (GetOptions = "GetMegaMeshDefinitionChannels", 
		NoResetToDefault, EditCondition = "bWriteToWeightChannel"))
	MeshPartition::FChannelName WeightChannelName;
	
	// When true, draws a red rectangle showing patch area in editor.
	UPROPERTY(EditAnywhere, Category = Visualization, AdvancedDisplay)
	bool bDrawPatchRectangle = true;

	// When true, draws a box showing affected volume in editor.
	UPROPERTY(EditAnywhere, Category = Visualization, AdvancedDisplay)
	bool bDrawAffectedBox = true;
	
};
} // namespace UE::MeshPartition

#undef UE_API
