// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionChannel.h" // MeshPartition::FChannelName
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionPatchModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UWaterBodyComponent;
class UWaterSplineComponent;
class AWaterBody;

namespace UE::MeshPartition
{
namespace MegaMeshPatchModifierLocals
{
	class FBackgroundOp;
}

/**
* A simple modifier that will deform a MegaMesh by pulling all vertices within `Radius` units to the same Z position of the patch component.
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Patch"), meta=(BlueprintSpawnableComponent, MegaMeshClassVersion = "1"))
class UPatchModifier : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()
	
public:
	UE_API UPatchModifier();

	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;

	// End MeshPartition::UModifierComponent Implementation
	
	UE_API void SetRadius(float InRadius);
	UE_API void SetFalloff(float InFalloff);
	UE_API void SetMaxZDistance(float InMaxZDistance);
	UE_API void SetWriteToWeightChannel(bool bInWriteToWeightChannel);
	UE_API void SetWeightChannelName(FName InWeightChannelName);

	float GetRadius() const { return Radius; }
	float GetFalloff() const { return Falloff; }
	float GetMaxZDistance() const { return MaxZDistance; }
	bool ShouldWriteToWeightChannel() const { return bWriteToWeightChannel; }
	FName GetWeightChannelName() const { return WeightChannelName.GetName(); }

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;
	
protected:
	// Helper struct to package settings that are common accross MeshPartition::UPatchModifier and MeshPartition::UInstancedPatchModifier
	struct FSettings
	{
		FSettings() {};
		FSettings(const UPatchModifier& Patch)
			: Radius(Patch.Radius)
			, Falloff(Patch.Falloff)
			, MaxZDistance(Patch.MaxZDistance)
			, bWriteToWeightChannel(Patch.bWriteToWeightChannel)
			, WeightChannelName(Patch.WeightChannelName.GetName())
		{}

		float Radius;
		float Falloff;
		float MaxZDistance;
		bool bWriteToWeightChannel;
		FName WeightChannelName;
	};
	static UE_API void ApplyDeformation(const FSettings& Settings, MeshPartition::FMeshView& InMeshView, const FVector& Location);
	friend class MegaMeshPatchModifierLocals::FBackgroundOp;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

	/** Radius around the patch component's location that is affected by the patch */
	UPROPERTY(EditAnywhere, Category=Patch)
	float Radius = 1000.f;

	UPROPERTY(EditAnywhere, Category=Patch)
	float Falloff = 1000.f;
	
	UPROPERTY(EditAnywhere, Category=Patch)
	float MaxZDistance = 20000.f;

	UPROPERTY(EditAnywhere, Category=Patch)
	bool bWriteToWeightChannel = false;

	/** When bWriteToWeightChannel is true, the channel to write to. */
	UPROPERTY(EditAnywhere, Category = Patch, Meta = (EditCondition = "bWriteToWeightChannel", GetOptions = "GetMegaMeshDefinitionChannels"))
	MeshPartition::FChannelName WeightChannelName;
};
} // namespace UE::MeshPartition

#undef UE_API
