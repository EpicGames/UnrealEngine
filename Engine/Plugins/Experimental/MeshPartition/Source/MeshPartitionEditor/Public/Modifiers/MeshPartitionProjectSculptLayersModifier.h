// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Math/MathFwd.h"
#include "MeshPartitionChannel.h" // MeshPartition::FChannelName
#include "Modifiers/CodeReusableMeshPartitionModifierInterface.h"
#include "Modifiers/MeshPartitionEditableModifierBase.h"
#include "MeshSculptLayersManagerAPI.h"
#include "UDynamicMesh.h"

#include "MeshPartitionModifierUtils.h"
// UProjectSculpLayersModifier should become a MeshBasedModifier
#include "Modifiers/MeshPartitionMeshBasedModifierBase.h"

#include "MeshPartitionProjectSculptLayersModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UDynamicMesh;

namespace UE::MeshPartition
{
UENUM(BlueprintType)
enum class ESculptLayerProjectMethod : uint8
{
	// Transfer sculpt layers by projection along a fixed direction
	FixedDirection,
	// Transfer sculpt layers from the closest point on the reference mesh
	ClosestPoints
};

UENUM()
enum class ESculptLayerSetWeightChannelMethod : uint8
{
	Disabled,
	Set,
	Add
};

USTRUCT()
struct FSculptLayerModifierWeightAttributeEntry
{
	GENERATED_BODY()

	/** The destination attribute channel to write. */
	UPROPERTY(EditAnywhere, Category = WeightAttribute, meta = (GetOptions = "GetMegaMeshDefinitionChannels"))
	MeshPartition::FChannelName ChannelName;

	/**
	* How to apply weight channel values.
	* Note that for an internal channel, Set and Add are equivalent.
	*/
	UPROPERTY(EditAnywhere, Category = WeightAttribute)
	ESculptLayerSetWeightChannelMethod Method = ESculptLayerSetWeightChannelMethod::Set;
	
	/**
	* Scale the effect of the modifier on this channel. 
	* Note that for an internal channel, this directly scales the channel values.
	*/
	UPROPERTY(EditAnywhere, Category = WeightAttribute)
	float BlendWeight = 1.0f;

	/** Whether to use the channel weights to scale the displacement applied by sculpt layers */
	UPROPERTY(EditAnywhere, Category = WeightAttribute)
	bool bScaleSculptLayers = false;

	/** Whether to clamp the channel weights to the zero-to-one range */
	UPROPERTY(EditAnywhere, Category = WeightAttribute)
	bool bClampRange = false;

	/** 
	* Whether to isolate this weight channel to this modifier -- and not read/write from the target mesh.
	* For example, use this to define a channel that is solely used for scaling sculpt layers on this modifier.
	*/
	UPROPERTY(EditAnywhere, Category = WeightAttribute, AdvancedDisplay)
	bool bInternalChannel = false;
};

/**
* Modifier to manage layers of custom mesh detail, and project them onto a partitioned mesh
*/
UCLASS(MinimalAPI, PrioritizeCategories=("Modifier","MeshLayers","AttributeWeights","Projection"), meta = (BlueprintSpawnableComponent, DisplayName = "BrushModifier", MegaMeshClassVersion = "1"))
class UProjectMeshLayersModifier : public MeshPartition::UEditableModifierBase
	, public IMeshSculptLayersManager
	, public ICodeReusableModifier
{
	GENERATED_BODY()

public:

	UE_API UProjectMeshLayersModifier();

	// ICodeReusableModifier
	UE_API virtual void SetDisabledByCode(bool bDisabledIn) override;
	UE_API virtual void ResetForReuse() override;
	UE_API virtual bool IsUsed() const override;

	// MeshPartition::UModifierComponent implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	UE_API virtual void InitializeModifier() override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;
	UE_API virtual bool IsTemporarilyDisabledInEditor() const override;
	UE_API virtual UE::Tasks::FTask GetAsyncPrepareResourcesTask() const override;

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;
	UE_API virtual void PostLoad() override; // called after load
	UE_API virtual void PostEditImport() override; // called after duplicate/copy

	// MeshPartition::UEditableModifierBase implementation
	UE_API virtual void ApplyEditWithMesh(const FDynamicMesh3& UpdatedMesh) override;
	UE_API virtual void PrepareForEdit(FDynamicMesh3& EditMesh) const override;
	UE_API virtual TArray<Geometry::FOrientedBox3d> GetBoundsForEdit() const override;

	// IMeshSculptLayersManager implementation
	virtual bool HasSculptLayers() const override
	{
		return true;
	}
	virtual int32 NumLockedBaseSculptLayers() const override
	{
		return 1;
	}

	// Call to directly set the sculpt layer mesh. Cached/dependent data will be automatically updated as well.
	UE_API void SetSculptLayerMesh(const FDynamicMesh3& Mesh, bool bTransact);

	TArray<FSculptLayerModifierWeightAttributeEntry> GetChannelEntries() const { return AttributeWeightChannels; }

	// Directly set the channels to write
	void SetChannelEntries(const TArray<FSculptLayerModifierWeightAttributeEntry>& EntriesIn) { AttributeWeightChannels = EntriesIn; }

	/** Set projection method to closest point, and sets the max distance */
	void SetClosestPointProjection(double MaxDistance)
	{
		ProjectMethod = ESculptLayerProjectMethod::ClosestPoints;
		MaxClosestPointDistance = MaxDistance;
	}
	//~ TODO: add a setter for other projection method once we use it. Not yet done because uncertain
	//~  whether the "extent down/up" parameterization is most convenient...

	void SetEditVolumeExtents(const FVector3d& EditExtentsIn) { EditVolumeExtents = EditExtentsIn; }
	
	/** Return the name of the sculpt layer at the provided index */
	UE_API FName GetLayerName(const int32 LayerIndex) const;

	/** Sets the name of the layer at the provided index to the provided name */
	UE_API void SetLayerName(const int32 InLayerIndex, const FName InName);
	
	/** Sets the weight of the layer at the provided index to the provided weight */
	UE_API void SetLayerWeight(const int32 InLayerIndex, const double InWeight, const EPropertyChangeType::Type ChangeType);

	// Gets the ActiveLayer
	int32 GetActiveLayer() const { return ActiveLayer; }
	
	DECLARE_MULTICAST_DELEGATE(OnMeshLayersModifierPanelRequestRebuild);
	OnMeshLayersModifierPanelRequestRebuild OnMeshLayersPanelRequestRebuild;

private:

	/**
	* Clear the sculpt mesh entirely
	*/
	UFUNCTION(CallInEditor, Category = Mesh)
	UE_API void ClearMesh();

	// Get the transform of the projection operator
	UE_API FTransform GetProjectionTransform() const;
	// Get the bounds of the projection operator, in projection space, from InputBounds in mesh space
	UE_API Geometry::FAxisAlignedBox3d GetProjectionBounds(const Geometry::FAxisAlignedBox3d& InputBounds, const FTransform& InProjectionTransform) const;
	// Get the bounds of the current edit extents, in projection space
	UE_API Geometry::FAxisAlignedBox3d GetProjectionEditBounds() const;
	// Called whenever internal MeshObject is modified, fires OnMeshChanged and OnMeshVerticesChanged above
	UE_API void OnMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo);
	// Build the AABB tree and the mesh copy for the current sculpt mesh
	UE_API void BuildCachedData();
	// Find the maximum offset values in each layer, to help quickly compute bounding boxes
	UE_API void ComputeMaxLayerOffsets();
	// Get the number of sculpt layers on the current sculpt mesh
	UE_API int32 GetProjectionMeshNumLayers() const;

	// UActorComponent override to rebuild cached data as needed -- for example, across e.g. blueprint re-running construction scripts
	UE_API virtual void OnRegister() override;
	
private:

	UE_API void ApplyMeshUpdate();

	// Size of the region to project and sculpt 
	UPROPERTY(EditAnywhere, Category = MeshLayers, meta = (ClampMin = "0", DisplayName = "Edit Projected Extents", EditCondition = "ProjectMethod == ESculptLayerProjectMethod::FixedDirection", EditConditionHides))
	FVector2D EditExtents = FVector2D(500, 500);

	// Size of the region to sculpt
	UPROPERTY(EditAnywhere, Category = MeshLayers, meta = (ClampMin = "0", EditCondition = "ProjectMethod == ESculptLayerProjectMethod::ClosestPoints", EditConditionHides))
	FVector EditVolumeExtents = FVector(500, 500, 500);

	// If enabled, unsculpted triangles will be discarded from the projection mesh, which can make the sculpts offsets faster to apply.
	// Note: Disabling this can help block undesired projections of the sculpt offsets on some finely-detailed or layered surfaces.
	// Note: Currently not supported if using AttributeWeightChannels -- no triangles will be discarded in this case.
	UPROPERTY(EditAnywhere, Category = MeshLayers)
	bool bDiscardUnsculpted = false;

	// Last layer edited/selected in tools (where 1 is the first sculpt layer, since 0 is the base)
	// Note this is saved for persistence across edits in tools, and does not affect the modifier result
	UPROPERTY()
	int32 ActiveLayer = 1;

public:
	// Weights to apply to each sculpt layer. Note the base layer is not included.
	UPROPERTY(EditAnywhere, EditFixedSize, NoClear, Category = MeshLayers, meta = (NoResetToDefault, UIMin = "-1", UIMax = "2"))
	TArray<double> LayerWeights;

private:

	// Helper to get the 'active' weight channels that will actually be used by the modifier (e.g., w/ disabled channels filtered out)
	UE_API TArray<FSculptLayerModifierWeightAttributeEntry> GetActiveWeightChannels() const;

	// Whether to transform sculpted vertices directly to corresponding positions of the reference sculpt mesh. Otherwise, only the offsets will be transferred.
	// Note: Absolute positioning can be blended away by using Attribute Weight Channels with 'Scale Sculpt Layers' enabled
	UPROPERTY(EditAnywhere, Category = MeshLayers)
	bool bSculptAbsolutePositions = false;

	// Optionally specify attribute weight channels to be transferred
	UPROPERTY(EditAnywhere, Category = AttributeWeights, Meta = (ShowOnlyInnerProperties))
	TArray<FSculptLayerModifierWeightAttributeEntry> AttributeWeightChannels;

	// Return a view of the active layer weights -- i.e., SourceLayerWeights if bUseSourceLayerWeights, or LayerWeights otherwise
	UE_API TConstArrayView<double> GetActiveLayerWeights() const;

	// Method to transfer offsets to the mesh partition mesh
	UPROPERTY(EditAnywhere, Category = Projection)
	ESculptLayerProjectMethod ProjectMethod = ESculptLayerProjectMethod::FixedDirection;

	// Farthest distance to search for a closest point on the reference mesh
	UPROPERTY(EditAnywhere, Category = Projection, meta = (EditCondition = "ProjectMethod == ESculptLayerProjectMethod::ClosestPoints", EditConditionHides))
	float MaxClosestPointDistance = 100;

	// Whether to apply a smooth falloff of influence for vertices where the closest point is on a mesh boundary
	UPROPERTY(EditAnywhere, Category = Projection, meta = (EditCondition = "ProjectMethod == ESculptLayerProjectMethod::ClosestPoints", EditConditionHides))
	bool bUseBoundaryFalloff = true;

	// The distance over which a smooth falloff is applied for vertices that map to boundary edges of the reference mesh, as a factor of Max Closest Point Distance.
	// A value of 1 will result in a smooth falloff over the full Max Closest Point Distance range, and a value of 0 will result in offsets clamping to 0 for all vertices that map to the boundary.
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0", UIMax = "1", EditCondition = "ProjectMethod == ESculptLayerProjectMethod::ClosestPoints && bUseBoundaryFalloff", EditConditionHides))
	float BoundaryFalloffDistanceFactor = 1;

	// How far up above the projection mesh the modifier affects megamesh vertices, relative to the direction of projection.	
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0", EditCondition = "ProjectMethod == ESculptLayerProjectMethod::FixedDirection", EditConditionHides), AdvancedDisplay)
	float VerticalExtentUp = 10000.f;

	//How far down below the projection mesh the modifier affects megamesh vertices, relative to the direction of projection.
	UPROPERTY(EditAnywhere, Category = Projection, meta = (ClampMin = "0", EditCondition = "ProjectMethod == ESculptLayerProjectMethod::FixedDirection", EditConditionHides), AdvancedDisplay)
	float VerticalExtentDown = 10000.f;

	// Whether to draw the bounds of the projection
	UPROPERTY(EditAnywhere, Category = Visualize, AdvancedDisplay)
	bool bDrawAffectedBox = true;

	// Whether to draw the bounds of the source geometry that sculpt layers are transferred from (not including sculpt offsets)
	UPROPERTY(EditAnywhere, Category = Visualize, AdvancedDisplay)	
	bool bDrawSculptSourceBounds = true;

	// Draws edges of the stored mesh for debugging
	UPROPERTY(EditAnywhere, Category = Visualize, AdvancedDisplay)
	bool bDrawDebugMesh = false;

	// Cache of maximum offset per layer, across all modifier meshes, used to conservatively predict how much the bounds could be expanded by the modifier
	TArray<FVector3d> MaxLayerOffset;

	// The mesh that holds the current sculpt layers
	UPROPERTY()
	TObjectPtr<UDynamicMesh> MeshObject;

	// Handle for OnMeshObjectChanged which is registered with MeshObject::OnMeshChanged delegate
	FDelegateHandle MeshObjectChangedHandle;

	TSharedPtr<FAsyncMeshInstanceData> MeshInstance;

	// See ICodeReusableModifier::SetDisabledByCode
	bool bDisabledByCode = false;
};
} // namespace UE::MeshPartition
DECLARE_LOG_CATEGORY_EXTERN(LogSculptLayersModifier, Warning, All);

#undef UE_API
