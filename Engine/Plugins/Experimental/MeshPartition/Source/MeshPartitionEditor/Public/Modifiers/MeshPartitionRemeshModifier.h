// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "CleaningOps/RemeshMeshOp.h" // ERemeshSmoothingType
#include "MeshPartitionModifierComponent.h"
#include "VectorTypes.h"
#include "Modifiers/MeshPartitionRemeshModifierTypes.h"
#include "MeshPartitionChannel.h"

#include "MeshPartitionRemeshModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UWaterBodyComponent;
class UWaterSplineComponent;
class AWaterBody;

namespace UE::MeshPartition
{
UENUM()
enum class ERemeshModifierOperation : uint8
{
	Remesh,
	Tessellate
};

/*
* Base class for modifiers that do Remeshing/Tessellation
*/
UCLASS(MinimalAPI, Abstract)
class URemeshModifierBase : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()
	
public:

	URemeshModifierBase() = default;
	virtual ~URemeshModifierBase() = default;

public:

	//
	// General Operation
	//

	ERemeshModifierOperation GetCurrentOperation() const { return CurrentOperation; }
	UE_API void SetCurrentOperation(const ERemeshModifierOperation InOperation);

	//
	// Remesh Properties
	//

	EMegaMeshRemeshModifierBoundaryMode GetBoundaryMode() const { return BoundaryMode; }
	UE_API void SetBoundaryMode(const EMegaMeshRemeshModifierBoundaryMode InMode);

	bool GetDisallowUnsafeBoundaryEdits() const { return bDisallowUnsafeBoundaryEdits; }
	UE_API void SetDisallowUnsafeBoundaryEdits(const bool bDisallow);

	float GetTargetEdgeLength() const { return TargetEdgeLength; }
	UE_API void SetTargetEdgeLength(const float InLength);

	int32 GetRemeshIterations() const { return RemeshIterations; }
	UE_API void SetRemeshIterations(const int32 InIterations);

	float GetSmoothingStrength() const { return SmoothingStrength; }
	UE_API void SetSmoothingStrength(const float InStrength);

	ERemeshSmoothingType GetSmoothingType() const { return SmoothingType; }
	UE_API void SetSmoothingType(const ERemeshSmoothingType InType);

	bool GetPreserveNormalSeams() const { return bPreserveNormalSeams; }
	UE_API void SetPreserveNormalSeams(const bool bPreserve);

	float GetSharpEdgeAngleThreshold() const { return SharpEdgeAngleThreshold; }
	UE_API void SetSharpEdgeAngleThreshold(const float InThreshold);

	bool GetProjectToInputMesh() const { return bProjectToInputMesh; }
	UE_API void SetProjectToInputMesh(const bool bProject);

	//
	// Tesselation Properties
	//

	EMegaMeshRemeshModifierTessellateMethod GetTessellationMethod() const { return TessellationMethod; }
	UE_API void SetTessellationMethod(const EMegaMeshRemeshModifierTessellateMethod InMethod);

	bool GetUseTargetEdgeLength() const { return bUseTargetEdgeLength; }
	UE_API void SetUseTargetEdgeLength(const bool bUse);

	float GetTessellationTargetEdgeLength() const { return TessellationTargetEdgeLength; }
	UE_API void SetTessellationTargetEdgeLength(const float InLength);

	int32 GetTessellationLevel() const { return TessellationLevel; }
	UE_API void SetTessellationLevel(const int32 InLevel);

	int32 GetMaxTessellationLevel() const { return MaxTessellationLevel; }
	UE_API void SetMaxTessellationLevel(const int32 InMaxLevel);

	int32 GetPostProcessingIterations() const { return PostProcessingIterations; }
	UE_API void SetPostProcessingIterations(const int32 InIterations);

	bool GetVertexSmoothing() const { return bVertexSmoothing; }
	UE_API void SetVertexSmoothing(const bool bSmoothing);

	float GetTessellateSmoothingStrength() const { return TessellateSmoothingStrength; }
	UE_API void SetTessellateSmoothingStrength(const float InStrength);

	bool GetResampleUVs() const { return bResampleUVs; }
	UE_API void SetResampleUVs(const bool bResample);

	bool GetEdgeFlips() const { return bEdgeFlips; }
	UE_API void SetEdgeFlips(const bool bFlips);

	//
	// Common Properties
	//

	bool GetUseDensityWeightChannel() const { return bUseDensityWeightChannel; }
	UE_API void SetUseDensityWeightChannel(const bool bUse);

	const FChannelName& GetDensityWeightChannelName() const { return DensityWeightChannelName; }
	UE_API void SetDensityWeightChannelName(const FChannelName& InName);

	float GetRelativeDensity() const { return RelativeDensity; }
	UE_API void SetRelativeDensity(const float InDensity);

protected:

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

	UPROPERTY(EditAnywhere, Category = "RemeshOperation", meta = (DisplayName = "Mode"))
	MeshPartition::ERemeshModifierOperation CurrentOperation = MeshPartition::ERemeshModifierOperation::Remesh;

	//
	// Remesh
	//

	/**
	* How to treat edges on the boundary of the covered region.
	*/
	UPROPERTY(EditAnywhere, Category = Remesh, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh"))
	EMegaMeshRemeshModifierBoundaryMode BoundaryMode = EMegaMeshRemeshModifierBoundaryMode::SplitOnly;

	/** Desired average mesh edge length */
	UPROPERTY(EditAnywhere, Category=Remesh, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh", UIMin = "0.1", ClampMin = "0.1"))
	float TargetEdgeLength = 1000.f;

	/** Number of remeshing passes over the whole mesh */
	UPROPERTY(EditAnywhere, Category=Remesh, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh"))
	int32 RemeshIterations = 15;
	
	/** How aggressively to move vertices during smoothing */
	UPROPERTY(EditAnywhere, Category = Remesh, meta = (DisplayName = "Smoothing Rate", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", EditCondition = "(CurrentOperation == ERemeshModifierOperation::Remesh)"))
	float SmoothingStrength = 0.25f;

	/** Vertex smoothing scheme */
	UPROPERTY(EditAnywhere, Category = Remesh, meta = (DisplayName = "Smoothing Mode", EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh"))
	ERemeshSmoothingType SmoothingType = ERemeshSmoothingType::Uniform;

	/** If true, precompute vertex normals over the mesh and mark normal discontinuities as constraints. This can help preserve sharp features of the mesh */
	UPROPERTY(EditAnywhere, Category = Remesh, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh"))
	bool bPreserveNormalSeams = false;

	/** Threshold on angle of change in face normals across an edge, above which we create a sharp edge if bPreserveNormalSeams is true */
	UPROPERTY(EditAnywhere, Category = Remesh, meta = (DisplayName = "Sharp Angle Threshold", Units = Degrees, UIMin = "0.0", UIMax = "180.0", ClampMin = "0.0", ClampMax = "180.0", EditCondition = "bPreserveNormalSeams && CurrentOperation == ERemeshModifierOperation::Remesh"))
	float SharpEdgeAngleThreshold = 60.0f;

	/** If true, vertices will be projected back to the input surface after each remeshing pass */
	UPROPERTY(EditAnywhere, Category = Remesh, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh"))
	bool bProjectToInputMesh = false;

	/**
	* When true, the region affected by the modifier is contracted such that any edges that have neighbors outside
	*  the view that the modifier is given are not modified. This ensures that the view can be stitched back into
	*  the larger mesh properly (otherwise, potentially invisible cracks would appear in cases where a vertex was
	*  added to the boundary of the view but not the base mesh).
	*/
	UPROPERTY(EditAnywhere, Category = Remesh, AdvancedDisplay, Meta = (EditCondition = "(BoundaryMode != EMegaMeshRemeshModifierBoundaryMode::FullyConstrained) && (CurrentOperation == ERemeshModifierOperation::Remesh)", DisplayName = "Disallow Unsafe Edits"))
	bool bDisallowUnsafeBoundaryEdits = true;

	/**
	* When true, the region affected by the modifier is contracted such that no triangles outside coverage are split.
	*  when false, some triangles that lie partially outside of the coverage but still inside the view (if the coverage
	*  is not aligned to view) are allowed to be split to allow for splits in the boundary of the fully contained region.
	* This can safely remain false to maximize the remeshed region as long as you accept some partially-covered triangles
	*  being modified.
	*/
	UPROPERTY(EditAnywhere, Category = Remesh, AdvancedDisplay, Meta = (EditCondition = "(BoundaryMode != EMegaMeshRemeshModifierBoundaryMode::FullyConstrained) && (CurrentOperation == ERemeshModifierOperation::Remesh)", DisplayName = "Disallow Edits Outside"))
	bool bDisallowSafeEditsOutsideCoverage = false;

	//
	// Tessellation
	//

	/** Type of tessellation to perform */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate"))
	EMegaMeshRemeshModifierTessellateMethod TessellationMethod = EMegaMeshRemeshModifierTessellateMethod::AdaptiveRegular;

	/** If true, subdivide each triangle until its longest edge is shorter than TargetEdgeLength */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate"))
	bool bUseTargetEdgeLength = false;

	/** Edge length threshold for tessellation */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate && bUseTargetEdgeLength", UIMin = "0.1", ClampMin = "0.1"))
	float TessellationTargetEdgeLength = 1000.f;

	/** Number of times to subdivide each triangle (possibly excluding triangles close to the region boundary) */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "!bUseTargetEdgeLength && (CurrentOperation == ERemeshModifierOperation::Tessellate)", UIMin = 0, UIMax = 10, ClampMin = 0))
	int32 TessellationLevel = 1;

	/** When using the target edge length, limit the tessellation level per triangle to this value to avoid excessive computation */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "bUseTargetEdgeLength && (CurrentOperation == ERemeshModifierOperation::Tessellate)", UIMin = 0, UIMax = 10, ClampMin = 0))
	int32 MaxTessellationLevel = 1;

	/** Number of  passes of mesh optimization to perform after tessellation is finished */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate", UIMin = 0, UIMax = 40, ClampMin = 0))
	int32 PostProcessingIterations = 0;

	/** Whether to apply vertex smoothing during post-processing */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate"))
	bool bVertexSmoothing = false;

	/** How aggressively to move vertices during smoothing */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate", UIMin = 0, UIMax = 1.0, ClampMin = 0, ClampMax = 1.0))
	float TessellateSmoothingStrength = 0.5f;

	/** Whether to resample texture coordinates from the input mesh to avoid texture distortion */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate"))
	bool bResampleUVs = false;

	/** Whether to flip edges to improve mesh quality during post-processing */
	UPROPERTY(EditAnywhere, Category = Tessellate, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Tessellate"))
	bool bEdgeFlips = false;

	//
	// Common properties
	//

	/** If true, use the weights on the specified channel to modulate the relative mesh density.
	* Where the density weight values are 0.0, the desired average edge length is TargetEdgeLength.
	* Where density weight values are 1.0, the desired average edge length is MinTargetEdgeLength.
	* For density weight values between 0 and 1, the desired average edge length is interpolated between TargetEdgeLength and MinTargetEdgeLength.
	*/
	UPROPERTY(EditAnywhere, Category = DensityWeightChannel, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh || bUseTargetEdgeLength"))
	bool bUseDensityWeightChannel = false;

	/** Name of the channel used to modulate the relative mesh density */
	UPROPERTY(EditAnywhere, Category = DensityWeightChannel, meta = (GetOptions = "GetMegaMeshDefinitionChannels", NoResetToDefault,
		EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh || bUseTargetEdgeLength"))
	MeshPartition::FChannelName DensityWeightChannelName;

	UPROPERTY(EditAnywhere, Category = DensityWeightChannel, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh || bUseTargetEdgeLength", UIMin = "-2.0", UIMax = "2.0"))
	float RelativeDensity = 0.f;

	// Whether to only change the mesh where weight values are above the MinWeightThreshold
	UPROPERTY(EditAnywhere, Category = DensityWeightChannel, meta = (EditCondition = "CurrentOperation == ERemeshModifierOperation::Remesh || bUseTargetEdgeLength"))
	bool bUseWeightThreshold = false;

	// Min weight threshold below which the mesh will not be changed
	UPROPERTY(EditAnywhere, Category = DensityWeightChannel, meta = (EditCondition = "bUseWeightThreshold && (CurrentOperation == ERemeshModifierOperation::Remesh || bUseTargetEdgeLength)", UIMin = "0.0", UIMax = "1.0"))
	float MinWeightThreshold = 0.01f;

	/** When true, draws a box showing affected volume in editor */
	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (DisplayName = "Local Bounds"))
	bool bDrawAffectedBox = true;

	/** When true, draws an axis-aligned box containing the potentially non-axis-aligned Local Bounds */
	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings")
	bool bWorldBounds = false;

};


/**
* MegaMeshModifier that does local remeshing or tessellation. Note that currently only triangles whose vertices are
*  entirely inside the modifier's box of influence will be affected.
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Coverage", "RemeshOperation", "Remesh", "Tessellate", "DensityWeightChannel"), meta = (BlueprintSpawnableComponent, MegaMeshClassVersion = "2"))
class URemeshModifier : public MeshPartition::URemeshModifierBase
{
	GENERATED_BODY()

public:

	UE_API URemeshModifier();
	virtual ~URemeshModifier() = default;

	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;

	UE_API virtual TArray<FBox> ComputeBounds() const override;

	UE_API void SetUnscaledCoverage(const FVector3d& InUnscaledCoverage);
	UE_API FVector3d GetUnscaledCoverage() const;

private:

	// MeshPartition::UModifierComponent
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

	UE_API Geometry::FAxisAlignedBox3d GetLocalCoverage() const;

	/**
	* Box in which the modifier remeshes, centered at modifier location. This will be scaled according to component scale.
	*/
	UPROPERTY(EditAnywhere, Category = Coverage, meta = (UIMin = "0", ClampMin = "0"))
	FVector3d UnscaledCoverage = FVector3d(2000, 2000, 10000);

};
} // namespace UE::MeshPartition

#undef UE_API
