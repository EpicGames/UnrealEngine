// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Math/MathFwd.h"
#include "MeshPartitionModifierComponent.h"
#include "Modifiers/CodeReusableMeshPartitionModifierInterface.h"
#include "Spatial/PointHashGrid3.h"
#include "Templates/SharedPointer.h"

#include "MeshPartitionSimpleWriteModifier.generated.h"

namespace UE::MeshPartition
{
namespace MegaMeshWriteModifierLocals
{
	class FBackgroundOp;
}

//~ This type exists because UHT doesn't allow a TArray as a map values for a UPROPERTY, 
//~  and it does not seem worth it to do our own serialization, as it
//~  is much easier to badly break if we edit things
USTRUCT()
struct FSimpleWriteModifier_ChannelValues
{
	GENERATED_BODY()
public:
	FSimpleWriteModifier_ChannelValues() = default;
	FSimpleWriteModifier_ChannelValues(const FName& NameIn, const TArray<float>& ValuesIn)
		: Name(NameIn)
		, Values(ValuesIn)
	{}

	UPROPERTY()
	FName Name;
	UPROPERTY()
	TArray<float> Values;
};

//~ TODO: This and other pcg modifier types should probably go into a separate editor-only module in
//~  PCGMegaMeshInterop (which can't be PCGMegaMeshInteropEditor because it would create a cyclical
//~  dependency with PCGMegaMeshInterop, which needs to use this modifier).
/**
* Modifier intended to be used with PCG
*/
UCLASS(MinimalAPI, meta = (MegaMeshClassVersion = "1"))
class USimpleWriteModifier : public MeshPartition::UModifierComponent
	, public ICodeReusableModifier
{
	GENERATED_BODY()

public:
	USimpleWriteModifier();

	// MeshPartition::UModifierComponent
	virtual TArray<FBox> ComputeBounds() const override;
	virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	virtual FGuid GetCodeVersionKey() const override;
	virtual bool IsTemporarilyDisabledInEditor() const override;

	// ICodeReusableModifier
	virtual void SetDisabledByCode(bool bDisabledIn) override;
	virtual void ResetForReuse() override;
	virtual bool IsUsed() const override;

	MESHPARTITIONEDITOR_API void ReinitializePoints(
		const TArray<FVector3d>& SourcePositions,
		const TArray<FVector3d>* DestPositions,
		const TArray<TPair<FName, TArray<float>>>& WeightChannelValues);

protected:
	virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

private:
	void ClearPoints();
	void ResetCachedData();
	
	UPROPERTY()
	TArray<FVector3d> SourcePositions;
	UPROPERTY()
	TOptional<TArray<FVector3d>> DestPositions;
	
	UPROPERTY()
	TArray<MeshPartition::FSimpleWriteModifier_ChannelValues> WeightChannelValues;
	UPROPERTY()
	FBox GlobalBounds;
	// This is the same as GlobalBounds but expanded by a very small number so that we can make sure 
	//  to grab any points that were on the edges and might otherwise be missed due to imprecision.
	UPROPERTY()
	FBox ExpandedBounds;

	// Cached data for CreateBackgroundOp
	mutable TSharedPtr<const TArray<FVector3d>> SourcePositions_BackgroundOp;
	mutable TSharedPtr<const TArray<FVector3d>> DestPositions_BackgroundOp;
	mutable TSharedPtr<const TArray<TPair<FName, TArray<float>>>> WeightChannelValues_BackgroundOp;
	mutable TSharedPtr<const Geometry::TPointHashGrid3<int32, double>> HashGrid_BackgroundOp;

	bool bDisabledByCode = false;

	friend class MegaMeshWriteModifierLocals::FBackgroundOp;
};
} // namespace UE::MeshPartition