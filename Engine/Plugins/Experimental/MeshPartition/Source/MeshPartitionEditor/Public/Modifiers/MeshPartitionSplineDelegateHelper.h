// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/SplineComponent.h"
#include "Delegates/IDelegateInstance.h"
#include "Misc/TransactionObjectEvent.h"

namespace UE::MeshPartition
{

/**
 * Helper that manages spline-change delegates (undo/redo, property, transform).
 * Both USplineModifier and USplineRemeshModifier own an instance of this to avoid duplicating
 * the registration/unregistration boilerplate.
 */
class FSplineDelegateHelper
{
public:
	/** Callback invoked when any spline delegate fires. */
	using FOnSplineChanged = TFunction<void()>;

	FSplineDelegateHelper(UObject* InOwner, FOnSplineChanged InCallback)
		: Owner(InOwner)
		, Callback(MoveTemp(InCallback))
	{
	}

	~FSplineDelegateHelper()
	{
		Unregister();
	}

	/** Bind delegates to a new spline (unbinds any previous one first). */
	void Register(USplineComponent* Spline)
	{
		Unregister();

		if (!Spline)
		{
			return;
		}

		RegisteredSpline = Spline;

		OnTransactedHandle = FCoreUObjectDelegates::OnObjectTransacted.AddWeakLambda(Owner,
			[this](UObject* Object, const FTransactionObjectEvent& Transaction)
			{
				if (Object == RegisteredSpline.Get() && Transaction.GetEventType() == ETransactionObjectEventType::UndoRedo)
				{
					Callback();
				}
			});

		OnChangedHandle = Spline->GetOnSplineChanged().AddWeakLambda(Owner,
			[this]()
			{
				Callback();
			});

		OnTransformUpdatedHandle = Spline->TransformUpdated.AddWeakLambda(Owner,
			[this](USceneComponent*, EUpdateTransformFlags, ETeleportType)
			{
				Callback();
			});
	}

	/** Unbind all delegates. */
	void Unregister()
	{
		if (OnTransactedHandle.IsValid())
		{
			FCoreUObjectDelegates::OnObjectTransacted.Remove(OnTransactedHandle);
			OnTransactedHandle.Reset();
		}

		if (OnChangedHandle.IsValid())
		{
			if (USplineComponent* const Spline = RegisteredSpline.Get())
			{
				Spline->GetOnSplineChanged().Remove(OnChangedHandle);
			}
			OnChangedHandle.Reset();
		}

		if (OnTransformUpdatedHandle.IsValid())
		{
			if (USplineComponent* const Spline = RegisteredSpline.Get())
			{
				Spline->TransformUpdated.Remove(OnTransformUpdatedHandle);
			}
			OnTransformUpdatedHandle.Reset();
		}

		RegisteredSpline.Reset();
	}

private:
	UObject* Owner = nullptr;
	FOnSplineChanged Callback;
	FDelegateHandle OnTransactedHandle;
	FDelegateHandle OnChangedHandle;
	FDelegateHandle OnTransformUpdatedHandle;
	TWeakObjectPtr<USplineComponent> RegisteredSpline;
};

} // namespace UE::MeshPartition
