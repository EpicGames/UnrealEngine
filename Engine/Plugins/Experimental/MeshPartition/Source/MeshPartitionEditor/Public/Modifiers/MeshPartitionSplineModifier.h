// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Components/SplineComponent.h"
#include "MeshPartitionChannel.h"
#include "MeshPartitionModifierComponent.h"
#include "Delegates/IDelegateInstance.h"
#include "Modifiers/MeshPartitionSplineDelegateHelper.h"
#include "Polygon2.h"
#include "Templates/SharedPointer.h"


#include "MeshPartitionSplineModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

struct FPropertyChangedEvent;
class UCurveBase;
class UMeshElementsVisualizer;
class USplineComponent;
namespace MegaMeshSplineModifierLocals
{
	class FBackgroundOp;
	class FOrientedPolyline3d;
}

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMeshOperator);

namespace UE::MeshPartition
{
class AMeshPartition;
struct FSplineCachedSurfaceData;

UENUM()
enum class ESplineModifierBlendMode : uint8
{
	/* No restrictions on which vertices are affected */
	Normal,

	/* Only move vertices lying on the positive Z side of the spline */
	Min,

	/* Only move vertices lying on the negative Z side of the spline */
	Max
};

/** Which outputs a spline modifier writes. */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class ESplineModifierWriteMode : uint8
{
	/** Displace vertex positions along the spline projection direction. */
	Positions	= (1 << 0),

	/** Write to weight channels using the spline falloff as the blending alpha. */
	Weights		= (1 << 1)
};
ENUM_CLASS_FLAGS(ESplineModifierWriteMode);

/** How a spline modifier's weight channel value is combined with existing values. */
UENUM()
enum class ESplineWeightBlendMode : uint8
{
	/** The new value replaces the existing value, blended by the spline falloff. */
	AlphaBlend,

	/** The new value is added to the existing value, scaled by the spline falloff. */
	Additive,

	/** Takes the minimum of the existing and new values, blended by the spline falloff. */
	Min,

	/** Takes the maximum of the existing and new values, blended by the spline falloff. */
	Max
};

UENUM()
enum class ESplineModifierInteriorSmoothMode : uint8
{
	/* Snaps the MegaMesh surface to the closest point on the region within the spline. */
	Simple = 1,

	/* Smooths the Megamesh surface against the spline boundary, accounting for falloff normals at the edge. */
	Smooth = 2,

	/* Transfers the underlying Megamesh surface detail to fit within the spline boundary. */
	DetailPreserving = 3
};

/**
* Describes a single weight channel entry for a spline modifier. The spline's own falloff (HeightAlpha)
* is used as the blending alpha.
*/
USTRUCT()
struct FSplineModifierWeightEntry
{
	GENERATED_BODY()

	/** Which weight channel to write. */
	UPROPERTY(EditAnywhere, Category = "", meta = (GetOptions = "GetMegaMeshDefinitionChannels", NoResetToDefault))
	MeshPartition::FChannelName WeightChannelName;

	/** The value to write at full influence. */
	UPROPERTY(EditAnywhere, Category = "")
	double Value = 1.0;

	/** How the value is blended with the existing channel value. */
	UPROPERTY(EditAnywhere, Category = "")
	MeshPartition::ESplineWeightBlendMode BlendMode = MeshPartition::ESplineWeightBlendMode::AlphaBlend;
};

/**
* A flexible spline based modifier, suitable for representing ridgelines, channels, raised hills, and lowered depression pockets.
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Coverage", "Spline", "EdgeFalloff", "Interior", "WeightChannels"), meta = (BlueprintSpawnableComponent, MegaMeshClassVersion = "-1"))
class USplineModifier : public MeshPartition::UModifierComponent, public MeshPartition::ISplineModifierBlueprintInterface
{
	GENERATED_BODY()

public:
	UE_API USplineModifier();
	UE_API virtual ~USplineModifier();

	// UObject Implementation
	UE_API virtual void PreEditChange(FProperty* InPropertyAboutToChange) override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostLoad() override;
	// End UObject Implementation

	// MeshPartition::UModifierComponent Implementation
	UE_API virtual void InitializeModifier() override;
	UE_API virtual void UninitializeModifier() override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override; 
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual bool IsContiguous() const override;
	// End MeshPartition::UModifierComponent Implementation

	// Rebuild modifier state based on the spline. Note: Use if anything seems weird or out of date!
	UFUNCTION(CallInEditor, Category = "Spline")
	UE_API void UpdateSplineData();

	UE_API void SetSplineComponent(USplineComponent* InSplineComponent, bool bUpdate = true);
	UE_API USplineComponent* GetSplineComponent() const;

	UFUNCTION(BlueprintCallable, Category = "Spline", meta = (DisplayName = "Set Spline Component"))
	UE_API virtual void BP_SetSplineComponent(USplineComponent* InSplineComponent) override;

	UE_API void SetProjectedSplineCurve(FInterpCurveVector&& NewProjectedCurve) const;

	float GetMaxZDistance() const 
	{ 
		return MaxZDistance; 
	}
	UE_API void SetMaxZDistance(float InMaxZDistance);

	UE_API void SetUseNearestSplineFrameForDisplacement(bool InUseNearestSplineFrameForDisplacement);

	float GetFalloffDistance() const
	{
		return FalloffDistance;
	}
	UE_API void SetFalloffDistance(float InFalloffDistance);

	float GetSplinePolygonErrorTolerance() const
	{
		return SplinePolygonErrorTolerance;
	}
	UE_API void SetSplinePolygonErrorTolerance(float InSplinePolygonErrorTolerance);

	double GetMeshedInteriorNumTriTarget() const
	{
		return MeshedInteriorNumTriTarget;
	}
	UE_API void SetMeshedInteriorNumTriTarget(double InMeshedInteriorNumTriTarget);

	float GetFalloffBoundsMultiplier() const 
	{
		return FalloffBoundsMultiplier;
	}
	UE_API void SetFalloffBoundsMultiplier(float InFalloffBoundsMultiplier);

	ESplineModifierInteriorSmoothMode GetInteriorSmoothMode() const
	{
		return InteriorSmoothMode;
	}
	UE_API void SetInteriorSmoothMode(ESplineModifierInteriorSmoothMode InInteriorSmoothMode);

	float GetMaxProjectionHeightExtent() const
	{
		return MaxProjectionHeightExtent;
	}
	UE_API void SetMaxProjectionHeightExtent(float InMaxProjectionHeightExtent);

	UE_API void SetMeshClosedInterior(bool InbMeshClosedInterior);

	UE_API void SetUseSplineScaleForFalloff(bool bInUseSplineScaleForFalloff);

	bool GetExpandBoundsBySplineScale() const
	{
		return bExpandBoundsBySplineScale;
	}
	UE_API void SetExpandBoundsBySplineScale(bool bInExpandBoundsBySplineScale);

private:

	friend class FSplineSoftObjectPointerDetails;

	UPROPERTY(meta = (DeprecatedProperty))
	FComponentReference SplineRef;

	UPROPERTY(EditAnywhere, Category = "Spline", meta = (DisplayName="Spline"))
	TSoftObjectPtr<USplineComponent> SplinePtr;
	
	/**
	* Allows specifying a spline by name if the spline component is not yet instanced (for example in a Blueprint class).
	* At instance creation (after RegisterAllComponents), the actual USplineComponent on the owning actor is resolved using this name.
	* Editable in Blueprint class (defaults) only, not per-instance.
	*/
	UPROPERTY(EditDefaultsOnly, Category = "Hidden", meta = (DisplayName = "Spline Component Name"))
	FString TemplateSplineComponentName;

	/**
	* Controls which outputs this spline modifier writes.
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (Bitmask, BitmaskEnum = "/Script/MeshPartitionEditor.ESplineModifierWriteMode"))
	uint8 WriteMode = static_cast<uint8>(MeshPartition::ESplineModifierWriteMode::Positions | MeshPartition::ESplineModifierWriteMode::Weights);

	/**
	* Determines how far in projected world distance the spline affects the mesh to either side of it. A higher value results in a slower slope.
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (UIMin = 5, UIMax = 1500, ClampMin = 0.1))
	float FalloffDistance = 50.f;

	/**
	* When true, the scale Y values along the spline are multiplied by FalloffDistance to get the final falloff distance to use.
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (DisplayName = "Spline Scale for Falloff"))
	bool bUseSplineScaleForFalloff = true;

	/**
	* Distance from the spline that remains at full influence before the falloff begins.
	* At 0, the falloff starts immediately at the spline (triangular profile).
	* Values greater than 0 create a flat plateau of that width, followed by the falloff region (trapezoid profile).
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (UIMin = 0, UIMax = 1500, ClampMin = 0))
	float PlateauDistance = 0.0f;

	/**
	* When true, the scale Y values along the spline are multiplied by PlateauDistance to get the final plateau distance to use.
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (DisplayName = "Spline Scale for Plateau"))
	bool bUseSplineScaleForPlateau = false;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use PlateauDistance instead"))
	float FalloffPlateauRatio = 0.0f;

	/**
	* When true, closed splines get meshed to create a plateau. When false, they continue to operate like open
	*  splines, creating volcano-like structures.
	*/
	UPROPERTY(EditAnywhere, Category = "Interior")
	bool bMeshClosedInterior = true;

	/**
	* When true, instead of using the Component Z as a single projection/height direction, the direction is taken from the
	*  rotation around the nearest spline point to each vertex. This is ignored when using a closed loop with bMeshClosedInterior
	*  set to true.
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (DisplayName = "Displace Nearest Frame"))
	bool bUseNearestSplineFrameForDisplacement = false;

	/**
	* When true use a faster but less accurate method for getting the nearest frame on the spline
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (EditCondition = bUseNearestSplineFrameForDisplacement))
	bool bNearestFrameFastApproximation = false;

	/**
	  * When set to Min only vertices on the +Z side of the spline are affected.
	  * When Max only vertices on the -Z side of the spline are affected, and when it's Normal there is no restriction based on the Z axis.
	  *
	  * If bUseNearestSplineFrameForDisplacement is also true, then the Min/Max is computed with respect to the nearest frame's rotated Z axis.
	  */
	UPROPERTY(EditAnywhere, Category = "Coverage")
	ESplineModifierBlendMode BlendMode = ESplineModifierBlendMode::Normal;

	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use BlendMode instead"))
	bool bSingleSidedProjection_DEPRECATED = false;

	/**
	* When projecting in a single direction, the amount to extend the Z bounds in each direction around modifier location
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (ClampMin = 0, UIMax = 10000))
	float MaxZDistance = 2000.f;

	//~ TODO: Should this be more customizable?
	/** 
	* This is used instead of MaxZDistance when bUseNearestSplineFrameForDisplacement is true, since we expand the bounds in
	*  all directions, not just Z, in that case, so we want to be more conservative.
	*/
	UPROPERTY(EditAnywhere, Category = "Coverage", meta = (EditCondition = "bUseNearestSplineFrameForDisplacement"))
	float MaxProjectionHeightExtent = 500;

	/**
	* When true, the bounds expansion automatically accounts for the maximum spline scale value,
	*  replacing FalloffBoundsMultiplier. When false, FalloffBoundsMultiplier is used instead.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "EdgeFalloff", meta = (EditCondition = "bUseSplineScaleForFalloff"))
	bool bExpandBoundsBySplineScale = true;

	/**
	* When calculating bounds, the falloff value is multiplied by this value. This can be set above 1 if the spline has scale
	*  above 1 along its length, and bUseSplineScaleForFalloff is true.
	*  Ignored when bExpandBoundsBySplineScale is true.
	*/
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "EdgeFalloff", meta = (UIMin = 0.1, UIMax = 10, ClampMin = 0.1, EditCondition = "!bExpandBoundsBySplineScale"))
	float FalloffBoundsMultiplier = 1;


	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (DisplayName = "Projection Plane", EditCondition = "!bUseNearestSplineFrameForDisplacement"))
	bool bDrawProjectionPlane = false;

	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (DisplayName = "Local Bounds"))
	bool bDrawLocalBounds = true;

	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (DisplayName = "Spline Frames"))
	bool bDrawSplineFrames = false;

	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (EditCondition = bDrawSplineFrames))
	int32 NumSplineFrames = 10;

	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (EditCondition = bDrawSplineFrames, UIMin = 0.1, UIMax = 2000, ClampMin = 0.1))
	double FrameScale = 100.0;

	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (DisplayName = "Projected Spline", EditCondition = "bUseNearestSplineFrameForDisplacement && !bNearestFrameFastApproximation"))
	bool bDrawProjectedSpline = false;

	/**
	* Uses a curve for the falloff shape.
	*/
	UPROPERTY(EditAnywhere, Category = "EdgeFalloff")
	bool bUseEdgeFalloffCurve = false;

	UPROPERTY(EditAnywhere, Category = "EdgeFalloff", meta = (EditCondition = "bUseEdgeFalloffCurve"))
	TObjectPtr<UCurveFloat> EdgeFalloffCurve = nullptr;

	/**
	* When false, the falloff curve in interpreted with the x axis as moving closer to the spline in the
	*  positive direction, i.e. the falloff is an alpha remapping. When true, the x axis moves away from
	*  the spline in the positive direction, i.e. the curve is a remapping of 1-Alpha.
	*/
	UPROPERTY(EditAnywhere, Category = "EdgeFalloff", meta = (DisplayName = "Spline Left of Falloff", EditCondition = "Spline Left of Falloff"))
	bool bSplineIsLeftOfFalloffCurve = false;
		
	FDelegateHandle EdgeFalloffUpdateHandle;
	
	/** 
	* Sets a smoothing operation that is run on the interior. Note that any value other than "Simple" increases
	*  the cost of the modifier due to its need for topology information.
	*/
	UPROPERTY(EditAnywhere, Category = "Interior", meta = (EditCondition = "bMeshClosedInterior"))
	MeshPartition::ESplineModifierInteriorSmoothMode InteriorSmoothMode = MeshPartition::ESplineModifierInteriorSmoothMode::Simple;

	/**
	* When meshing the interior, controls the resolution of a polyline estimate of the boundary.
	*/
	UPROPERTY(EditAnywhere, Category = "Interior", meta = (DisplayName = "Spline Error Tolerance", EditCondition = "bMeshClosedInterior", UIMin = 5, ClampMin = 5))
	float SplinePolygonErrorTolerance = 5.f;
	
	/**
	* When meshing the interior, roughly controls the granularity of the remeshing (higher is more triangles). This is
	*  not actually a true target number of triangles- instead we target an edge length corresponding to equilateral triangles
	*  of area sized TotalArea/NumTriTarget.
	*/
	UPROPERTY(EditAnywhere, Category = "Interior", meta = (DisplayName = "Num Tri Target", EditCondition = "bMeshClosedInterior", UIMax = 2000, ClampMin = 1))
	double MeshedInteriorNumTriTarget = 500;

	UPROPERTY(EditAnywhere, Category = "Interior", meta = (EditCondition = "bMeshClosedInterior"))
	bool bShowDebugSurface = false;

	/**
	* Weight channels written by this modifier, using the spline falloff as the blending alpha.
	*/
	UPROPERTY(EditAnywhere, Category = "WeightChannels")
	TArray<FSplineModifierWeightEntry> WeightChannels;

	UE_API void UpdateWhenEdgeFalloffChanges(UCurveBase* Curve, EPropertyChangeType::Type);
	
	UE_API FBox ComputeLocalBounds() const;

	UE_API bool ShouldFillInterior() const;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;
	mutable TSharedPtr<const FSplineCachedSurfaceData> CurrentCachedData;
	mutable uint32 LastSplineVersion = 0;

	mutable FInterpCurveVector ProjectedSplineCurve;
	mutable FCriticalSection ProjectedSplineCurveMutex;

	/** Manages spline-change delegate registration/unregistration. */
	TUniquePtr<FSplineDelegateHelper> SplineDelegateHelper;

	friend class MegaMeshSplineModifierLocals::FBackgroundOp; // for FSharedData
};
} // namespace UE::MeshPartition

#undef UE_API
