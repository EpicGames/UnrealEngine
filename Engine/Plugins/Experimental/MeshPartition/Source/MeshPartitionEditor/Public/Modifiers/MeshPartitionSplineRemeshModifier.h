// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionRemeshModifier.h"
#include "Modifiers/MeshPartitionSplineDelegateHelper.h"
#include "MeshPartitionSplineRemeshModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UCurveBase;

namespace UE::MeshPartition
{
struct FSplineCachedSurfaceData;
	
/**
* A modifier that will remesh or tessellate all triangles within some distance to a spline
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Coverage", "Spline", "RemeshOperation", "Remesh", "Tessellate", "DensityWeightChannel"), meta = (BlueprintSpawnableComponent, MegaMeshClassVersion = "-1"))
class USplineRemeshModifier : public MeshPartition::URemeshModifierBase
{
	GENERATED_BODY()

public:
	UE_API USplineRemeshModifier();
	virtual ~USplineRemeshModifier() = default;

	// Rebuild modifier state based on the spline. Note: Use if anything seems weird or out of date!
	UFUNCTION(CallInEditor, Category = "Spline")
	UE_API void UpdateSplineData();

	UE_API void SetSplineComponent(USplineComponent*, bool bUpdate = true);

private:

	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& InPropertyChangedEvent) override;

	UE_API virtual void InitializeModifier() override;
	UE_API virtual void UninitializeModifier() override;

	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

	UE_API USplineComponent* GetSplineComponent() const;
	UE_API FBox ComputeLocalBounds() const;

	// Spline 

	UPROPERTY(EditAnywhere, Category = Spline, meta = (DisplayName="Spline", UseComponentPicker, AllowedClasses = "/Script/Engine.SplineComponent"))
	FComponentReference SplineRef;
	
	/** Radius around the spline that defines the region of effect. If the spline is closed, this also defines the thickness of the generated volume */
	UPROPERTY(EditAnywhere, Category = Coverage, meta = (UIMin = 5, UIMax = 1500, ClampMin = 0.1))
	float SplineRadius = 500.f;

	/** If true and if the spline is closed, generate a volume defined by the Spline and Spline Radius. Triangles within the volume are affected by the modifier */
	UPROPERTY(EditAnywhere, Category = Interior)
	bool bCreateVolumeFromClosedSpline = false;

	/** When generating a volume from a closed spline, this is the maximum distance from the spline to its polyline estimate */
	UPROPERTY(EditAnywhere, Category = Interior, AdvancedDisplay, meta = (UIMin = 5, ClampMin = 5, EditCondition = bCreateVolumeFromClosedSpline))
	float ROIVolumePolygonErrorTolerance = 5.f;

	/** When generating a volume from a closed spline, this roughly controls the resolution of the enclosing surface */
	UPROPERTY(EditAnywhere, Category = Interior, AdvancedDisplay, meta = (UIMax = 2000, ClampMin = 1, EditCondition = bCreateVolumeFromClosedSpline))
	double ROIVolumeTargetResolution = 500;


	// Visualization

	/** Whether to visualize the spline radius using circles */
	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings")
	bool bDrawSplineRadius = false;

	/** If we are visualizing the spline radius, how many circles to use */
	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings", meta = (EditCondition = bDrawSplineRadius, UIMin = 0, UIMax = 100, ClampMin = 0, ClampMax = 1000))
	int SplineRadiusSamples = 10;

	/** Whether to visualize surface generated from the closed spline */
	UPROPERTY(EditAnywhere, Category = "Modifier|ModifierSettings")
	bool bDrawSurface = false;

	mutable TSharedPtr<const FSplineCachedSurfaceData> CurrentCachedData;
	mutable uint32 LastSplineVersion = 0;

	/** Manages spline-change delegate registration/unregistration. */
	TUniquePtr<FSplineDelegateHelper> SplineDelegateHelper;
};
} // namespace UE::MeshPartition

#undef UE_API
