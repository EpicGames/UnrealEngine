// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/MathFwd.h" // FVector2D
#include "MeshPartitionChannel.h" // MeshPartition::FChannelName
#include "MeshPartitionModifierComponent.h"
#include "Image/ImageBuilder.h"
#include "UObject/UnrealType.h" // EPropertyChangeType
#include "MeshPartitionModifierUtils.h"
#include "MeshPartitionTexturePatchModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

class UCurveBase;
class UCurveFloat;
class UTexture2D;

namespace UE::Geometry
{
	class FDisplacementMap;
}

namespace UE::MeshPartition
{
class FMegaMeshTexturePatchDetails;
struct FChannelFieldData;

UENUM(BlueprintType)
enum class ETexturePatchBlendMode : uint8
{
	// The value in the patch determines the "vertical" (in patch Z direction) target position, and
	//  this is blended with the existing position using alpha/falloff.
	AlphaBlend,

	// The value in the patch is an offset to add to the current positions in the patch Z direction.
	Additive,

	// Like Alpha Blend mode, but limited to only moving existing vertices downward in patch Z direction.
	Min,

	// Like Alpha Blend mode, but limited to only moving existing vertices upward in patch Z direction.
	Max
};

UENUM(BlueprintType)
enum class ETexturePatchFalloffMode : uint8
{
	Linear,
	Smooth,
	CustomCurve
};

UENUM(BlueprintType)
enum class ETexturePatchAlphaMode : uint8
{
	// Alpha is always 1.0
	AlwaysOne,

	// Alpha is 0 wherever the texture patch has a zero value, and 1.0 where it is nonzero
	SelfMask,

	// A channel on the texture being used for values
	ThisAlphaChannel,

	// A channel on some other texture, other than the value texture
	OtherAlphaChannel,
};


UENUM(BlueprintType) 
enum class ETexturePatchTessellationMode : uint8
{
	/**
	* Vertex displacement only
	*/
	NonAdaptive UMETA(DisplayName = "None"),

	/**
	* Adaptive Tessellation (high quality)
	*/
	Adaptive,

	/**
	* Parallel Adaptive Tessellation (fast)
	*/
	AdaptiveFast
};


UENUM(BlueprintType)
enum class ETexturePatchTessellationErrorMode : uint8
{
	/**
	* Target error specified in scene-units.
	*/
	Absolute,

	/**
	* Scale-invariant tessellation detail, relative to object bounds.
	*/
	Relative
};


UCLASS(Abstract)
class UTexturePatchEntry : public UObject
{
	GENERATED_BODY()

public:
	void SetTextureAsset(UTexture2D* InTextureAsset);
	UTexture2D* GetTextureAsset() const;

	void SetTextureChannelIndex(const int32 InTextureChannelIndex);
	int32 GetTextureChannelIndex() const;

	void SetAlphaBlendingMode(const MeshPartition::ETexturePatchAlphaMode InTexturePatchAlphaMode);
	ETexturePatchAlphaMode GetAlphaBlendingMode() const;

	void SetTexturePatchBlendMode(const MeshPartition::ETexturePatchBlendMode InTexturePatchBlendMode);
	ETexturePatchBlendMode GetTexturePatchBlendMode() const;

	void SetUseValueCurve(const bool bInUseValueCurve);
	bool GetUseValueCurve() const;

	void SetValueCurve(UCurveFloat* InValueCurve);
	UCurveFloat* GetValueCurve() const;

private:

	//~ TODO: Probably wouldn't be too hard to make this a UTexture instead, in which case it will also
	//~  support render targets.
	/**
	* Texture that the patch samples. Note that currently the patch will not know to trigger a reapplication if the
	*  data inside the asset changes somehow.
	*/
	UPROPERTY(EditAnywhere, Category = "", meta = (DisallowedAssetDataTags = "VirtualTextureStreaming=True"))
	TObjectPtr<UTexture2D> TextureAsset = nullptr;

	/**
	* Channel index to sample in the texture asset. R: 0, G: 1, B:2, A:3 
	*/
	UPROPERTY(EditAnywhere, Category = "", DisplayName = "Texture Channel",  meta = (ClampMin = "0", ClampMax = "3"))
	int32 TextureChannelIndex = 0;


	UPROPERTY(EditAnywhere, Category = "Alpha Blending")
	MeshPartition::ETexturePatchAlphaMode AlphaMode = MeshPartition::ETexturePatchAlphaMode::SelfMask;

	UPROPERTY(EditAnywhere, Category = "Alpha Blending", meta = (DisallowedAssetDataTags = "VirtualTextureStreaming=True", 
		EditCondition = "AlphaMode == ETexturePatchAlphaMode::OtherAlphaChannel", EditConditionHides))
	TObjectPtr<UTexture2D> AlphaTextureAsset = nullptr;
	
	UPROPERTY(EditAnywhere, Category = "Alpha Blending", meta = (ClampMin = "0", ClampMax = "3", EditConditionHides,
		EditCondition = "AlphaMode == ETexturePatchAlphaMode::ThisAlphaChannel || AlphaMode == ETexturePatchAlphaMode::OtherAlphaChannel"))
	int32 AlphaChannelIndex = 3;

	/**
	* When using "Self Mask" alpha mode, the alpha will be set to 0 if absolute value of the texture sampled values is this or lower.
	*/
	UPROPERTY(EditAnywhere, Category = "Alpha Blending", meta = (ClampMin = "0", UIMax = "1", 
		EditCondition = "AlphaMode == ETexturePatchAlphaMode::SelfMask", EditConditionHides))
	double SelfMaskTolerance = 0.001;

	/**
	* How the values from the texture affect the intensity.
	*/
	UPROPERTY(EditAnywhere, Category = "Alpha Blending")
	MeshPartition::ETexturePatchBlendMode BlendMode = MeshPartition::ETexturePatchBlendMode::AlphaBlend;

	/**
	* When using Min or Max BlendMode, allows for the transition between current values and new values to be smoothed out, to prevent
	*  sharp edges when intersecting existing geometry in the center of the modifier (where the falloff does not reach). The parameter
	*  corresponds to distance between the desired and current heights across which the min/max is blended.
	*/
	UPROPERTY(EditAnywhere, Category = "Alpha Blending", meta = (ClampMin = "0.0", UIMax = "1000", EditConditionHides,
		EditCondition = "BlendMode == ETexturePatchBlendMode::Min || BlendMode == ETexturePatchBlendMode::Max"))
	double SoftnessParameter = 0.0;

	/**
	* How effects of the patch are decreased along the edges of the modifier. The effect will take place across Falloff Distance from
	*  the modifier edges.
	*/
	UPROPERTY(EditAnywhere, Category = "Falloff", DisplayName = "Mode")
	MeshPartition::ETexturePatchFalloffMode FalloffMode = MeshPartition::ETexturePatchFalloffMode::Smooth;

	/**
	* Distance (in local space) across which to fall off the patch effects from the edge.
	*/
	UPROPERTY(EditAnywhere, Category = "Falloff", meta = (ClampMin = "0", UIMax = "2000"))
	float FalloffDistance = 0;

	/**
	* When not 0, the corners are rounded with the given radius (clamped to half of the smallest dimension).
	*  A square patch with maximal corner radius becomes a circle.
	*/
	UPROPERTY(EditAnywhere, Category = "Falloff", meta = (ClampMin = "0", UIMax = "2000"))
	float CornerRadius = 0;

	UPROPERTY(EditAnywhere, Category = "Falloff", DisplayName = "Curve", meta = (
		EditCondition = "FalloffMode == ETexturePatchFalloffMode::CustomCurve", EditConditionHides))
	TObjectPtr<UCurveFloat> FalloffCurve;

	/**
	* When true, allows a custom curve to remap values from the texture, before Scale and Zero Value are applied.
	*/
	UPROPERTY(EditAnywhere, Category = "Curve Adjustment")
	bool bUseValueCurve = false;

	/**
	* Optional adjustment curve that can be applied to texture values (similar to a contrast adjustment).
	*/
	UPROPERTY(EditAnywhere, Category = "Curve Adjustment", meta = (EditCondition = "bUseValueCurve", EditConditionHides))
	TObjectPtr<UCurveFloat> ValueCurve = nullptr;

	//~ TODO: are these worth having?
	/**
	* If the texture values are not in the range [0,1], gives a way to remap to a [0,1] range for using the adjustment curve.
	*  The input to the curve will be transformed via (Input - ValueCurveOffset) / ValueCurveScale, and the output
	*  will be transformed back via Output * AjustmentCurveScale + AjustmentCurveOffset.
	*/
	UPROPERTY(EditAnywhere, Category = "Curve Adjustment",  meta = (UIMin = "0", UIMax = "2000", EditCondition="bUseValueCurve", EditConditionHides), AdvancedDisplay)
	double ValueCurveOffset = 0;
	/**
	* If the texture values are not in the range [0,1], gives a way to remap to a [0,1] range for using the adjustment curve.
	*  The input to the curve will be transformed via (Input - ValueCurveOffset) / ValueCurveScale, and the output
	*  will be transformed back via Output * AdjustmentCurveScale + AdjustmentCurveOffset.
	*/
	UPROPERTY(EditAnywhere, Category = "Curve Adjustment", meta = (ClampMin = "0.000001", UIMax = "2000", EditCondition = "bUseValueCurve", EditConditionHides), AdvancedDisplay)
	double ValueCurveScale = 1.0;

	FDelegateHandle FalloffCurveListenerHandle;
	FDelegateHandle ValueCurveListenerHandle;

	void TriggerModifierUpdate()
	{
		MeshPartition::UModifierComponent* OwningModifier = Cast<MeshPartition::UModifierComponent>(GetOuter());
		if (OwningModifier == nullptr)
		{
			return;
		}

		OwningModifier->OnChanged(OwningModifier->ComputeBounds(), EChangeType::StateChange);
	}

	void OnCurveChanged(UCurveBase* Curve, EPropertyChangeType::Type ChangeType);
	void AttachCurveListeners();
	void DetachCurveListeners();

	// UObject
	virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;

	friend class UTexturePatchModifier;
	friend class UInstancedTexturePatchModifier;
	// For now, we keep everything private instead of protected, so we'll friend or child classes.
	friend class UTexturePatchHeightEntry;
	friend class UTexturePatchWeightEntry;
	friend class FMegaMeshTexturePatchDetails;
	friend class FReadTextureAdapter;
};

UCLASS(EditInlineNew)
class UTexturePatchHeightEntry : public MeshPartition::UTexturePatchEntry
{
	GENERATED_BODY()

private:

	/**
	* How to scale values read from the texture, controlling the intensity in the local space of the component. 
	* If the texture stores values in the range [0,1], then you will need
	* to scale this up quite high to turn those values into noticeable effects on the mesh. 
	*/
	UPROPERTY(EditAnywhere, Category = "", DisplayName = "Scale", meta = (DisplayAfter = "TextureChannelIndex", UIMin = "0", UIMax = "10000"))
	double EncodingScale = 1000.;

	/**
	* What value is considered 0 in the texture. For instance if your texture stores values in the range [0,1], then
	*  0.5 may be your zero. In alpha blend mode, vertices would be moved to patch plane at this value, and in additive
	*  mode, no displacement would be made.
	*/
	UPROPERTY(EditAnywhere, Category = "", DisplayName = "Zero Value", meta = (DisplayAfter = "EncodingScale", UIMin = "0", UIMax = "1"))
	double ZeroInEncoding = 0.5;

	/**
	* Optional weight channel to attenuate height scale.
	*/
	UPROPERTY(EditAnywhere, Category = "", DisplayName = "Scale Weight Channel", meta = (DisplayAfter = "ZeroInEncoding", GetOptions = "GetMegaMeshDefinitionChannels", NoResetToDefault))
	MeshPartition::FChannelName HeightScaleWeightChannel;

	UFUNCTION()
	TArray<FName> GetMegaMeshDefinitionChannels() const;

	friend class UTexturePatchModifier;
	friend class UInstancedTexturePatchModifier;
};

UCLASS(EditInlineNew)
class UTexturePatchWeightEntry : public MeshPartition::UTexturePatchEntry
{
	GENERATED_BODY()

	/**
	* Copy all falloff settings from the displacement channel to all weight channels.
	*/
	UFUNCTION(CallInEditor, Category = "", DisplayName = "Copy Falloff", BlueprintCallable)
	void CopyFalloffSettings();


	UPROPERTY(EditAnywhere, Category = "", meta = (GetOptions = "GetMegaMeshDefinitionChannels", NoResetToDefault))
	MeshPartition::FChannelName WeightChannelName;

	/**
	* Only relevant when Min or Max blend mode is used for height. When true, the weights are not applied in
	*  areas where the height data was ignored due to above/below the current values in min/max blend mode.
	*/
	UPROPERTY(EditAnywhere, Category = "")
	bool bApplyHeightMinMaxBlend = false;

	/**
	* When using "Apply Height Min Max Blend," this parameter sets the range of heights across which the weights
	* will blend. If this parameter is equal to Softness Parameter, then heights blended by the Softness Parameter
	* will also have their weights linearly blended.
	*/
	UPROPERTY(EditAnywhere, Category = "", meta = (ClampMin = "0", UIMax = "1000", 
		EditCondition = "bApplyHeightMinMaxBlend", EditConditionHides))
	double HeightMinMaxBlendDistance = 0.0;

	//~ These could be in the base class, but it is helpful to have different defaults for weights vs height. Also
	//~  this is not as frequently needed for weights, so it is AdvancedDisplay, though this seems to not work for
	//~  instanced objects, so we explicitly use "Advanced" subcategory
	
	/**
	* How to scale values obtained from the texture.
	* Note: this is applied after the value curve, if that is used.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced", DisplayName = "Scale", meta = (UIMin = "0", UIMax = "5"))
	double EncodingScale = 1.0;

	/**
	* What value is considered 0 in the texture. For instance you may choose to change the zero to 0.5, so that
	*  values of 0 become -0.5 when applied.
	* Note: this is applied after the value curve, if that is used.
	*/
	UPROPERTY(EditAnywhere, Category = "Advanced", DisplayName = "Zero Value", meta = (UIMin = "0", UIMax = "1"))
	double ZeroInEncoding = 0.0;


	UFUNCTION()
	TArray<FName> GetMegaMeshDefinitionChannels() const;

	friend class UTexturePatchModifier;
	friend class UInstancedTexturePatchModifier;
};

/**
* A modifier that reads from a given texture and moves vertices in the megamesh in some "vertical"
*  direction (where vertical is determined by the Z direction of this component).
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "HeightDisplacement", "WeightChannels", "AdaptiveTessellation"), meta=(BlueprintSpawnableComponent, MegaMeshClassVersion = "1"))
class UTexturePatchModifier : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()
	
public:
	UE_API UTexturePatchModifier();

	/**
	* Gives size in unscaled world coordinates (ie before applying patch transform) of the patch as measured
	* between the centers of the outermost pixels. Measuring the coverage this way means that a patch can
	* affect the same region of the landscape regardless of patch resolution.
	* This is also the range across which bilinear interpolation always has correct values, so the area outside
	* this center portion is usually set as a "dead" border that doesn't affect the landscape.
	*/
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual FVector2D GetUnscaledCoverage() const { return FVector2D(UnscaledPatchCoverage); }

	//~ TODO: Should this trigger PostEditChange, maybe?
	/**
	* Set the patch coverage (see GetUnscaledCoverage for description).
	*/
	UFUNCTION(BlueprintCallable, Category = LandscapePatch)
	virtual void SetUnscaledCoverage(const FVector2D Coverage);

	void SetApplyComponentZScale(const bool bInApplyComponentZScale);
	bool GetApplyComponentZScale() const;

	void SetHeightChannel(UTexturePatchHeightEntry* InHeightEntry);
	UTexturePatchHeightEntry* GetHeightChannel() const;

	void SetAdaptiveTessellationMode(const ETexturePatchTessellationMode InTessellationMode);
	ETexturePatchTessellationMode GetAdaptiveTesselationMode() const;

	// MeshPartition::UModifierComponent
	UE_API virtual void InitializeModifier() override;
	UE_API virtual void UninitializeModifier() override;
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual void DrawVisualization(const FSceneView* View, FPrimitiveDrawInterface* PDI) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	UE_API virtual Tasks::FTask GetAsyncPrepareResourcesTask() const override;
	
	// UObject
	UE_API virtual void Serialize(FArchive& Ar) override;
	using Super::PreEditChange;
	UE_API virtual void PreEditChange(FEditPropertyChain& PropertyAboutToChange) override;
	UE_API virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
	UE_API virtual void PostEditUndo() override;

	void TriggerUpdate()
	{
		OnChanged(ComputeBounds(), EChangeType::StateChange);
	}

	UE_API void OnInnerObjectModified(const FPropertyChangedEvent& InnerPropertyChangedEvent);

protected:

	//=====================================================================================================================
	// General modifier settings
	//=====================================================================================================================
	
	/** 
	* At scale 1.0, the X and Y of the region affected by the height patch. This corresponds to the distance from the 
	*  center of the first pixel to the center of the last pixel in the patch texture in the X and Y directions. 
	*/
	UPROPERTY(EditAnywhere, Category = "Modifier|Extent", meta = (UIMin = "0", ClampMin = "0"))
	FVector2D UnscaledPatchCoverage = FVector2D(1000, 1000);

	/**
	* How far up above the center it affects vertices in the mesh.
	*/
	UPROPERTY(EditAnywhere, Category = "Modifier|Extent", meta = (ClampMin = "0"), AdvancedDisplay, DisplayName = "Vertical Extent Up")
	float VerticalPatchExtentUp = 10000.f;

	/**
	* How far down below the patch it affects vertices in the mesh.
	*/
	UPROPERTY(EditAnywhere, Category = "Modifier|Extent", meta = (ClampMin = "0"), AdvancedDisplay, DisplayName = "Vertical Extent Down")
	float VerticalPatchExtentDown = 10000.f;

	/**
	* When true, draws a red rectangle showing patch area in editor.
	*/
	UPROPERTY(EditAnywhere, Category = "Modifier|Visualization")
	bool bDrawPatchRectangle = true;

	/**
	* When true, draws a box showing affected volume in editor.
	*/
	UPROPERTY(EditAnywhere, Category = "Modifier|Visualization")
	bool bDrawAffectedBox = true;

	//=====================================================================================================================
	// Channels settings
	//=====================================================================================================================
	
	UPROPERTY(EditAnywhere, Instanced, Category = HeightDisplacement, DisplayName = "Input", meta=(ShowOnlyInnerProperties))
	TObjectPtr<MeshPartition::UTexturePatchHeightEntry> HeightChannel;

	UPROPERTY(EditAnywhere, Instanced, Category = WeightChannels, DisplayName = "Outputs", meta=(ShowOnlyInnerProperties))
	TArray<TObjectPtr<MeshPartition::UTexturePatchWeightEntry>> WeightChannels;

	/**
	* Copy falloff settings from height channel to all weight layers
	*/
	UFUNCTION(BlueprintCallable, Category = WeightChannels)
	UE_API void MatchWeightFalloffsToHeight();

	/**
	* Copy alpha blend settings from height channel to all weight layers
	*/
	UFUNCTION(BlueprintCallable, Category = WeightChannels)
	UE_API void MatchWeightAlphaBlendToHeight();

	/**
	* Copy curve settings from height channel to all weight layers
	*/
	UFUNCTION(BlueprintCallable, Category = WeightChannels)
	UE_API void MatchWeightCurveToHeight();

	/**
	* Trigger reading data from all texture asset again. This automatically
	*  happens if the texture asset, channel, or the "use alpha" flag is changed.
	*/
	UFUNCTION(CallInEditor, Category = HeightDisplacement, DisplayName = "Force All Texture Assets Reread", BlueprintCallable)
	UE_API void UpdateFromTexture();

	/**
	* Whether to apply the patch Z scale to the height stored in the patch.
	*/
	UPROPERTY(EditAnywhere, Category = HeightDisplacement, AdvancedDisplay, meta = (DisplayName = "Apply Component Z Scale"))
	bool bApplyComponentZScale = true;

	//=====================================================================================================================
	// Tessellation settings
	//=====================================================================================================================

	/**
	* Tessellation mode to apply.
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, DisplayName = "Mode")
	MeshPartition::ETexturePatchTessellationMode TessellationMode = MeshPartition::ETexturePatchTessellationMode::NonAdaptive;

	/**
	* Primary parameter to controlling tessellation detail when using adaptive tessellation (lower values produce finer meshes). When mode is absolute, correspond to world units error. 
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, DisplayName = "Error Tolerance")
	float TessellationError = 50;

	/**
	* Control whether error criterion is related to world units (absolute) or scale-invariant (relative).
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, AdvancedDisplay, DisplayName = "Error Mode")
	MeshPartition::ETexturePatchTessellationErrorMode TessellationErrorMode = MeshPartition::ETexturePatchTessellationErrorMode::Absolute;

	/**
	* Tessellation will not refine edges smaller than this value.
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, AdvancedDisplay)
	float MinimumEdgeLength = 0;

	/**
	* Tessellation will always refine edges larger than this value.
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, AdvancedDisplay)
	float MaximumEdgeLength = 10000;

	/**
	* Whether additional mesh regularization will be applied after tessellation.
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, AdvancedDisplay, meta = (DisplayName = "Mesh Optimization"));
	bool bTessellationMeshRegularization = false;

	/**
	* Controls the sensitivity to features. A higher value will refine more at high curvature areas. Only applies in mode Adaptive Fast.
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, AdvancedDisplay, meta = (DisplayName = "Feature Sensitivity", ClampMin = "0", ClampMax = "1", UIMin = "0", UIMax = "1"))
	float TessellationFeatureSensitivity = 0.75f;

	/**
	* Optional channel to adjust tessellation detail. Positive values increase detail and negative values decrease detail. 
	*/
	UPROPERTY(EditAnywhere, Category = AdaptiveTessellation, AdvancedDisplay, DisplayName = "Detail Adjustment Channel", meta = (GetOptions = "GetMegaMeshDefinitionChannels", NoResetToDefault))
	MeshPartition::FChannelName DetailAdjustmentChannelName;

protected:
	UE_API bool HasScalarField() const;

	UE_API void UpdateAllTextureData();
	UE_API void UpdateHeightChannel();
	UE_API void UpdateWeightChannel(int32 Index);
	UE_API void UpdateChannelData(const UTexturePatchEntry* Channel);
	UE_API void UpdateDisplacementMap();

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

	/*
	 * Since patch application happens in parallel threads, we prep our texture data into a cache (so that
	 * we can complete texture compilation, etc) and then use that. To make this cache work across any
	 * UnscaledPatchCoverage, it is stored such that it can be sampled using Local2DCoordinates/UnscaledPatchCoverage.
	 * This is almost like a UV lookup into the texture with a (-0.5,-0.5) offset but not quite, because the [0,1]
	 * range stretches between the middles of the extremal pixels rather than their edges.
	 * TODO: We might want to cache the read data for textures in some external system to reuse it across many patches.
	 * Also, it would be good to update that data when the texture changes by watching its hash.
	 */
	TSharedPtr<const FChannelFieldData> HeightChannelField;
	TArray<TSharedPtr<const FChannelFieldData>> WeightChannelFields;
	TSharedPtr<Utils::TAsyncTransform<Geometry::FDisplacementMap>> FilteredDisplacementMap;

	friend class UTexturePatchEntry;
	friend class UTexturePatchWeightEntry;
	friend class FMegaMeshTexturePatchDetails;
};
} // namespace UE::MeshPartition

#undef UE_API
