// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionWeightUtilityModifier.generated.h"

#define UE_API MESHPARTITIONEDITOR_API

namespace UE::MeshPartition
{
namespace MegaMeshWeightUtilityModifierLocals
{
	class FBackgroundOp;
}

/**
* A simple modifier that will write a radial profile with falloff to a weight channel. 
*/
UCLASS(MinimalAPI, PrioritizeCategories = ("Modifier", "Profile"), meta=(BlueprintSpawnableComponent, MegaMeshClassVersion = "1"))
class UWeightUtilityModifier : public MeshPartition::UModifierComponent
{
	GENERATED_BODY()
	
public:
	UE_API UWeightUtilityModifier();

	// Begin MeshPartition::UModifierComponent Implementation
	UE_API virtual TArray<FBox> ComputeBounds() const override;
	UE_API virtual TSharedPtr<const UE::MeshPartition::IModifierBackgroundOp> CreateBackgroundOp(const MeshPartition::EBuildType InBuildType) const override;
	UE_API virtual FGuid GetCodeVersionKey() const override;
	// End MeshPartition::UModifierComponent Implementation

protected:

	friend class MegaMeshWeightUtilityModifierLocals::FBackgroundOp;

	UE_API virtual void GatherDependencies(MeshPartition::IDependencyInterface& Dependencies) const override;

	/* 
	* Radius of the constant inner disk. 
	*/
	UPROPERTY(EditAnywhere, Category = Profile)
	float Radius = 1000.f;

	/*
	* Falloff distance starting from the inner disk.
	*/
	UPROPERTY(EditAnywhere, Category = Profile)
	float Falloff = 1000.f;

	/*
	* Weight value within the inner disk.
	*/
	UPROPERTY(EditAnywhere, Category = Profile)
	float InnerValue = 1.f;

	/*
	* Weight value at the outside of the falloff region.
	*/
	UPROPERTY(EditAnywhere, Category = Profile)
	float OuterValue = 0.f;

	/*
	* Multiply the clamped positive cosine between the component Z direction and the mesh normal (front-sidedness factor). 
	*/
	UPROPERTY(EditAnywhere, Category = Profile)
	bool bCosineWeighted = { false };

	/*
	* Maximum distance from the component center in Z-direction to affect base region.
	*/
	UPROPERTY(EditAnywhere, Category = Profile)
	float MaxZDistance = 20000.f;

	/*
	* Target weight channel to write to. 
	*/
	UPROPERTY(EditAnywhere, Category = Profile, meta = (GetOptions = "GetMegaMeshDefinitionChannels", NoResetToDefault))
	FName WeightChannelName;
};
} // namespace UE::MeshPartition

#undef UE_API
