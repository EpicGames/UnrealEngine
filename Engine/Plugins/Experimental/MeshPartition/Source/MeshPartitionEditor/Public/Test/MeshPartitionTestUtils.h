// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_DEV_AUTOMATION_TESTS

#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"

#include "Containers/Array.h"
#include "Misc/AutomationTest.h"
#include "UObject/SoftObjectPtr.h"
#include "Engine/StaticMesh.h"
#include "MeshDescription.h"
#include "Modifiers/MeshPartitionPatchModifier.h"
#include "Modifiers/MeshPartitionRemeshModifier.h"

// Enable this to create the test assets in the current editor world, e.g. to compare it with the reference assets.
#define AUTOMATED_TESTING_EXECUTE_IN_EDITOR_WORLD 0

class UWorld;

namespace UE::Geometry { class FDynamicMesh3; }

namespace UE::MeshPartition
{
	class AMeshPartition;
	class UModifierComponent;
}

namespace UE::MeshPartition::TestUtils
{

MESHPARTITIONEDITOR_API bool ShouldUpdateReferenceAssets();

/**
* Helper utility that creates a single base modifier plane mesh with specified worldspace width/height and vertex counts.
*/
MESHPARTITIONEDITOR_API AMeshPartition* CreateTestMesh(UWorld* InWorld, int InWidth, int InHeight, int InWidthVertexCount, int InHeightVertexCount);

/**
* Creates a rectangular mesh modifier with specified complexity and transform
*
* @param MeshPartitionEditorComponent	Component for affected MeshPartition
* @param Width							Width of rectangle
* @param Height							Height of rectangle
* @param WidthVertexCount				Number of vertices along width of the rectangle
* @param HeightVertexCount				Number of vertices along height of the rectangle
* @param OutBounds						Array to store the resulting bounding box
* @param Transform						World transform to apply to the modifier (defaults to identity)
* @return								Pointer to the created actor
*/
AActor* CreateRectangleModifier(UMeshPartitionEditorComponent* MeshPartitionEditorComponent, const double Width, const double Height, const int32 WidthVertexCount,
	const int32 HeightVertexCount, TArray<FBox>& OutBounds, const FTransform& Transform = FTransform::Identity);

/**
 * Creates Patch Modifier with specified world location and build priority
 *
 * @param PatchActor				Modifier Actor
 * @param MeshPartition				Affected MeshPartition by modifier
 * @param ModifierWorldLocation		Location of the modifier actor
 * @param Priority					Build priority (Default = 0.0)
 * @return							Pointer to the created UPatchModifier
 */
UPatchModifier* CreatePatchModifier(AActor* PatchActor, AMeshPartition* MeshPartition, const FVector& ModifierWorldLocation, const float Priority = 0.0);

/**
 * Creates Remesh modifier with specified Location and UnscaledCoverage property
 *
 * @param Actor						Modifier's actor
 * @param MeshPartition				Affected MeshPartition
 * @param ModifierWorldLocation		Location of the modifier
 * @param UnscaledCoverage			Remesh Modifier's Unscaled Coverage property (Bounding box)
 * @return							Returns RemeshModifier
 */
MESHPARTITIONEDITOR_API URemeshModifier* CreateRemeshModifier(AActor* Actor, AMeshPartition* MeshPartition, const FVector& ModifierWorldLocation, const FVector3d& UnscaledCoverage);

/**
* Helper utility that creates a multiple base modifier plane meshes with specified worldspace width/height and vertex counts.
*/
MESHPARTITIONEDITOR_API AMeshPartition* CreateTestMeshSectioned(UWorld* InWorld, int InSectionCountX, int InSectionCountY, int InSectionWidth, int InSectionHeight, int InSectionWidthVertexCount, int InSectionHeightVertexCount);
	
/**
* Helper utility to load a bunch of soft object pointers to static meshes so we can test against their vertex data.
*/
MESHPARTITIONEDITOR_API bool LoadMeshes(TArray<TObjectPtr<UStaticMesh>>& OutMeshes, TArrayView<const TSoftObjectPtr<UStaticMesh>> InSoftPointers);

/**
 * Loads a region of partitioned world.
 *
 * @param InWorld World pointer.
 * @param InWorldBoxExtent FVector extent for WorldBox that is to be loaded, defaults to 10k if unspecified.
 * @return true if the region was loaded successfully.
 */
MESHPARTITIONEDITOR_API bool LoadWorldPartitionRegion(UWorld* InWorld, const FVector& InWorldBoxExtent = FVector(10000));

/**
* Helper utility that compares the vertex positions of a saved ground truth static mesh asset with the generated dynamic mesh.
*/
template <typename MeshType>
bool CompareMesh(const UStaticMesh* InExpected, const MeshType* InActual)
{
	if (InExpected == nullptr || !InExpected->IsMeshDescriptionValid(0))
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Failed to load expected mesh data");
		return false;
	}
	const FMeshDescription* ExpectedMeshDescription = InExpected->GetMeshDescription(0);
	check(ExpectedMeshDescription != nullptr);

	const FVertexArray& ExpectedVertices = ExpectedMeshDescription->Vertices();
	
	if (ExpectedVertices.Num() != InActual->VertexCount())
	{
		UE_LOGF(LogMegaMeshEditor, Error, "Vertex counts do not match between actual and expected mesh. Actual %d, Expected: %d", InActual->VertexCount(), ExpectedVertices.Num());
		return false;
	}

	TArray<FVector3f> ExpectedPositions;
	ExpectedPositions.Reserve(ExpectedVertices.Num());
	for (const FVertexID& VertexID : ExpectedVertices.GetElementIDs())
	{
		const int32 VertexIndex = VertexID.GetValue();
		ExpectedPositions.Add(ExpectedMeshDescription->GetVertexPosition(VertexID));
	}

	TArray<FVector3f> ActualPositions;
	ActualPositions.Reserve(InActual->VertexCount());
	for (int VertexID : InActual->VertexIndicesItr())
	{
		ActualPositions.Add(FVector3f(InActual->GetVertex(VertexID)));
	}
		
	auto SortPredicate = [](const FVector3f& LHS, const FVector3f& RHS)
	{
		if (LHS.X < RHS.X)
		{
			return true;
		}
		else if (LHS.X == RHS.X)
		{
			if (LHS.Y < RHS.Y)
			{
				return true;
			}
			else if (LHS.Y == RHS.Y)
			{
				return LHS.Z < RHS.Z;
			}
		}
		return false;
	};
		
	Algo::Sort(ExpectedPositions, SortPredicate);
	Algo::Sort(ActualPositions, SortPredicate);
		
	for (int VertexIndex = 0; VertexIndex < ActualPositions.Num(); ++VertexIndex)
	{
		const FVector3f ActualPosition = ActualPositions[VertexIndex];
		const FVector3f ExpectedPosition = ExpectedPositions[VertexIndex];
		if (!ActualPosition.Equals(ExpectedPosition, 1.e-2f))
		{
			UE_LOGF(LogMegaMeshEditor, Error, "Expected vertex position and actual vertex position differ: Actual %ls, Expected %ls", *ActualPosition.ToString(), *ExpectedPosition.ToString());
			return false;
		}
	}

	return true;
}

/**
 * Helper function that validates generated mesh against ground truth mesh
 * 
 * @param InMeshPartitionComponent		MeshPartitionEditorComponent of an existing MeshPartition
 * @param InGroundTruthPaths			Array of paths for ground truth meshes
 * @param OutErrorMessage				Error message, if failure was found
 * @return								True if validation passed, False if there was a failure
 */
MESHPARTITIONEDITOR_API bool ValidatePreviewMeshAgainstGroundTruth(const UMeshPartitionEditorComponent* InMeshPartitionComponent,
	const TArray<FString>& InGroundTruthPaths, FString& OutErrorMessage);

/**
 * Helper function that creates or updates the ground truth mesh using the generated mesh
 * 
 * @param MeshPartitionComponent		MeshPartitionEditorComponent of an existing MeshPartition
 * @param ReferencePackage				Package containing the reference meshes
 * @param ReferenceMeshNames			Array of names for reference meshes stored in the package
 * @param TestName						Name of the test this function is being called from for logging purposes.
 * @return								True if ground truth creation/update was successful, False if there was a failure
 */
MESHPARTITIONEDITOR_API bool CreateOrUpdateReference(const UMeshPartitionEditorComponent* MeshPartitionComponent,
                                                     const FString& ReferencePackage,
                                                     const TArray<FString>& ReferenceMeshNames,
                                                     const FString& TestName);

/**
 * RAII guard that saves a CVar's current value on construction,
 * forces it to a new value for the duration of the scope, then restores it
 * on destruction, even if the test throws or fails early.
 */
struct FScopedCVarOverride
{
	FScopedCVarOverride(const TCHAR* InName, const TCHAR* NewValue, const EConsoleVariableFlags SetBy = ECVF_SetByTemp)
		: CVar(IConsoleManager::Get().FindConsoleVariable(InName))
		, SetByPriority(SetBy)
	{
		if (CVar)
		{
			OriginalValue = CVar->GetString();
			CVar->Set(NewValue, SetBy);
		}
		else
		{
			UE_LOGF(LogMegaMeshEditor, Warning, "FScopedCVarOverride: CVar '%ls' not found.", InName);
		}
	}

	~FScopedCVarOverride()
	{
		if (CVar)
		{
			CVar->Set(*OriginalValue, SetByPriority);
		}
	}

	// Non-copyable, non-movable, must live on the stack
	FScopedCVarOverride(const FScopedCVarOverride&) = delete;
	FScopedCVarOverride& operator=(const FScopedCVarOverride&) = delete;
	FScopedCVarOverride(FScopedCVarOverride&&) = delete;
	FScopedCVarOverride& operator=(FScopedCVarOverride&&) = delete;

private:
	IConsoleVariable* CVar;
	FString OriginalValue;
	EConsoleVariableFlags SetByPriority;
};
} // namespace UE::MeshPartition::TestUtils


#endif // WITH_DEV_AUTOMATION_TESTS