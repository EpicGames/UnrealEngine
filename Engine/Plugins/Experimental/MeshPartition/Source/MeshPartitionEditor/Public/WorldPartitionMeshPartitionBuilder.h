// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionEditorComponent.h" // FModifierDescriptorPair
#include "MeshPartitionModifierTaskGraph.h"
#include "Tasks/Task.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionBuilder.h"
#include "WorldPartition/WorldPartitionBuilderHelpers.h"

#include "WorldPartitionMeshPartitionBuilder.generated.h"

class AActor;
class UPackage;
class UWorld;
class UWorldPartition;
enum class EEditorBuildResult : uint8;

namespace UE::MeshPartition
{
	struct FModifierDescriptorCache;
	class ACompiledSection;
	struct FCompiledSectionBuildInfo;

	UENUM()
	enum class EMeshPartitionReuseMode
	{
		ForceRebuild,			// slowest: rebuild all compiled sections, even if we don't think it is necessary
		ByModifierHash,			// fast:	skip building compiled sections where the modifier cachekey hash has not changed (requires loading the modifiers to calculate their cache keys)
		ByPackageHash,			// fastest: does not require loading modifiers. skip building compiled sections where the dependent asset packages (as declared by GatherDependencies) have not changed.
		ValidateHashes,			// fast:	runs both package hash and modifier hash paths, and validates their behavior.  Used mainly for debugging.
		Count UMETA(Hidden)
	};
}

// ENUM_RANGE_BY_COUNT defines a symbol that needs to be declared in the global namespace.
ENUM_RANGE_BY_COUNT(UE::MeshPartition::EMeshPartitionReuseMode, UE::MeshPartition::EMeshPartitionReuseMode::Count);

namespace UE::MeshPartition
{
// Example Command Line: ProjectName MapName -run=WorldPartitionBuilderCommandlet -SCCProvider=Perforce -Builder=WorldPartitionMeshPartitionBuilder
UCLASS()
class UWorldPartitionMeshPartitionBuilder : public UWorldPartitionBuilder
{
	GENERATED_UCLASS_BODY()

	enum class ETaskState
	{
		Invalid,
		/** Task is ready to execute */
		Ready,
		/**	The modifier and base actor are loaded */
		ActorLoaded,
		/**	Launched the async process modifier task, but the task has not yet started */
		LaunchedProcessModifierTask,
		/**	The task is currently processing the modifiers, building the FDynamicMesh3 */
		ProcessingModifiers,
		/**	Modifiers are processed. Built mesh data is ready */
		ModifiersProcessed,
		/**	The Transformers pipeline is launched and processing. */
		TransformersLaunched,
		/**	The Transformers pipeline execution is done. */
		TransformersExecuted,
		/**	Task's work is done */
		Completed
	};

	enum class ETaskResult
	{
		/* Initial state, not yet complete */
		None,
		/* Failed to build a new compiled section */
		BuildFail,
		/* Successfully built a new compiled section */
		BuildSuccess,
		/* Reused an existing compiled section */
		ReuseExisting
	};
	
	struct FModifierGroupTask
	{
		// Inputs
		UE::MeshPartition::FModifierGroup		ModifierGroup;
		MeshPartition::FBuilderSettings 		BuilderSettings;
		MeshPartition::FCompiledSectionBuildInfo	BuildInfo;
		const MeshPartition::FCompiledSectionBuildVariant&		BuildVariant;
		UMeshPartitionEditorComponent*			MeshPartitionEditorComponent = nullptr;
		UWorldPartitionMeshPartitionBuilder&		WorldPartitionMeshPartitionBuilder;

		// array of pre-existing compiled sections (potentially re-usable for incremental rebuild)
		// NOTE: this is NOT thread safe, and only works because we always synchronously complete this task
		// and deallocate it  before destroying the array this view references.
		TArrayView<const FCompiledSectionDescriptor> OldCompiledSectionDescriptors;

		// Grid configuration for this variant (CellSize=0 = no grid splitting). Set once per variant, before the task loop.
		MeshPartition::FGridSettings		GridSettings;

		// Results
		ETaskResult							Result = ETaskResult::None;

		// All compiled sections produced for this group (N per WP grid cell; 1 for single-section path)
		TArray<MeshPartition::ACompiledSection*>		CompiledSections;

		TArray<FGuid>						ReuseCompiledSectionActorDescGuids;

		// internal intermediate state
		ETaskState							State = ETaskState::Invalid;

		TUniquePtr<MeshPartition::FTransformerContext> TransformerContext;
		UE::Tasks::TTask<FMeshData>				ProcessModifierTask;
		TSharedPtr<MeshPartition::FModifierTaskGraph> 			ModifierTaskGraph;

		TArray<UPackage*>					ModifiedPackages;
		TSet<AActor*>						ModifierOwners;			// set of actors containing one of our modifier components
		TSet<FWorldPartitionReference>		ActorRefs;				// references to world partition actors that must remain loaded while this task is running
		
		FModifierGroupTask(UE::MeshPartition::FModifierGroup&& InModifierGroup, const MeshPartition::FBuilderSettings& InBuilderSettings, const MeshPartition::FCompiledSectionBuildInfo& InBuildInfo, const MeshPartition::FCompiledSectionBuildVariant& InBuildVariant, UMeshPartitionEditorComponent* InMeshPartitionEditorComponent, UWorldPartitionMeshPartitionBuilder& InWorldPartitionMeshPartitionBuilder, TArrayView<const FCompiledSectionDescriptor> InOldCompiledSectionDescriptors)
			: ModifierGroup(MoveTemp(InModifierGroup))
			, BuilderSettings(InBuilderSettings)
			, BuildInfo(InBuildInfo)
			, BuildVariant(InBuildVariant)
			, MeshPartitionEditorComponent(InMeshPartitionEditorComponent)
			, WorldPartitionMeshPartitionBuilder(InWorldPartitionMeshPartitionBuilder)
			, OldCompiledSectionDescriptors(InOldCompiledSectionDescriptors)
		{}

		FModifierGroupTask(const UE::MeshPartition::FModifierGroup& InModifierGroup, const MeshPartition::FBuilderSettings& InBuilderSettings, const MeshPartition::FCompiledSectionBuildInfo& InBuildInfo, const MeshPartition::FCompiledSectionBuildVariant& InBuildVariant, UMeshPartitionEditorComponent* InMeshPartitionEditorComponent, UWorldPartitionMeshPartitionBuilder& InWorldPartitionMeshPartitionBuilder, TArrayView<const FCompiledSectionDescriptor> InOldCompiledSectionDescriptors)
			: ModifierGroup(InModifierGroup)
			, BuilderSettings(InBuilderSettings)
			, BuildInfo(InBuildInfo)
			, BuildVariant(InBuildVariant)
			, MeshPartitionEditorComponent(InMeshPartitionEditorComponent)
			, WorldPartitionMeshPartitionBuilder(InWorldPartitionMeshPartitionBuilder)
			, OldCompiledSectionDescriptors(InOldCompiledSectionDescriptors)
		{}
	};

public:
	// UWorldPartitionBuilder interface begin
	virtual bool RequiresCommandletRendering() const override { return false; }
	virtual ELoadingMode GetLoadingMode() const override { return ELoadingMode::Custom; }
	virtual bool PreRun(UWorld* World, FPackageSourceControlHelper& PackageHelper) override;
	virtual bool RunInternal(UWorld* World, const FCellInfo& InCellInfo, FPackageSourceControlHelper& InPackageHelper) override;
	virtual bool CanProcessNonPartitionedWorlds() const override { return false; }
	// UWorldPartitionBuilder interface end

	static bool CanBuildMeshPartitions(const UWorld* InWorld, FName InBuildOption);
	static EEditorBuildResult BuildMeshPartitions(UWorld* InWorld, EMeshPartitionReuseMode ReuseMode);

private:
	/**
	* Build a MeshPartition
	*/
	bool BuildMeshPartition(const AMeshPartition* InMeshPartition, const FGuid& BuildKey, UE::MeshPartition::FModifierDescriptorCache& DescriptorCache, FSourceControlHelper& InSourceControlHelper);

	/**
	* Executes and manages all modifier group tasks upon completion.
	* @Param InModifierGroupTasks The modifier group tasks to execute.
	* @param InWorld The current world.
	* @param InMeshPartition The AMeshPartition the modifiers belong to.
	* @param InSourceControlHelper A helper for saving and source control operations.
	*/
	void ExecuteModifierGroupTasks(TIndirectArray<FModifierGroupTask>& InModifierGroupTasks, UWorld* InWorld, const AMeshPartition* InMeshPartition, FSourceControlHelper& InSourceControlHelper);

	/**
	* Loads the modifier owners. Populating the lookup table and refcounting the loaded modifier owners.
	* @param InMeshPartition The AMeshPartition the modifiers belong to.
	* @param InModifierGroupTask The ModifierGroupTask that needs processing.
	*/
	void LoadActors(const AMeshPartition* InMeshPartition, FModifierGroupTask& InModifierGroupTask);

	void AssignSectionToPlatformDataLayers(MeshPartition::ACompiledSection* InCompiledSection) const;

	/**
	* Gather the set of all package dependencies from the modifiers in the ModifierGroup.
	* @param AssetRegistry the asset registry we use to track recursive dependencies.  Must be up to date before calling this function for correct results.
	* @param ModifierGroup the modifier group to gather dependencies from
	* @return An array of package names for packages containing dependencies of the modifier group, sorted in a deterministic order.
	*/
	void GatherDependenciesAndComputeHashes(FModifierGroupTask& InOutTask, IAssetRegistry& AssetRegistry);

	/**
	* Helper to save a package.
	* @param PackagesToSave The package to save.
	* @param InSourceControlHelper A helper for saving and source control operations.
	* @return 
	*/
	static bool SavePackages(const TArray<UPackage*>& PackagesToSave, FSourceControlHelper& InSourceControlHelper);

	/**
	* Waits for all compilable before saving and cleaning resources if memory is needed and the resources are not needed anymore.
	* @param InWorld The current world.
	* @param InSourceControlHelper A helper for saving and source control operations.
	* @param InModifierGroupTask The ModifierGroupTask that needs saving.
	*/
	void SaveAndCollectGarbage(UWorld* InWorld, FSourceControlHelper& InSourceControlHelper, FModifierGroupTask& InModifierGroupTask);

	bool SubmitToSourceControl(UWorld* InWorld);

private:
	UPROPERTY(meta=(CommandLineArgument))
	EMeshPartitionReuseMode ReuseMode = EMeshPartitionReuseMode::ByPackageHash;

	UPROPERTY(meta=(CommandLineArgument))
	int32 MaxTasksInFlight = 16;

	TObjectPtr<UWorldPartition> WorldPartition = nullptr;

	// Build state
	TSet<FGuid> CompiledSectionActorDescGuidsToKeep;
	int32 TotalBuiltCompiledSections = 0;
	int32 TotalFailedCompiledSections = 0;
	int32 TotalReusedCompiledSections = 0;
	int32 TotalNumberOfSectionsToBuild = 0;

	// Tracks all package files (added / edited / deleted) produced during the build, for source control submission.
	FBuilderModifiedFiles ModifiedFiles;
};
} // namespace UE::MeshPartition