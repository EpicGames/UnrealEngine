// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshPartitionEditorUI : ModuleRules
{
	public MeshPartitionEditorUI(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bAllowUETypesInNamespaces = true;
		
		PublicIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			}
			);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"DynamicMesh",
				"Engine",
				"GeometryCore",
				"GeometryFramework",
				"GeometryAlgorithms",
				"GeometryScriptingEditor",
				"InputCore",
				"MeshPartition",
				"MeshPartitionEditor",
				"MeshConversion",
				"PropertyEditor",
				"RenderCore",
				"Renderer",
				"RHI",
				"ToolMenus",
				"Slate",
				"SlateCore",
				"SourceControl",
				"TypedElementFramework",
				"TypedElementRuntime",
				"UnrealEd",
				"Projects",
				"LevelEditor",
				"SceneOutliner",
				"WorkspaceMenuStructure",
				"TedsEditorCompatibility",
				"TedsTableViewer",
				"TedsQueryStack",
				"TedsOutliner",
				"ToolWidgets",
				"ModelingEditorUI"
			}
			);
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
