// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorUIModule.h"

#include "Components/StaticMeshComponent.h"
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "EditorBuildUtils.h"
#include "Engine/StaticMesh.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "LevelEditor.h"
#include "LevelEditorOutlinerSettings.h"
#include "Interfaces/IPluginManager.h"
#include "MeshPartitionChannel.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorUIStyle.h"
#include "MeshPartitionModifiersCustomizations.h"
#include "MeshPartitionToggleableConstraintNameCustomization.h"
#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"
#include "Modifiers/MeshPartitionSplineModifier.h"
#include "PropertyEditorModule.h"
#include "ShaderCore.h"
#include "ToolMenus.h"
#include "UI/MeshPartitionSettingsWidget.h"
#include "UnrealEdGlobals.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "UI/MeshPartitionSettingsWidget.h"
#include "MeshPartitionEditorUIStyle.h"
#include "MeshPartitionModifiersCustomizations.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "FMegaMeshEditorUIModule"

DEFINE_LOG_CATEGORY(LogMegaMeshEditorUI);

namespace UE::MeshPartition
{
void FMegaMeshEditorUIModule::StartupModule()
{
	FMegaMeshEditorUIStyle::Initialize();

	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FMegaMeshEditorUIModule::OnPostEngineInit);

	bHasRegisteredTabSpawners = false;
	RegisterTabSpawners(nullptr);

	FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
	ScalabilityMenuExtender = MakeShared<FExtender>();

	ScalabilityMenuExtender->AddMenuExtension("PerformanceAndScalability",
							EExtensionHook::After,
							nullptr,
							FMenuExtensionDelegate::CreateRaw(this, &FMegaMeshEditorUIModule::AddScalabilitySubMenu));

	LevelEditorModule.GetMenuExtensibilityManager()->AddExtender(ScalabilityMenuExtender);
}

void FMegaMeshEditorUIModule::ShutdownModule()
{
	if (ScalabilityMenuExtender.IsValid())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.GetMenuExtensibilityManager()->RemoveExtender(ScalabilityMenuExtender);
	}

	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);

	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		for (FName ClassName : ClassesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomClassLayout(ClassName);
		}
		for (FName PropertyName : PropertiesToUnregisterOnShutdown)
		{
			PropertyEditorModule->UnregisterCustomPropertyTypeLayout(PropertyName);
		}
	}

	UnregisterTabSpawners();
}

void FMegaMeshEditorUIModule::OnPostEngineInit()
{
	RegisterPropertyEditorCategories();
}

void FMegaMeshEditorUIModule::RegisterPropertyEditorCategories()
{
	static const FName PropertyEditor("PropertyEditor");
	FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(PropertyEditor);

	PropertyModule.RegisterCustomClassLayout("ProjectMeshLayersModifier",
		FOnGetDetailCustomizationInstance::CreateStatic(&FMegaMeshProjectSculptLayersModifierDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(MeshPartition::UProjectMeshLayersModifier::StaticClass()->GetFName());

	PropertyModule.RegisterCustomClassLayout("TexturePatchModifier", 
		FOnGetDetailCustomizationInstance::CreateStatic(&FMegaMeshTexturePatchDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(MeshPartition::UTexturePatchModifier::StaticClass()->GetFName());

	PropertyModule.RegisterCustomClassLayout("SplineModifier",
		FOnGetDetailCustomizationInstance::CreateStatic(&FSplineSoftObjectPointerDetails::MakeInstance));
	ClassesToUnregisterOnShutdown.Add(MeshPartition::USplineModifier::StaticClass()->GetFName());

	PropertyModule.RegisterCustomPropertyTypeLayout(MeshPartition::FChannelName::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::MeshPartition::FToggleableConstraintNameCustomization::MakeInstance));
	PropertiesToUnregisterOnShutdown.Add(MeshPartition::FChannelName::StaticStruct()->GetFName());

	PropertyModule.RegisterCustomPropertyTypeLayout(MeshPartition::FPriorityLayerName::StaticStruct()->GetFName(),
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&UE::MeshPartition::FToggleableConstraintNameCustomization::MakeInstance));
	PropertiesToUnregisterOnShutdown.Add(MeshPartition::FPriorityLayerName::StaticStruct()->GetFName());

	// ModifierComponent Sections
	// All modifier are registered on the base class so that the "Mesh Partition" section button works when the actor is selected (it doesn't drill down to subclasses)
	{
		TSharedRef<FPropertySection> Section = PropertyModule.FindOrCreateSection("ModifierComponent", "MeshPartition", LOCTEXT("MeshPartition", "Mesh Partition"));
		// Base class categories (from UModifierComponent - includes Modifier|* subcategories)
		Section->AddCategory("Modifier");
		Section->AddCategory("ModifierSettings");
		Section->AddCategory("Advanced");
		Section->AddCategory("Visualization");
		Section->AddCategory("Extent");
		Section->AddCategory("Transform");
		Section->AddCategory("Tags");
		// Subclass categories (scraped from Modifiers/*.h headers and detail customizations)
		Section->AddCategory("AdaptiveTessellation");
		Section->AddCategory("AlphaBlending");
		Section->AddCategory("AttributeWeights");
		Section->AddCategory("Boolean");
		Section->AddCategory("Channels");
		Section->AddCategory("Coverage");
		Section->AddCategory("CurveAdjustment");
		Section->AddCategory("DensityWeightChannel");
		Section->AddCategory("EdgeFalloff");
		Section->AddCategory("Falloff");
		Section->AddCategory("HeightDisplacement");
		Section->AddCategory("Instance");
		Section->AddCategory("Interior");
		Section->AddCategory("Interpolation");
		Section->AddCategory("Lattice");
		Section->AddCategory("Mesh");
		Section->AddCategory("MeshLayers");
		Section->AddCategory("Noise");
		Section->AddCategory("NoiseTransform");
		Section->AddCategory("Patch");
		Section->AddCategory("Profile");
		Section->AddCategory("Projection");
		Section->AddCategory("Remesh");
		Section->AddCategory("RemeshOperation");
		Section->AddCategory("Settings");
		Section->AddCategory("Spline");
		Section->AddCategory("Tessellate");
		Section->AddCategory("Visualize");
		Section->AddCategory("Weight");
		Section->AddCategory("WeightAttribute");
		Section->AddCategory("WeightChannels");
	}
}

void FMegaMeshEditorUIModule::RegisterTabSpawners(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup)
{
	if (bHasRegisteredTabSpawners)
	{
		UnregisterTabSpawners();
	}

	FTabSpawnerEntry& MegaMeshSettingsSpawnerEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner("MeshPartitionOutliner",
		FOnSpawnTab::CreateRaw(this, &FMegaMeshEditorUIModule::MakeMegaMeshSettingsTab))
		.SetDisplayName(LOCTEXT("MeshPartitionEditorMeshPartitionOutlinerTitle", "Mesh Partition Outliner"))
		.SetTooltipText(LOCTEXT("MeshPartitionEditorMeshPartitionOutlinerTooltipText", "Open the Mesh Partition Outliner tab, which allows control over Mesh Partition processing layers and modifier visuals."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsMiscCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ClassIcon.LandscapeComponent"));

	if (WorkspaceGroup.IsValid())
	{
		MegaMeshSettingsSpawnerEntry.SetGroup(WorkspaceGroup.ToSharedRef());
	}

	bHasRegisteredTabSpawners = true;
}

void FMegaMeshEditorUIModule::UnregisterTabSpawners()
{
	bHasRegisteredTabSpawners = false;

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner("MeshPartitionOutliner");
}

TSharedRef<SDockTab> FMegaMeshEditorUIModule::MakeMegaMeshSettingsTab(const FSpawnTabArgs&)
{
	TSharedRef<SDockTab> WidgetReflectorTab = SNew(SDockTab)
		.TabRole(NomadTab);
	WidgetReflectorTab->SetContent(GetMegaMeshSettings(WidgetReflectorTab));
	return WidgetReflectorTab;
}

TSharedRef<SWidget> FMegaMeshEditorUIModule::GetMegaMeshSettings(const TSharedRef<SDockTab>& InParentTab)
{
	TSharedPtr<SMegaMeshSettingsWidget> MegaMeshSettings = MegaMeshSettingsPtr.Pin();

	if (!MegaMeshSettings.IsValid())
	{
		MegaMeshSettings = SNew(SMegaMeshSettingsWidget);
		MegaMeshSettingsPtr = MegaMeshSettings;
	}

	return MegaMeshSettings.ToSharedRef();
}

void FMegaMeshEditorUIModule::AddScalabilitySubMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.AddSubMenu(LOCTEXT("MegaMeshScalability", "Mesh Partition Scalability"),
							LOCTEXT("MegaMeshScalability_Tooltip", "Mesh Partition Scalability settings"),
							FNewMenuDelegate::CreateRaw(this, &FMegaMeshEditorUIModule::FillScalabilitySubmenu));
}

void FMegaMeshEditorUIModule::FillScalabilitySubmenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection(NAME_None, LOCTEXT("MegaMeshPreviewSectionScalabilitySettings", "Preview Section Simplification"));
	{
		InMenuBuilder.AddWidget(CreateEnablePreviewSectionSimplificationWidget(),
								LOCTEXT("MegaMeshScalabilityEnablePreviewSectionSimplification", "Enable"),
								true,
								true,
								LOCTEXT("MegaMeshScalabilityEnablePreviewSectionSimplification_Tooltip", "Enables PreviewSection Simplification."));

		InMenuBuilder.AddWidget(CreatePreviewSectionSimplificationEdgeLengthWidget(),
								LOCTEXT("MegaMeshScalabilityPreviewSectionSimplificationEdgeLength", "Edge Length"),
								true,
								true,
								LOCTEXT("MegaMeshScalabilityPreviewSectionSimplificationEdgeLength_Tooltip", "Target Edge Length after simplification."));

		InMenuBuilder.AddWidget(CreatePreviewSectionSimplificationMinVertexNumberWidget(),
								LOCTEXT("MegaMeshScalabilityPreviewSectionSimplificationMinVertexNumber", "Min Vertex Number"),
								true,
								true,
								LOCTEXT("MegaMeshScalabilityPreviewSectionSimplificationMinVertexNumber_Tooltip", "Min Vertex Number needed for the simplification to affect preview sections."));
	}
	InMenuBuilder.EndSection();
}

TSharedRef<SWidget> FMegaMeshEditorUIModule::CreateEnablePreviewSectionSimplificationWidget()
{
	return SNew(SBox)
	.HAlign(HAlign_Right)
	[
		SNew(SBox)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.WidthOverride(100.0f)
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([]()
			{
			return CVarMegaMeshEnablePreviewSimplification->GetBool() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			.OnCheckStateChanged_Lambda([](const ECheckBoxState NewState)
			{
				const bool bEnabled = NewState == ECheckBoxState::Checked;
				CVarMegaMeshEnablePreviewSimplification->Set(bEnabled);
			})
		]
	];
}

TSharedRef<SWidget> FMegaMeshEditorUIModule::CreatePreviewSectionSimplificationEdgeLengthWidget()
{
	return SNew(SBox)
	.HAlign(HAlign_Right)
	[
		SNew(SBox)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.WidthOverride(100.0f)
		.IsEnabled_Lambda([]()
		{
			return CVarMegaMeshEnablePreviewSimplification->GetBool();
		})
		[
			SNew(SBorder)
			.Padding(FMargin(1.0f))
			[
				SNew(SSpinBox<float>)
				.Style(&FAppStyle::Get(), "Menu.SpinBox")
				.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				//todo(luc.eygasier): define these values somewhere, probably project settings.
				.MinSliderValue(0.1)
				.MaxSliderValue(100)
				.Value_Lambda([]()
				{
					return CVarMegaMeshPreviewSimplificationEdgeLength->GetFloat();
				})
				.OnValueChanged_Lambda([](const float NewValue)
				{
					CVarMegaMeshPreviewSimplificationEdgeLength->Set(NewValue);
				})
			]
		]
	];
}

TSharedRef<SWidget> FMegaMeshEditorUIModule::CreatePreviewSectionSimplificationMinVertexNumberWidget()
{
	return SNew(SBox)
	.HAlign(HAlign_Right)
	[
		SNew(SBox)
		.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
		.WidthOverride(100.0f)
		.IsEnabled_Lambda([]()
		{
			return CVarMegaMeshEnablePreviewSimplification->GetBool();
		})
		[
		SNew(SBorder)
		.Padding(FMargin(1.0f))
		[
			SNew(SSpinBox<int32>)
			.Style(&FAppStyle::Get(), "Menu.SpinBox")
			.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				//todo(luc.eygasier): define these values somewhere, probably project settings.
			.MinSliderValue(1)
			.MaxSliderValue(5000000)
			.Value_Lambda([]()
				{
					return CVarMegaMeshPreviewSimplificationMinVertexNumber->GetInt();
				})
			.OnValueChanged_Lambda([](const int32 NewValue)
				{
					CVarMegaMeshPreviewSimplificationMinVertexNumber->Set(NewValue);
				})
		]
		]
	];
}

IMPLEMENT_MODULE(FMegaMeshEditorUIModule, MeshPartitionEditorUI)
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
