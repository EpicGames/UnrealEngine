// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionEditorUIStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "Trace/Detail/LogScope.inl"
#include "SlateOptMacros.h"

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FMegaMeshEditorUIStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )
#define SVG_PLUGIN_BRUSH( RelativePath, ... ) FSlateVectorImageBrush( FMegaMeshEditorUIStyle::InContent( RelativePath, ".svg" ), __VA_ARGS__ )

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FMegaMeshEditorUIStyle::StyleInstance = nullptr;

FString FMegaMeshEditorUIStyle::InContent(const FString& InRelativePath, const ANSICHAR* InExtension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MeshPartition"))->GetContentDir();
	return (ContentDir / InRelativePath) + InExtension;
}

void FMegaMeshEditorUIStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FMegaMeshEditorUIStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FMegaMeshEditorUIStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("MegaMeshEditorUIStyle"));
	return StyleSetName;
}

const FVector2D Icon8x8(8.0f, 8.0f);
const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);
const FVector2D Icon64x64(64.0f, 64.0f);

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
TSharedRef< FSlateStyleSet > FMegaMeshEditorUIStyle::Create()
{
	TSharedRef<FSlateStyleSet> Style = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("MeshPartition"))->GetContentDir());
	Style->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	Style->Set("MeshPartition.ViewportToolBar", new SVG_PLUGIN_BRUSH("Icons/ModelingSphere", Icon16x16));

	Style->Set("MeshPartitionDefinition", new SVG_PLUGIN_BRUSH("Icons/ModelingSphere", Icon16x16));

	/* Style for a checkbox to represent visibility in the editor level */
	static const FCheckBoxStyle VisibilityToggleStyle = FCheckBoxStyle()
		.SetCheckBoxType(ESlateCheckBoxType::ToggleButton)
		.SetUncheckedImage(SVG_PLUGIN_BRUSH("Icons/NoVisibility", Icon20x20))
		.SetUncheckedHoveredImage(SVG_PLUGIN_BRUSH("Icons/NoVisibility", Icon20x20))
		.SetUncheckedPressedImage(SVG_PLUGIN_BRUSH("Icons/NoVisibility", Icon20x20))
		.SetCheckedImage(SVG_PLUGIN_BRUSH("Icons/Visibility", Icon20x20))
		.SetCheckedHoveredImage(SVG_PLUGIN_BRUSH("Icons/Visibility", Icon20x20))
		.SetCheckedPressedImage(SVG_PLUGIN_BRUSH("Icons/Visibility", Icon20x20));
	Style->Set("VisibilityToggleStyle", VisibilityToggleStyle);

	FSlateColor SelectorColor = FAppStyle::GetSlateColor("SelectorColor");
	FSlateColor SelectionColor = FAppStyle::GetSlateColor("SelectionColor");
	FSlateColor SelectionColor_Inactive = FAppStyle::GetSlateColor("SelectionColor_Inactive");
	FSlateColor SelectionColor_Pressed = FAppStyle::GetSlateColor("SelectionColor_Pressed");

	Style->Set("AutoVisibleIcon16x", new SVG_PLUGIN_BRUSH("Icons/visible_secondary", Icon16x16));
	Style->Set("AutoVisibleHighlightIcon16x", new SVG_PLUGIN_BRUSH("Icons/visible_secondary", Icon16x16));
	Style->Set("VisibleIcon16x", new SVG_PLUGIN_BRUSH("Icons/visible", Icon16x16));
	Style->Set("VisibleHighlightIcon16x", new SVG_PLUGIN_BRUSH("Icons/visible", Icon16x16));
	Style->Set("NotVisibleIcon16x", new SVG_PLUGIN_BRUSH("Icons/hidden", Icon16x16));
	Style->Set("NotVisibleHighlightIcon16x", new SVG_PLUGIN_BRUSH("Icons/hidden", Icon16x16));

	Style->Set("BuildToHeader", new SVG_PLUGIN_BRUSH("Icons/BuildToHeader", Icon16x16));
	Style->Set("BoundingBoxHeader", new SVG_PLUGIN_BRUSH("Icons/ModifierBounds_16", Icon16x16));

	Style->Set("VisibleAutoColor", FSlateColor(FLinearColor::Yellow));
	Style->Set("VisibleColor", FSlateColor(FLinearColor::Green));
	Style->Set("HiddenColor", FSlateColor(FLinearColor::Red));

	Style->Set("BlackBorderFillBrush", new FSlateColorBrush(FStyleColors::Panel));
	Style->Set("BuildCostToolTipTitle", new FSlateColorBrush(FStyleColors::Header));

	Style->Set("BuildCostLow", new SVG_PLUGIN_BRUSH("Icons/BuildCostLow", Icon16x16));
	Style->Set("BuildCostMedium", new SVG_PLUGIN_BRUSH("Icons/BuildCostMedium", Icon16x16));
	Style->Set("BuildCostHigh", new SVG_PLUGIN_BRUSH("Icons/BuildCostHigh", Icon16x16));
	Style->Set("BuildCostAggregate", new SVG_PLUGIN_BRUSH("Icons/BuildTo_Circle", Icon16x16));

	auto SetModifierIcon = [&Style](const FName PropertyName, const FString& RelativePath)
	{
		const FString PropertyNameStr = PropertyName.ToString();
		const FName ClassIconName("ClassIcon." + PropertyNameStr);
		const FName ClassThumbnailName("ClassThumbnail." + PropertyNameStr);
		Style->Set(PropertyName, new SVG_PLUGIN_BRUSH(RelativePath, Icon16x16));
		Style->Set(ClassIconName, new SVG_PLUGIN_BRUSH(RelativePath, Icon16x16));
		Style->Set(ClassThumbnailName, new SVG_PLUGIN_BRUSH(RelativePath, Icon64x64));
	};

	SetModifierIcon("BooleanModifier", "Icons/ModifierBoolean_16");
	SetModifierIcon("NoiseModifier", "Icons/ModifierNoise_16");
	SetModifierIcon("PatchModifier", "Icons/ModifierPatch_16");
	SetModifierIcon("InstancedPatchModifier", "Icons/ModifierPatchInstance_16");
	SetModifierIcon("SplineModifier", "Icons/ModifierPath_16");
	SetModifierIcon("MeshProjectModifier", "Icons/ModifierProjection_16");
	SetModifierIcon("RemeshModifier", "Icons/ModifierRemesh_16");
	SetModifierIcon("ProjectMeshLayersModifier", "Icons/ModifierSculpt_16");
	SetModifierIcon("TexturePatchModifier", "Icons/ModifierTexturePatch_16");
	SetModifierIcon("LatticeModifier", "Icons/ModifierLattice_16");
	// TODO:
	//SetModifierIcon("MegaMeshInstancedProjectionModifier", "Icons/ModifierInstancedProjection_16");
	//SetModifierIcon("MegaMeshSplineRemeshModifier", "Icons/ModifierSplineRemesh_16");

	Style->Set("BuildTo_Disabled", new SVG_PLUGIN_BRUSH("Icons/BuildTo_CircleHidden", Icon16x16));
	Style->Set("BuildTo_Target", new SVG_PLUGIN_BRUSH("Icons/BuildTo_Circle", Icon16x16));
	Style->Set("BuildTo_Layer_Auto", new SVG_PLUGIN_BRUSH("Icons/BuildTo_UpDown_Slim", Icon16x16));
	Style->Set("BuildTo_Layer_Targeted", new SVG_PLUGIN_BRUSH("Icons/BuildTo_CircleDown", Icon16x16));
	Style->Set("BuildTo_Modifier_Auto", new SVG_PLUGIN_BRUSH("Icons/BuildTo_UpDown_Slim", Icon16x16));
	Style->Set("BuildTo_Modifier_Targeted", new SVG_PLUGIN_BRUSH("Icons/BuildTo_CircleDown_Slim", Icon16x16));





	FToolBarStyle FilterToolBar = FToolBarStyle(FAppStyle::GetWidgetStyle<FToolBarStyle>("SlimToolBar"))
		.SetBackground(FSlateNoResource())
		.SetLabelPadding(FMargin(0))
		.SetComboButtonPadding(FMargin(0))
		.SetBlockPadding(FMargin(3, 0))
		.SetIndentedBlockPadding(FMargin(0))
		.SetBackgroundPadding(FMargin(0))
		.SetButtonPadding(FMargin(0))
		.SetCheckBoxPadding(FMargin(0))
		.SetSeparatorBrush(FSlateNoResource())
		.SetSeparatorPadding(FMargin(0));

	Style->Set("OutlinerToolbar", FilterToolBar);

	Style->Set("OutlinerButton", FButtonStyle(FAppStyle::GetWidgetStyle<FButtonStyle>("Button"))
		.SetNormal(FSlateNoResource())
		.SetHovered(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor))
		.SetPressed(BOX_BRUSH("Common/FlatButton", 2.0f / 8.0f, SelectionColor_Pressed))
		.SetDisabled(FSlateNoResource())
		.SetNormalPadding(FMargin(0, 0))
		.SetPressedPadding(FMargin(0, 0))
	);

	return Style;
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FMegaMeshEditorUIStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

TSharedRef<ISlateStyle> FMegaMeshEditorUIStyle::Get()
{
	return StyleInstance.ToSharedRef();
}

#undef IMAGE_PLUGIN_BRUSH
#undef SVG_PLUGIN_BRUSH
#undef RootToContentDir