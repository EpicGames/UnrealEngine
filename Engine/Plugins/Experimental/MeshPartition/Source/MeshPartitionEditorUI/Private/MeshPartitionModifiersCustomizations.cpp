// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModifiersCustomizations.h"

#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "IDetailCustomization.h"
#include "IDetailGroup.h"
#include "DetailCategoryBuilder.h"
#include "ObjectEditorUtils.h"
#include "PropertyHandle.h"
#include "Engine/SimpleConstructionScript.h"
#include "Engine/SCS_Node.h"
#include "Engine/BlueprintGeneratedClass.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"

#include "ModelingWidgets/SMeshLayersStack.h"
#include "SculptLayersModifierController.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"
#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Modifiers/MeshPartitionSplineModifier.h"

#include <type_traits>

#define LOCTEXT_NAMESPACE "MegaMeshModifiersCustomizations"

namespace UE::MeshPartition
{
FMegaMeshProjectSculptLayersModifierDetails::FMegaMeshProjectSculptLayersModifierDetails()
	: Controller(MakeShared<FSculptLayersModifiersController>()) { }

TSharedRef<IDetailCustomization> FMegaMeshProjectSculptLayersModifierDetails::MakeInstance()
{
	return MakeShareable(new FMegaMeshProjectSculptLayersModifierDetails);
}

FMegaMeshProjectSculptLayersModifierDetails::~FMegaMeshProjectSculptLayersModifierDetails()
{
	if (Modifier.IsValid())
	{
		Modifier.Get()->OnMeshLayersPanelRequestRebuild.Remove(ModifierUpdatesHandle);
	}
}

void FMegaMeshProjectSculptLayersModifierDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);
	MeshPartition::UProjectMeshLayersModifier* SLModifier = Cast<MeshPartition::UProjectMeshLayersModifier>(ObjectsBeingCustomized[0]);
	Modifier = SLModifier;
	Controller->SetProperties(SLModifier);

	TSharedPtr<IPropertyHandle> LayerWeightsHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(MeshPartition::UProjectMeshLayersModifier, LayerWeights), MeshPartition::UProjectMeshLayersModifier::StaticClass());
	ensure(LayerWeightsHandle->IsValidHandle());

	DetailBuilder.EditDefaultProperty(LayerWeightsHandle)->CustomWidget()
		.OverrideResetToDefault(FResetToDefaultOverride::Hide())
		.WholeRowContent()
	[
		SNew(SMeshLayersStack, Controller)
		.InAllowAddRemove(false)
		.InAllowReordering(false)
	];

	// ensure this stack is rebuilt when Mesh Layers are updated elsewhere in the editor
	TWeakPtr<IMeshLayersController> WeakController = Controller;
	ModifierUpdatesHandle = Modifier.Get()->OnMeshLayersPanelRequestRebuild.AddLambda([WeakController]() {
		if (TSharedPtr<IMeshLayersController> PinnedController = WeakController.Pin())
		{
			PinnedController->RefreshLayersStackView();
		}
	});
}

//======================================================================================
// Texture Patch Details Customization
//======================================================================================

TSharedRef<IDetailCustomization> FMegaMeshTexturePatchDetails::MakeInstance()
{
	return MakeShareable(new FMegaMeshTexturePatchDetails());
}



void FMegaMeshTexturePatchDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	ReparentHeightDisplacement(DetailBuilder);
	AddWeightChannelCopyButtons(DetailBuilder);
	AddTessellationCheckbox(DetailBuilder);

	DetailBuilder.GetDetailsViewSharedPtr()->InvalidateCachedState();
	DetailBuilder.ForceRefreshDetails();
}


void FMegaMeshTexturePatchDetails::ReparentHeightDisplacement(IDetailLayoutBuilder& DetailBuilder)
{
	// Instead of
	//
	// - HeightDisplacement
	//   - Input                [ dropdown ]
	//     - PropertyA
	//     - InnerCategory
	//       - Inner property
	//       - ...
	//
	// we want
	// - HeightDisplacement
	//   - PropertyA
	//   - InnerCategory
	//     - Inner property
	//       - ...
	//
	// The following code walks the properties and hangs them directly under the main category
	// or creates new detail groups for the sub-categories.
	//

	TSharedRef<IPropertyHandle> HeightChannelHandle =
		DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(MeshPartition::UTexturePatchModifier, HeightChannel));

	IDetailCategoryBuilder& HeightCategory = DetailBuilder.EditCategory("HeightDisplacement");

	// Hide the original UObject property to remove it from the panel
	DetailBuilder.HideProperty(HeightChannelHandle);

	UObject* HeightChannelObject = nullptr;
	HeightChannelHandle->GetValue(HeightChannelObject);


	if (HeightChannelObject)
	{
		// Maps category names to their detail groups
		TMap<FName, IDetailGroup*> CreatedGroups;

		// Should appear at the top, so add them first.
		const TArray<FName> PriorityProperties = {
			GET_MEMBER_NAME_CHECKED(MeshPartition::UTexturePatchHeightEntry, TextureAsset),
			GET_MEMBER_NAME_CHECKED(MeshPartition::UTexturePatchHeightEntry, TextureChannelIndex) };

		FName TextureAssetName = GET_MEMBER_NAME_CHECKED(MeshPartition::UTexturePatchHeightEntry, TextureAsset);

		for (const FName& PriotityPropertyName : PriorityProperties)
		{
			TSharedPtr<IPropertyHandle> ChildHandle = DetailBuilder.AddObjectPropertyData({ HeightChannelObject }, PriotityPropertyName);
			if (ChildHandle)
			{
				HeightCategory.AddProperty(ChildHandle);
			}
		}

		// Iterate over all child properties
		for (TFieldIterator<FProperty> PropIt(HeightChannelObject->GetClass()); PropIt; ++PropIt)
		{
			FProperty* ChildProperty = *PropIt;

			if (!ensure(ChildProperty))
			{
				continue;
			}

			if (PriorityProperties.Contains(ChildProperty->GetFName()))
			{
				// already added.
				continue;
			}

			if (ChildProperty->HasAnyPropertyFlags(CPF_Edit | CPF_EditConst))
			{
				TSharedPtr<IPropertyHandle> ChildHandle = DetailBuilder.AddObjectPropertyData({ HeightChannelObject }, ChildProperty->GetFName());

				if (!ChildHandle)
				{
					continue;
				}

				const FName InnerCategoryName = FObjectEditorUtils::GetCategoryFName(ChildProperty);

				if (InnerCategoryName.IsNone())
				{
					HeightCategory.AddProperty(ChildHandle);
				}
				else
				{
					// Find or create a group for this inner category
					IDetailGroup* InnerGroup = CreatedGroups.FindRef(InnerCategoryName);

					if (!InnerGroup)
					{
						FText InnerCategoryDisplay = FText::FromName(InnerCategoryName);
						InnerGroup = &HeightCategory.AddGroup(InnerCategoryName, InnerCategoryDisplay);
						CreatedGroups.Add(InnerCategoryName, InnerGroup);
					}

					InnerGroup->AddPropertyRow(ChildHandle.ToSharedRef());
				}
			}
		}
	}
}

void FMegaMeshTexturePatchDetails::AddWeightChannelCopyButtons(IDetailLayoutBuilder& DetailBuilder)
{
	// Get the objects currently being customized (usually only one)
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() == 1)
	{
		if (MeshPartition::UTexturePatchModifier* Modifier = Cast<MeshPartition::UTexturePatchModifier>(ObjectsBeingCustomized[0].Get()))
		{
			IDetailCategoryBuilder& WeightChannelCategory = DetailBuilder.EditCategory("WeightChannels");

			WeightChannelCategory.AddProperty(GET_MEMBER_NAME_CHECKED(MeshPartition::UTexturePatchModifier, WeightChannels));

			WeightChannelCategory.AddCustomRow(FText::FromString("CopyHeightSettingsButtons"))
				.NameContent()
				[
					SNew(STextBlock)
						.Text(FText::FromString("Copy from Height Channel"))
						.Font(IDetailLayoutBuilder::GetDetailFont())
				]
				.ValueContent()
				[
					SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.f)
						[
							SNew(SButton)
								.Text(FText::FromString("Falloff"))
								.OnClicked_Lambda([Modifier]()
									{
										Modifier->MatchWeightFalloffsToHeight();
										return FReply::Handled();
									})
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.f)
						[
							SNew(SButton)
								.Text(FText::FromString("Alpha Blend"))
								.OnClicked_Lambda([Modifier]()
									{
										Modifier->MatchWeightAlphaBlendToHeight();
										return FReply::Handled();
									})
						]

					+ SHorizontalBox::Slot()
						.AutoWidth()
						.Padding(2.f)
						[
							SNew(SButton)
								.Text(FText::FromString("Curve"))
								.OnClicked_Lambda([Modifier]()
									{
										Modifier->MatchWeightCurveToHeight();
										return FReply::Handled();
									})
						]
				];
		}
	}
}


void FMegaMeshTexturePatchDetails::AddTessellationCheckbox(IDetailLayoutBuilder& DetailBuilder)
{
	// Get the objects currently being customized (usually only one)
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);

	if (ObjectsBeingCustomized.Num() == 1)
	{
		if (MeshPartition::UTexturePatchModifier* Modifier = Cast<MeshPartition::UTexturePatchModifier>(ObjectsBeingCustomized[0].Get()))
		{
			IDetailCategoryBuilder& TessellationCategory = DetailBuilder.EditCategory("AdaptiveTessellation");

			TessellationModeProperty = DetailBuilder.GetProperty(
				GET_MEMBER_NAME_CHECKED(MeshPartition::UTexturePatchModifier, TessellationMode));

			DetailBuilder.HideProperty(TessellationModeProperty);

			std::underlying_type<MeshPartition::ETexturePatchTessellationMode>::type TessellationMode = 0;
			TessellationModeProperty->GetValue(TessellationMode);
			bTessellationFast = static_cast<MeshPartition::ETexturePatchTessellationMode>(TessellationMode) == MeshPartition::ETexturePatchTessellationMode::AdaptiveFast;

			if (TessellationModeProperty && TessellationModeProperty->IsValidHandle())
			{
				TessellationCategory.AddCustomRow(FText::FromString("AdaptiveTessellationCheckbox"))
					.NameContent()
					[
						SNew(STextBlock)
							.Text(FText::FromString("Enable"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						SAssignNew(CheckBoxTessellation, SCheckBox)
							.IsChecked_Lambda([this]()
								{
									std::underlying_type<MeshPartition::ETexturePatchTessellationMode>::type TessellationMode = 0;
									TessellationModeProperty->GetValue(TessellationMode);
									return static_cast<MeshPartition::ETexturePatchTessellationMode>(TessellationMode) != MeshPartition::ETexturePatchTessellationMode::NonAdaptive ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
								{
									MeshPartition::ETexturePatchTessellationMode Mode = MeshPartition::ETexturePatchTessellationMode::NonAdaptive;
									if (State == ECheckBoxState::Checked)
									{
										Mode = (CheckBoxTessellationFast.IsValid() && CheckBoxTessellationFast->IsChecked()) ?
											MeshPartition::ETexturePatchTessellationMode::AdaptiveFast : MeshPartition::ETexturePatchTessellationMode::Adaptive;
									}

									TessellationModeProperty->SetValue(static_cast<std::underlying_type<MeshPartition::ETexturePatchTessellationMode>::type>(Mode));

									if (CheckBoxTessellationFast.IsValid())
									{
										CheckBoxTessellationFast->SetEnabled(Mode != MeshPartition::ETexturePatchTessellationMode::NonAdaptive);
									}
								})
					];

				TessellationCategory.AddCustomRow(FText::FromString("AdaptiveTessellationFastCheckbox"))
					.NameContent()
					[
						SNew(STextBlock)
							.Text(FText::FromString("Fast Mode"))
							.Font(IDetailLayoutBuilder::GetDetailFont())
					]
					.ValueContent()
					[
						SAssignNew(CheckBoxTessellationFast, SCheckBox)
							.IsChecked_Lambda([this]()
								{
									return bTessellationFast ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
								})
							.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
								{
									bTessellationFast = (State == ECheckBoxState::Checked);

									MeshPartition::ETexturePatchTessellationMode Mode = MeshPartition::ETexturePatchTessellationMode::NonAdaptive;
									if (CheckBoxTessellation.IsValid() && CheckBoxTessellation->IsChecked())
									{
										Mode = bTessellationFast ?
											MeshPartition::ETexturePatchTessellationMode::AdaptiveFast : MeshPartition::ETexturePatchTessellationMode::Adaptive;
									}
									
									TessellationModeProperty->SetValue(static_cast<std::underlying_type<MeshPartition::ETexturePatchTessellationMode>::type>(Mode));
								})
					];
			}
		}
	}
}


TSharedRef<IDetailCustomization> FSplineSoftObjectPointerDetails::MakeInstance()
{
	return MakeShareable(new FSplineSoftObjectPointerDetails());
}

void FSplineSoftObjectPointerDetails::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);

	if (MeshPartition::USplineModifier* const SplineModifierObject = Cast<MeshPartition::USplineModifier>(ObjectsBeingCustomized[0]))
	{
		SplineModifier = SplineModifierObject;

		if (const AActor* const Owner = SplineModifierObject->GetOwner())
		{
			TArray<USplineComponent*> ActorSplines;
			Owner->GetComponents(ActorSplines, /*bIncludeFromChildActors = */ false);

			SplineNames.Reset();
			Splines.Reset();
			SelectedName = nullptr;
			for (USplineComponent* SplineComponent : ActorSplines)
			{
				Splines.Add(SplineComponent);
				SplineNames.Add(MakeShared<FString>(SplineComponent->GetName()));

				if (const USplineComponent* const ModifierSplineComponent = SplineModifier->GetSplineComponent())
				{
					if (SplineComponent->GetName() == ModifierSplineComponent->GetName())
					{
						SelectedName = SplineNames.Last();
					}
				}
			}
		}
		else
		{
			const bool bIsTemplate = (SplineModifierObject->IsTemplate() || SplineModifierObject->HasAnyFlags(RF_ArchetypeObject));

			if (bIsTemplate)
			{
				if (const UBlueprintGeneratedClass* const BPGC = SplineModifierObject->GetTypedOuter<UBlueprintGeneratedClass>())
				{
					SplineNames.Reset();
					Splines.Reset();
					SelectedName = nullptr;

					for (const UBlueprintGeneratedClass* BPGCAncestor = BPGC; BPGCAncestor; BPGCAncestor = Cast<UBlueprintGeneratedClass>(BPGCAncestor->GetSuperClass()))
					{
						const TArray<USCS_Node*>& BlueprintNodes = BPGCAncestor->SimpleConstructionScript->GetAllNodes();

						for (const USCS_Node* const BPNode : BlueprintNodes)
						{
							if (BPNode->ComponentClass && BPNode->ComponentClass->IsChildOf(USplineComponent::StaticClass()))
							{
								SplineNames.Add(MakeShared<FString>(BPNode->GetVariableName().ToString()));
								Splines.Add(nullptr);

								if (SplineModifier->TemplateSplineComponentName == BPNode->GetVariableName().ToString())
								{
									SelectedName = SplineNames.Last();
								}
							}
						}
					}
				}
			}
		}

		AddComboBox(DetailBuilder);

		// Hide the existing UI
		const TSharedRef<IPropertyHandle> SoftSplineProp = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(MeshPartition::USplineModifier, SplinePtr));
		DetailBuilder.HideProperty(SoftSplineProp);
	}
}

void FSplineSoftObjectPointerDetails::AddComboBox(IDetailLayoutBuilder& DetailBuilder)
{
	if (!ensure(SplineModifier.IsValid()))
	{
		return;
	}

	TArray<TWeakObjectPtr<UObject>> ObjectsBeingCustomized;
	DetailBuilder.GetObjectsBeingCustomized(ObjectsBeingCustomized);
	check(ObjectsBeingCustomized.Num() > 0);

	TSharedRef<IPropertyHandle> SplineProperty = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(MeshPartition::USplineModifier, SplinePtr));
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory("Spline");
	CategoryBuilder.AddCustomRow(LOCTEXT("SplineComponentRowLabel", "Spline Components"))
		.NameContent()
		[
			SplineProperty->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SNew(SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&SplineNames)
				.InitiallySelectedItem(SelectedName)
				[
					SNew(STextBlock)
						.Text_Lambda([this]()
							{
								if (SelectedName.IsValid())
								{
									return FText::FromString(*SelectedName);
								}
								else
								{
									return LOCTEXT("NoSelectedSplineComponentText", "<None>");
								}
							})
				]
				.OnSelectionChanged_Lambda([this](TSharedPtr<FString> NewSelectedName, ESelectInfo::Type SelectInfo)
					{
						if (!NewSelectedName.IsValid())
						{
							return;
						}

						bool bFound = false;
						for (const TWeakObjectPtr<USplineComponent> SplineComponent : Splines)
						{
							if (const TStrongObjectPtr<USplineComponent> PinnedSplineComponent = SplineComponent.Pin())
							{
								if (PinnedSplineComponent->GetName() == *NewSelectedName && SplineModifier.IsValid())
								{
									SplineModifier->Modify();
									SplineModifier->SetSplineComponent(PinnedSplineComponent.Get());
									bFound = true;
								}
							}
						}
						SelectedName = NewSelectedName;

						if (!bFound)
						{
							SplineModifier->Modify();
							SplineModifier->TemplateSplineComponentName = *SelectedName;
						}
					})
				.OnGenerateWidget_Lambda([this](TSharedPtr<FString> InItem)
					{
						return SNew(STextBlock)
							.Text(FText::FromString(InItem.IsValid() ? *InItem : FString()));
					})
		];

}


} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE