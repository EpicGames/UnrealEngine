// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionTedsFactory.h"

#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionPreviewSection.h"

#include "DataStorage/Features.h"

#include "Editor.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementIndexHasher.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"

#include "ActorDesc/TedsActorDescColumns.h"
#include "Columns/LayerOutlinerColumns.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Widgets/SceneOutlinerWidget.h"
#include "MeshPartitionCompiledSection.h"
#include "MeshPartitionInteractiveSection.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

#include "UI/MeshPartitionLayerVisibilityWidget.h"
#include "UI/MeshPartitionLayerBuildWidget.h"
#include "UI/MeshPartitionHierarchyWidget.h"
#include "UI/MeshPartitionLayerNameWidget.h"
#include "UI/MeshPartitionBuildCostWidget.h"
#include "UI/ParentActorWidget.h"

#include "ToolMenuSection.h"
#include "ToolMenu.h"

#define LOCTEXT_NAMESPACE "UMegaMeshTedsFactory"

namespace UE::MeshPartition
{
namespace UMegaMeshEditorUISubsystem::Local
{
	static const FName SceneMegaMeshesTableName("MegaMesh_SceneMegaMeshes");
	static const FName SceneMegaMeshDefinitionTableName("MegaMesh_SceneMegaMeshDefinitions");
	static const FName LayerVisibilityTableName("MegaMesh_LayerVisibilityTable");
	static const FName SceneMegaMeshesModifiersTableName("MegaMesh_SceneMegaMeshesModifiers");
	static const FName MegaMeshMappingDomain("MegaMesh");
	static const FName MegaMeshHierarchy("MegaMeshHierarchy");

	FString GenerateLayerMapKey(FName LayerID, const AMeshPartition& MegaMeshOwner)
	{
		return FString(LayerID.ToString() + FString::FromInt(MegaMeshOwner.GetUniqueID()));
	}

	FString GenerateLayerMapKey(int32 LayerIndex, const AMeshPartition& MegaMeshOwner)
	{
		return "Layer_" + FString::FromInt(LayerIndex) + "_M" + FString::FromInt(MegaMeshOwner.GetUniqueID());
	}
}

void UMegaMeshTedsFactory::PreRegister(Editor::DataStorage::ICoreProvider& DataStorage)
{

	if (UMeshPartitionEditorSubsystem* EditorSubsystem = UMeshPartitionEditorSubsystem::Get())
	{
		EditorSubsystem->OnMegaMeshChanged().AddUObject(this, &UMegaMeshTedsFactory::OnMegaMeshChanged);
	}

	using namespace Editor::DataStorage;
	MegaMeshTable = InvalidTableHandle;
	MegaMeshDefinitionTable = InvalidTableHandle;
	MegaMeshDefinitionLayerTable = InvalidTableHandle;
	MegaMeshModifierTable = InvalidTableHandle;

	DefinitionLayerQueryRO = InvalidQueryHandle;
	DefinitionLayerQueryRW = InvalidQueryHandle;
	DefinitionQuery = InvalidQueryHandle;
	MegaMeshQuery = InvalidQueryHandle;
	MegaMeshQueryRW = InvalidQueryHandle;
	UnresolvedRetrievalQuery = InvalidQueryHandle;
	ActiveLayerQuery = InvalidQueryHandle;
	BoundsFilterObjectQuery = InvalidQueryHandle;

#if WITH_EDITOR
	FEditorDelegates::MapChange.AddUObject(this, &UMegaMeshTedsFactory::CleanMegaMeshRows);
#endif
}

void UMegaMeshTedsFactory::PostRegister(Editor::DataStorage::ICoreProvider& DataStorage)
{
#if WITH_EDITOR
	if (UTedsOutlinerWidgetFactory* Factory = DataStorage.FindFactory<UTedsOutlinerWidgetFactory>())
	{
		Factory->RegisterExternalFilterProvider(
    	FName("MeshPartition"),
    	[](TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters, Editor::DataStorage::ICoreProvider* DataStorage)
    	{
    		using namespace Editor::DataStorage;
    		using namespace Editor::DataStorage::Queries;
    		using namespace UE::Editor::Outliner;

    		TSharedRef<FFilterCategory> MeshPartitionCategory = MakeShared<FFilterCategory>(      
				LOCTEXT("MeshPartitionFilterCategory", "Mesh Partition"),
				LOCTEXT("MeshPartitionFilterCategoryTooltip", "Mesh Partition Filters")
			);

    		Filters.Add(MakeShared<FTedsOutlinerFilter>(
    			FName("MeshPartitionFilter"),
    			LOCTEXT("MeshPartitionFilterName", "Mesh Partition"),
    			LOCTEXT("MeshPartitionFilterTooltip", "Show only Mesh Partition objects"),
    			FName(),
    			MeshPartitionCategory,
    			BuildConstQueryFunction<bool>([](TConstQueryContext<SingleRowInfo, CurrentTableInfo> Context)
    			{
    				return Context.CurrentTableHasColumns<FIsMegaMeshObjectTag>()
    					|| Context.CurrentTableHasColumns<FIsMegaMeshModifierTag>()
    					|| Context.CurrentTableHasColumns<FIsMegaMeshDefinitionLayerTag>();
    			})
    		));

    		Filters.Add(MakeShared<FTedsOutlinerFilter>(
    			FName("MeshPartitionBaseModifierFilter"),
    			LOCTEXT("MeshPartitionBaseModifierFilterName", "Mesh Partition Base Modifiers"),
    			LOCTEXT("MeshPartitionBaseModifierFilterTooltip", "Show only non-base Mesh Partition modifiers"),
    			FName(),
    			MeshPartitionCategory,
    			BuildConstQueryFunction<bool>([](TConstQueryContext<SingleRowInfo, CurrentTableInfo> Context)
    			{
    				if (Context.CurrentTableHasColumns<FIsMegaMeshModifierTag>())
    				{
    					return !Context.CurrentTableHasColumns<FMegaMeshNotHiddenInOutlinerTag>();
    				}
    				return true;
    			})
    		));

    		Filters.Add(MakeShared<FTedsOutlinerFilter>(
    			FName("MeshPartitionBuiltSectionFilter"),
    			LOCTEXT("MeshPartitionBuiltSectionFilterName", "Mesh Partition Built Sections"),
    			LOCTEXT("MeshPartitionBuiltSectionFilterTooltip", "Hide preview and compiled Mesh Partition sections"),
    			FName(),
    			MeshPartitionCategory,
    			BuildConstQueryFunction<bool>([DataStorage](TConstQueryContext<SingleRowInfo, CurrentTableInfo> Context)
    			{
    				if (DataStorage)
    				{
    					if (const FTypedElementUObjectColumn* ObjectColumn =
    						DataStorage->GetColumn<FTypedElementUObjectColumn>(Context.GetCurrentRow()))
    					{
    						if (const AActor* Actor = Cast<AActor>(ObjectColumn->Object.Get()))
    						{
								return !Actor->IsA<APreviewSection>()
									&& !Actor->IsA<ACompiledSection>()
									&& !Actor->IsA<AInteractiveSection>();
							}
    					}
    					if (const FWorldPartitionHandleColumn* HandleColumn = DataStorage->GetColumn<FWorldPartitionHandleColumn>(Context.GetCurrentRow());
    						HandleColumn && HandleColumn->Handle.IsValid())
    					{
							if (const FWorldPartitionActorDescInstance* const ActorDescInstance = *HandleColumn->Handle)
							{
								UClass* NativeClass = ActorDescInstance->GetActorNativeClass();
								if (ensure(NativeClass))
								{
									return !NativeClass->IsChildOf<ACompiledSection>();
								}
							}
						}
    				}
    				return true;
    			})
    		));
    	}
	);
	}
#endif
}

void UMegaMeshTedsFactory::RegisterTables(Editor::DataStorage::ICoreProvider& DataStorage,
	Editor::DataStorage::ICompatibilityProvider& CompatibilityDataStorage)
{
	using namespace Editor::DataStorage;
	
	MegaMeshTable = DataStorage.RegisterTable<
		FTypedElementUObjectColumn, FIsMegaMeshObjectTag, FMegaMeshRowParentColumn,
		FMegaMeshDrawBoundsColumn, FMegaMeshBuildToStatus, FMegaMeshTimingStatistics>(
			UMegaMeshEditorUISubsystem::Local::SceneMegaMeshesTableName,
			FTableRegistrationOptions{ .SourceTable = DataStorage.FindTable(FName(TEXT("Editor_StandardActorTable"))) });

	MegaMeshDefinitionTable = DataStorage.RegisterTable<FTypedElementUObjectColumn, FIsMegaMeshDefinitionObjectTag>(
		UMegaMeshEditorUISubsystem::Local::SceneMegaMeshDefinitionTableName,
		FTableRegistrationOptions{ .SourceTable = DataStorage.FindTable(FName(TEXT("Editor_StandardUObjectTable"))) });

	MegaMeshModifierTable = DataStorage.RegisterTable<
		FTypedElementUObjectColumn, FTypedElementLabelColumn, FTypedElementLabelHashColumn,
		FIsMegaMeshModifierTag, FMegaMeshLayerNameColumn, FMegaMeshRowParentColumn,
		FMegaMeshReferenceColumn, FMegaMeshModifierTiming, FMegaMeshPriorityColumn,
		FMegaMeshBuildUpToThisLayerColumn, FMegaMeshModifierSortColumn, FParentActorRefColumn>(
			UMegaMeshEditorUISubsystem::Local::SceneMegaMeshesModifiersTableName,
			FTableRegistrationOptions{ .SourceTable = DataStorage.FindTable(FName(TEXT("Editor_StandardUObjectTable"))) });

	MegaMeshDefinitionLayerTable = DataStorage.RegisterTable<
		FTypedElementRowReferenceColumn, FIsMegaMeshDefinitionLayerTag, FTypedElementLabelColumn,
		FTypedElementLabelHashColumn, FMegaMeshRowParentColumn, FMegaMeshPriorityColumn,
		FMegaMeshLayerNameColumn, FMegaMeshDrawBoundsColumn, FMegaMeshBuildUpToThisLayerColumn,
		FMegaMeshModifierTiming
		>(UMegaMeshEditorUISubsystem::Local::LayerVisibilityTableName);

	CompatibilityDataStorage.RegisterTypeTableAssociation(AMeshPartition::StaticClass(), MegaMeshTable);
	CompatibilityDataStorage.RegisterTypeTableAssociation(UMeshPartitionDefinition::StaticClass(), MegaMeshDefinitionTable);
	CompatibilityDataStorage.RegisterTypeTableAssociation(MeshPartition::UModifierComponent::StaticClass(), MegaMeshModifierTable);
}

void UMegaMeshTedsFactory::RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace Editor::DataStorage;

	FHierarchyRegistrationParams Params;
	Params.Name = UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy;
	
	MegaMeshHierarchy = DataStorage.RegisterHierarchy(Params);
}

namespace LocalQueryLogic
{
	using namespace Editor::DataStorage;
	using namespace Editor::DataStorage::Queries;

	void HandlePotentialLayerAdd(IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshReferenceColumn& MeshReference, FMegaMeshLayerNameColumn& LayerNameColumn)
	{
		if (MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(Object.Object); Modifier != nullptr)
		{
			TWeakObjectPtr<AMeshPartition> MegaMeshInstance = MeshReference.Mesh;

			if(MegaMeshInstance.IsValid())
			{
				FString LayerName = Modifier->GetType().ToString();
				ensure(LayerNameColumn.Name == Modifier->GetType());

				FString LayerKey = UMegaMeshEditorUISubsystem::Local::GenerateLayerMapKey(LayerNameColumn.Name, *MegaMeshInstance);

				RowHandle LayerRow = Context.LookupMappedRow(UMegaMeshEditorUISubsystem::Local::MegaMeshMappingDomain, FMapKeyView(LayerKey));
				if (Context.IsRowAvailable(LayerRow))
				{
					Context.AddColumn<FMegaMeshRowParentColumn>(Row, FMegaMeshRowParentColumn{ .Parent = LayerRow });
					Context.SetParentRow(Row, LayerRow);

					Context.AddColumns<FTypedElementSyncBackToWorldTag>(Row);

					
					if (Context.HasColumn<FIsMegaMeshActiveInOutlinerTag>(LayerRow))
					{
						Context.AddColumns<FIsMegaMeshActiveInOutlinerTag>(Row);
					}

					FQueryResult Result2 = Context.RunSubquery(1, LayerRow, CreateSubqueryCallbackBinding([Row](ISubqueryContext& SubContext, RowHandle SubqueryRow,
						FTypedElementRowReferenceColumn& RefRow, FMegaMeshRowParentColumn& MegaMeshChildren)
						{
							MegaMeshChildren.ChildrenSet.Add(Row, 0);
							SubContext.AddColumns<FTypedElementSyncBackToWorldTag>(SubqueryRow);
						}));

				}
				else
				{
					Context.AddColumn(Row, FUnresolvedMegaMeshLayer
						{
							.LayerName = MoveTemp(LayerName),
							.Layer = LayerNameColumn.Name
						});
				}
			}
		}
	}

	void HandlePotentialLayerRemoval(IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshReferenceColumn& MeshReference, FMegaMeshLayerNameColumn& LayerNameColumn, FMegaMeshRowParentColumn& Parent)
	{
		if (!Context.IsRowAvailable(Parent.Parent))
		{
			return;
		}

		if (MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(Object.Object); Modifier != nullptr)
		{
			RowHandle LayerRow = Parent.Parent;
			FMegaMeshRowParentColumn LayerHierarchyColumn;

			FQueryResult Result2 = Context.RunSubquery(1, LayerRow, CreateSubqueryCallbackBinding([&LayerHierarchyColumn](ISubqueryContext& Context, RowHandle SubqueryRow,
				FTypedElementRowReferenceColumn& RefRow, FMegaMeshRowParentColumn& MegaMeshChildren, FMegaMeshLayerNameColumn& LayerNameColumn)
				{
					LayerHierarchyColumn = MegaMeshChildren;
				}));

			int32 ModifierPriority;
			LayerHierarchyColumn.ChildrenSet.RemoveAndCopyValue(Row, ModifierPriority);
			Parent.Parent = InvalidRowHandle;
			Context.SetParentRow(Row, InvalidRowHandle);
			Context.AddColumns<FTypedElementSyncBackToWorldTag>(Row);

			if (LayerHierarchyColumn.ChildrenSet.IsEmpty())
			{
				RowHandle MeshRow = LayerHierarchyColumn.Parent;
				if (Context.IsRowAvailable(MeshRow))
				{
					FMegaMeshRowParentColumn MeshHierarchyColumn;

					FQueryResult Result = Context.RunSubquery(0, MeshRow, CreateSubqueryCallbackBinding([&MeshHierarchyColumn](ISubqueryContext& Context, RowHandle SubqueryRow,
						FTypedElementUObjectColumn& MegaMeshObject, FMegaMeshRowParentColumn& MegaMeshChildren)
						{
							MeshHierarchyColumn = MegaMeshChildren;
						}));

					int32 LayerPriority;
					MeshHierarchyColumn.ChildrenSet.RemoveAndCopyValue(LayerRow, LayerPriority);

					Context.AddColumn<FMegaMeshRowParentColumn>(MeshRow, MoveTemp(MeshHierarchyColumn));
					Context.AddColumns<FTypedElementSyncBackToWorldTag>(MeshRow);

				}
				Context.RemoveRow(LayerRow);
			}
			else
			{
				Context.AddColumn<FMegaMeshRowParentColumn>(LayerRow, MoveTemp(LayerHierarchyColumn));
				Context.AddColumns<FTypedElementSyncBackToWorldTag>(LayerRow);

			}

		}
	}


	void HandlePotentialLayerAddAndRemoval(IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshReferenceColumn& MeshReference, FMegaMeshLayerNameColumn& LayerNameColumn, FMegaMeshRowParentColumn& Parent)
	{
		LocalQueryLogic::HandlePotentialLayerRemoval(Context, Row, Object, MeshReference, LayerNameColumn, Parent);
		LocalQueryLogic::HandlePotentialLayerAdd(Context, Row, Object, MeshReference, LayerNameColumn);

		Context.RemoveColumns<FMegaMeshLayerUpdatedTag>(Row);
	}

	void HandleMegaMeshActivationInOutliner(IQueryContext&  Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshRowParentColumn&  Parent)
	{		
		Context.AddColumns<FIsMegaMeshActiveInOutlinerTag>(Parent.Children);
	}


	void HandleMegaMeshDeactivationInOutliner(IQueryContext&  Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshRowParentColumn&  Parent)
	{
		Context.RemoveColumns<FIsMegaMeshActiveInOutlinerTag>(Parent.Children);
	}

}

void UMegaMeshTedsFactory::RegisterQueries(Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace Editor::DataStorage;
	using namespace Editor::DataStorage::Queries;


	DefinitionLayerQueryRO = DataStorage.RegisterQuery(Select()
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.ReadOnly<FTypedElementRowReferenceColumn, FMegaMeshRowParentColumn, FMegaMeshLayerNameColumn, FMegaMeshPriorityColumn>()
		.Where()
		.All<FIsMegaMeshDefinitionLayerTag>()
		.Compile()
	);

	DefinitionLayerQueryRW = DataStorage.RegisterQuery(Select()
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.ReadWrite<FTypedElementRowReferenceColumn, FMegaMeshRowParentColumn, FMegaMeshLayerNameColumn, FMegaMeshPriorityColumn>()
		.Where()
		.All<FIsMegaMeshDefinitionLayerTag>()
		.Compile()
	);

	DefinitionQuery = DataStorage.RegisterQuery(Select()
		.ReadOnly<FTypedElementUObjectColumn>()
		.Where()
		.All<FIsMegaMeshDefinitionObjectTag>()
		.Compile()
	);

	MegaMeshQuery = DataStorage.RegisterQuery(Select()
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.ReadOnly<FTypedElementUObjectColumn, FMegaMeshRowParentColumn>()
		.Where()
		.All<FIsMegaMeshObjectTag>()
		.Compile()
	);

	MegaMeshQueryRW = DataStorage.RegisterQuery(Select()
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.ReadWrite<FTypedElementUObjectColumn, FMegaMeshRowParentColumn>()
		.Where()
		.All<FIsMegaMeshObjectTag>()
		.Compile()
	);

	UnresolvedRetrievalQuery = DataStorage.RegisterQuery(
		Select()
		.Where()
		.All<FIsMegaMeshModifierTag, FUnresolvedMegaMeshLayer>()
		.Compile());

	MegaMeshModifierQueryRW = DataStorage.RegisterQuery(Select()
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.ReadWrite<FTypedElementUObjectColumn, FMegaMeshReferenceColumn, FMegaMeshRowParentColumn, FMegaMeshModifierTiming>()
		.Where()
		.All<FIsMegaMeshModifierTag>()
		.Compile());

	ActiveLayerQuery = DataStorage.RegisterQuery(
		Select()
		.Where()
		.All<FIsMegaMeshDefinitionLayerTag, FMegaMeshActiveLayerTag>()
		.Compile());

	BoundsFilterObjectQuery = DataStorage.RegisterQuery(
		Select()
		.Where()
		.All<FMegaMeshBoundsFilterSourceTag, FTypedElementUObjectColumn>()
		.Compile());


	DataStorage.RegisterQuery(
		Select(
			TEXT("Create needed layers when modifier is created."),
			FObserver::OnAdd<FMegaMeshLayerUpdatedTag>(),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshReferenceColumn& MeshReference, FMegaMeshLayerNameColumn& LayerNameColumn, FMegaMeshRowParentColumn& Parent)
			{
				LocalQueryLogic::HandlePotentialLayerAddAndRemoval(Context, Row, Object, MeshReference, LayerNameColumn, Parent);
			})
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.Where()
		.All<FIsMegaMeshModifierTag>()
		.DependsOn()
		.SubQuery(MegaMeshQueryRW)
		.SubQuery(DefinitionLayerQueryRW)
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Ensure the build to settings are reset when a megamesh is added."),
			FObserver::OnAdd<FIsMegaMeshObjectTag>(),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshBuildToStatus& BuildToStatus)
			{
				// This is the same value the Builder uses... TODO: Makes sure these are kept in sync in case the builder changes
				constexpr uint32 UngroupedLayerIndex = TNumericLimits<uint32>::Max();

				BuildToStatus.LayerBuildToIndex = UngroupedLayerIndex;
				BuildToStatus.ModiferToBuildTo = InvalidRowHandle;
			})
		.Where()
		.All<FIsMegaMeshObjectTag>()
		.DependsOn()
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Activate Sublayers of a selected megamesh in the outliner"),
			FObserver::OnAdd<FIsMegaMeshActiveInOutlinerTag>(),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, const FMegaMeshRowParentColumn& Parent)
			{
				TArray<RowHandle> RowsToProcess;
				Parent.ChildrenSet.GetKeys(RowsToProcess);
				for (RowHandle ProcessRow : RowsToProcess)
				{
					Context.AddColumns<FIsMegaMeshActiveInOutlinerTag>(ProcessRow);
				}
				for (RowHandle Child : RowsToProcess)
				{
					FQueryResult Result = Context.RunSubquery(0, Child, CreateSubqueryCallbackBinding([](ISubqueryContext& Context, RowHandle SubqueryRow, const FTypedElementRowReferenceColumn& RowRef, const FMegaMeshRowParentColumn& Children)
						{
							TArray<RowHandle> ChildrenRowsToProcess;
							Children.ChildrenSet.GetKeys(ChildrenRowsToProcess);
							for (RowHandle ProcessRow : ChildrenRowsToProcess)
							{
								Context.AddColumns<FIsMegaMeshActiveInOutlinerTag>(ProcessRow);
							}
						}));
				}
			})
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.Where()
		.All<FIsMegaMeshObjectTag>()
		.DependsOn()
		.SubQuery(DefinitionLayerQueryRW)
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Activate Sublayers of a selected megamesh in the outliner"),
			FObserver::OnRemove<FIsMegaMeshActiveInOutlinerTag>(),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, const FMegaMeshRowParentColumn& Parent)
			{
				TArray<RowHandle> RowsToProcess;
				Parent.ChildrenSet.GetKeys(RowsToProcess);
				for (RowHandle ProcessRow : RowsToProcess)
				{
					Context.RemoveColumns<FIsMegaMeshActiveInOutlinerTag>(ProcessRow);
				}
				for (RowHandle Child : RowsToProcess)
				{
					FQueryResult Result = Context.RunSubquery(0, Child, CreateSubqueryCallbackBinding([](ISubqueryContext& Context, RowHandle SubqueryRow, const FTypedElementRowReferenceColumn& RowRef, const FMegaMeshRowParentColumn& Children)
						{
							TArray<RowHandle> ChildrenRowsToProcess;
							Children.ChildrenSet.GetKeys(ChildrenRowsToProcess);
							for (RowHandle ProcessRow : ChildrenRowsToProcess)
							{
								Context.RemoveColumns<FIsMegaMeshActiveInOutlinerTag>(ProcessRow);
							}
						}));
				}
			})
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.Where()
		.All<FIsMegaMeshObjectTag>()
		.DependsOn()
		.SubQuery(DefinitionLayerQueryRW)
		.Compile()
	);

	DataStorage.OnUpdateCompleted().AddUObject(this, &UMegaMeshTedsFactory::ResolveLayerRows);
	DataStorage.OnUpdateCompleted().AddUObject(this, &UMegaMeshTedsFactory::ResolveBoundsFiltering);
	DataStorage.OnUpdateCompleted().AddUObject(this, &UMegaMeshTedsFactory::GenerateModifierSortKeys);


	DataStorage.RegisterQuery(
		Select(
			TEXT("Update the Build To info in the Builder"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, const FMegaMeshBuildToStatus& BuildToStatus)
			{
				if (AMeshPartition* MegaMeshInstance = Cast<AMeshPartition>(Object.Object); MegaMeshInstance != nullptr)
				{
					UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMeshInstance->GetMeshPartitionComponent());
					if (EditorComponent == nullptr)
					{
						return;
					}

					if (Context.IsRowAvailable(BuildToStatus.ModiferToBuildTo))
					{
						FQueryResult Result = Context.RunSubquery(0, BuildToStatus.ModiferToBuildTo, CreateSubqueryCallbackBinding([EditorComponent](ISubqueryContext& Context, RowHandle SubqueryRow,
							const FTypedElementUObjectColumn& Object)
							{
								if (MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(Object.Object); Modifier != nullptr)
								{
									EditorComponent->SetBuildModifierFilterFunction(MeshPartition::FilterHelpers::FilterModifiersByLastModifierToBuild(*Modifier, true));
								}
							}));
					}
					else
					{
						EditorComponent->SetBuildModifierFilterFunction(MeshPartition::FilterHelpers::FilterModifiersByIndexToBuild(BuildToStatus.LayerBuildToIndex, true));
					}
					EditorComponent->ForceRebuildAllSections(MeshPartition::EChangeType::TransientChange);
				}
			})
		.Where()
		.All<FIsMegaMeshObjectTag, FTypedElementSyncBackToWorldTag>()
		.DependsOn()
		.SubQuery(MegaMeshModifierQueryRW)
		.Compile()
	);

	DataStorage.RegisterQuery(
				Select(
					TEXT("Update Mega Mesh Modifier Entries if Required, Phase 1"),
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
					[this](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshReferenceColumn& MeshReference, FTypedElementLabelColumn& Label,
						FMegaMeshLayerNameColumn& LayerName, FMegaMeshRowParentColumn& Parent, FMegaMeshPriorityColumn& ModifierPriority, FParentActorRefColumn& ParentActorRef)
					{
						if (MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(Object.Object); Modifier != nullptr)
						{
							Label.Label = Modifier->GetName();

							bool bNeedsLayerUpdates = false;

							if (Modifier->GetType() != LayerName.Name)
							{
								bNeedsLayerUpdates = true;
								LayerName.Name = Modifier->GetType();
							}

							// If modifier is assigned a controlling Mega Mesh, add tag indicating that state
							if (AMeshPartition* MegaMeshInstance = Modifier->GetAffectedMeshPartition(); MegaMeshInstance != MeshReference.Mesh)
							{
								bNeedsLayerUpdates = true;
								MeshReference.Mesh = MegaMeshInstance;
							}

							if (!Modifier->IsBase())
							{
								Context.AddColumns<FMegaMeshNotHiddenInOutlinerTag>(Row);
							}


							if (bNeedsLayerUpdates && !Modifier->IsBase())
							{
								Context.AddColumns<FMegaMeshLayerUpdatedTag>(Row);
							}

							ParentActorRef.ParentActor = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(Modifier->GetOwner()));

							ModifierPriority.Priority = Modifier->GetPriority();

							bShouldUpdateModifierSortKeys.store(true);
							bShouldResolveBoundsFiltering.store(true);
						}
					}
		)
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.Where()
		.All<FTypedElementSyncFromWorldTag, FIsMegaMeshModifierTag>()
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Update Mega Mesh Layer Timings"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FMegaMeshRowParentColumn& Parent, FMegaMeshModifierTiming& Timing)
			{
				Timing.InstanceCount = 0;
				Timing.TotalTime = 0;
				Timing.MinTime = 0;
				Timing.MaxTime = 0;

				for (TPair<RowHandle, int> ModifierRow : Parent.ChildrenSet)
				{

					FQueryResult Result = Context.RunSubquery(0, ModifierRow.Key, CreateSubqueryCallbackBinding([&Timing](ISubqueryContext& Context, RowHandle SubqueryRow,
						const FTypedElementUObjectColumn& MegaMeshModifierObject, const FMegaMeshReferenceColumn& MeshRefColumn, const FMegaMeshRowParentColumn&, FMegaMeshModifierTiming& TimingColumn)
						{
							Timing.MaxTime = FMath::Max(Timing.MaxTime, TimingColumn.MaxTime);
						}));
				}

			})
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.Where()
		.All<FIsMegaMeshDefinitionLayerTag, FIsMegaMeshActiveInOutlinerTag>()
		.DependsOn()
		.SubQuery(MegaMeshModifierQueryRW)
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Update Mega Mesh Modifier Timings"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, FMegaMeshRowParentColumn& Parent, FMegaMeshTimingStatistics& Statistics)
			{
				if (AMeshPartition* MegaMesh = Cast<AMeshPartition>(Object.Object); MegaMesh != nullptr)
				{
					if (UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent()); EditorComponent != nullptr)
					{
						TMap<FSoftObjectPath, MeshPartition::FPerModifierBuildPerfStats> PerModifierTimings;

						auto MergePerfTiming = [&PerModifierTimings](const MeshPartition::FBuildPerfStats& Stats)
							{
								for (TPair< FSoftObjectPath, MeshPartition::FPerModifierBuildPerfStats> Stat : Stats.PerModifierTimings)
								{
									if (PerModifierTimings.Contains(Stat.Key))
									{
										PerModifierTimings[Stat.Key].InstanceCount += Stat.Value.InstanceCount;
										PerModifierTimings[Stat.Key].TotalExecutionTime += Stat.Value.TotalExecutionTime;
										PerModifierTimings[Stat.Key].MinInstanceExecutionTime = FMath::Min(Stat.Value.MinInstanceExecutionTime, PerModifierTimings[Stat.Key].MinInstanceExecutionTime);
										PerModifierTimings[Stat.Key].MaxInstanceExecutionTime = FMath::Max(Stat.Value.MaxInstanceExecutionTime, PerModifierTimings[Stat.Key].MaxInstanceExecutionTime);
									}
									else
									{
										PerModifierTimings.Add(Stat);
									}
								}
							};

						EditorComponent->ForAllPreviewSections([&MergePerfTiming](MeshPartition::APreviewSection* PreviewSection)
							{
								MergePerfTiming(PreviewSection->GetBuildPerfStats());
								return true;
							});

						if (PerModifierTimings.Num() > 0)
						{
							const double ModifierCount = PerModifierTimings.Num();
							double Mean = 0;
							for (TPair< FSoftObjectPath, MeshPartition::FPerModifierBuildPerfStats> ModifierStats : PerModifierTimings)
							{
								Mean += ModifierStats.Value.TotalExecutionTime;
							}
							Statistics.TotalTimeMean = Mean / ModifierCount;

							double StandardDeviation = 0;
							for (TPair< FSoftObjectPath, MeshPartition::FPerModifierBuildPerfStats> ModifierStats : PerModifierTimings)
							{
								StandardDeviation += FMath::Square(ModifierStats.Value.TotalExecutionTime - Statistics.TotalTimeMean);
							}
							Statistics.TotalTimeStandardDeviation = FMath::Sqrt(StandardDeviation / ModifierCount);
						}

						FQueryResult Result = Context.RunSubquery(0, CreateSubqueryCallbackBinding([&PerModifierTimings, MegaMesh](ISubqueryContext& Context, RowHandle SubqueryRow,
							const FTypedElementUObjectColumn& MegaMeshModifierObject, const FMegaMeshReferenceColumn& MeshRefColumn, const FMegaMeshRowParentColumn&, FMegaMeshModifierTiming& TimingColumn)
							{
								FSoftObjectPath KeyPath(MegaMeshModifierObject.Object.Get());

								if (PerModifierTimings.Contains(KeyPath))
								{
									TimingColumn.InstanceCount = PerModifierTimings[KeyPath].InstanceCount;
									TimingColumn.TotalTime = PerModifierTimings[KeyPath].TotalExecutionTime;
									TimingColumn.MinTime = PerModifierTimings[KeyPath].MinInstanceExecutionTime;
									TimingColumn.MaxTime = PerModifierTimings[KeyPath].MaxInstanceExecutionTime;
								}
								else
								{
									TimingColumn.InstanceCount = 0;
									TimingColumn.TotalTime = 0.0;
									TimingColumn.MinTime = 0.0;
									TimingColumn.MaxTime = 0.0;
								}
							}));
					}
				}
			}
		)
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.Where()
		.All<FIsMegaMeshObjectTag, FIsMegaMeshActiveInOutlinerTag>()
		.DependsOn()
		.SubQuery(MegaMeshModifierQueryRW)
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("MegaMeshSubsystem: Sync visibility to World"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FTypedElementRowReferenceColumn& RowRefColumn, FMegaMeshRowParentColumn& HierarchyColumn, FMegaMeshPriorityColumn& LayerIndex, FMegaMeshLayerNameColumn& LayerName, FMegaMeshDrawBoundsColumn& DrawBounds)
			{
				for (TPair<RowHandle, int> ChildModifierRow : HierarchyColumn.ChildrenSet)
				{
					FQueryResult DefinitionQueryResult = Context.RunSubquery(0, ChildModifierRow.Key, Queries::CreateSubqueryCallbackBinding([LayerName, DrawBounds](ISubqueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ObjectColumn)
						{
							if (MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(ObjectColumn.Object.Get()); Modifier != nullptr)
							{
								Modifier->SetDrawBounds(DrawBounds.bEnabled);
							}
						}));
				}

				Context.RemoveColumns<FMegaMeshUpdatedTag>(Row);
			}
		)
		.AccessesHierarchy(UMegaMeshEditorUISubsystem::Local::MegaMeshHierarchy)
		.Where()
		.All<FIsMegaMeshDefinitionLayerTag, FMegaMeshUpdatedTag>()
		.DependsOn()
		.SubQuery(MegaMeshModifierQueryRW)
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Recompute affected modifiers when our filter objects change."),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[this](IQueryContext& Context, RowHandle BaseRow)
			{
				bShouldResolveBoundsFiltering.store(true);
			}
		)
		.Where()
		.All<FTypedElementSyncFromWorldTag, FMegaMeshBoundsFilterSourceTag>()
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Recompute affected modifiers when our filter objects are added to."),
			FObserver::OnAdd<FMegaMeshBoundsFilterSourceTag>(),
			[this](IQueryContext& Context, RowHandle BaseRow)
			{
				bShouldResolveBoundsFiltering.store(true);
			}
		)
		.Where()
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Recompute affected modifiers when our filter objects are removed from."),
			FObserver::OnRemove<FMegaMeshBoundsFilterSourceTag>(),
			[this](IQueryContext& Context, RowHandle BaseRow)
			{
				bShouldResolveBoundsFiltering.store(true);
			}
		)
		.Where()
		.Compile()
	);


	Super::RegisterQueries(DataStorage);
}

void UMegaMeshTedsFactory::RegisterWidgetConstructors(Editor::DataStorage::ICoreProvider& DataStorage,
	Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace Editor::DataStorage::Queries;

	Editor::DataStorage::IUiProvider::FPurposeID DefaultOutlinerLabelID(
		Editor::DataStorage::IUiProvider::FPurposeInfo(
		"SceneOutliner", "RowLabel", NAME_None).GeneratePurposeID());

	Editor::DataStorage::IUiProvider::FPurposeID DefaultOutlinerCellID(
		Editor::DataStorage::IUiProvider::FPurposeInfo(
			"SceneOutliner", "Cell", NAME_None).GeneratePurposeID());

	Editor::DataStorage::IUiProvider::FPurposeID SceneOutlinerHeaderID(
		IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None).GeneratePurposeID());

	Editor::DataStorage::IUiProvider::FPurposeInfo OutlinerPurpose(
		TEXT("MegaMesh"), TEXT("Outliner"), NAME_None,
		Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("MegaMeshOutlinerPurposeDesc", "Purpose for generic Mesh Partition Outliner uses"),
		DefaultOutlinerCellID);

	Editor::DataStorage::IUiProvider::FPurposeInfo HeaderPurpose(
		TEXT("MegaMesh"), TEXT("OutlinerHeader"), NAME_None,
		Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("MegaMeshOutlinerHeaderPurposeDesc", "Purpose for generic Mesh Partition Outliner Header uses"),
		SceneOutlinerHeaderID);

	Editor::DataStorage::IUiProvider::FPurposeInfo LayerNamePurpose(
		TEXT("MegaMesh"), TEXT("Outliner"), TEXT("LayerName"),
		Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("MegaMeshLayerNamePurposeDesc", "Purpose for displaying layer names"),
		DefaultOutlinerLabelID);

	Editor::DataStorage::IUiProvider::FPurposeInfo BuildCostPurpose(
		TEXT("MegaMesh"), TEXT("Outliner"), TEXT("BuildCost"),
		Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("MegaMeshOutlinerModifierPurposeDesc", "Purpose for BuildCost widgets in the Mesh Partition Outliner"),
		OutlinerPurpose.GeneratePurposeID());

	DataStorageUi.RegisterWidgetPurpose(LayerNamePurpose);
	DataStorageUi.RegisterWidgetPurpose(OutlinerPurpose);
	DataStorageUi.RegisterWidgetPurpose(HeaderPurpose);
	DataStorageUi.RegisterWidgetPurpose(BuildCostPurpose);

	DataStorageUi.RegisterWidgetFactory<FNameWidgetHeaderConstructor>(DataStorageUi.FindPurpose(HeaderPurpose.GeneratePurposeID()), TColumn<FTypedElementLabelColumn>());
	DataStorageUi.RegisterWidgetFactory<FMeshPartitionTypeInfoHeaderConstructor>(DataStorageUi.FindPurpose(HeaderPurpose.GeneratePurposeID()), TColumn<FTypedElementClassTypeInfoColumn>());
	DataStorageUi.RegisterWidgetFactory<FMeshPartitionTypeInfoCellConstructor>(DataStorageUi.FindPurpose(OutlinerPurpose.GeneratePurposeID()), TColumn<FTypedElementClassTypeInfoColumn>());

	DataStorageUi.RegisterWidgetFactory<FMegaMeshNameWidgetConstructorDelegator>(DataStorageUi.FindPurpose(OutlinerPurpose.GeneratePurposeID()), TColumn<FTypedElementLabelColumn>());
	DataStorageUi.RegisterWidgetFactory<FMegaMeshLayerNameWidgetConstructor>(DataStorageUi.FindPurpose(LayerNamePurpose.GeneratePurposeID()), TColumn<FTypedElementLabelColumn>() && TColumn<FIsMegaMeshDefinitionLayerTag>());
	DataStorageUi.RegisterWidgetFactory<FMegaMeshModifierNameWidgetConstructor>(DataStorageUi.FindPurpose(LayerNamePurpose.GeneratePurposeID()), TColumn<FTypedElementLabelColumn>() && TColumn<FIsMegaMeshModifierTag>());

	DataStorageUi.RegisterWidgetFactory<FMegaMeshVisibilityWidgetHeaderConstructor>(
		DataStorageUi.FindPurpose(HeaderPurpose.GeneratePurposeID()), TColumn<FMegaMeshDrawBoundsColumn>());
	DataStorageUi.RegisterWidgetFactory<FMegaMeshVisibilityFlagWidget>(DataStorageUi.FindPurpose(OutlinerPurpose.GeneratePurposeID()), TColumn<FMegaMeshDrawBoundsColumn>());
	
	DataStorageUi.RegisterWidgetFactory<FMegaMeshLayerBuildWidgetHeaderConstructor>(
		DataStorageUi.FindPurpose(HeaderPurpose.GeneratePurposeID()), TColumn<FMegaMeshBuildUpToThisLayerColumn>());
	DataStorageUi.RegisterWidgetFactory<FMegaMeshLayerBuildWidget>(DataStorageUi.FindPurpose(OutlinerPurpose.GeneratePurposeID()), TColumn<FMegaMeshBuildUpToThisLayerColumn>());
	
	DataStorageUi.RegisterWidgetFactory<MeshPartition::FHierarchyWidget>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FMegaMeshRowParentColumn>());
	DataStorageUi.RegisterWidgetFactory<FMegaMeshBuildCostWidgetHeaderConstructor>(
		DataStorageUi.FindPurpose(HeaderPurpose.GeneratePurposeID()), TColumn<FMegaMeshModifierTiming>());

	DataStorageUi.RegisterWidgetFactory<FMegaMeshBuildCostWidgetConstructorDelegator>(DataStorageUi.FindPurpose(OutlinerPurpose.GeneratePurposeID()), TColumn<FMegaMeshModifierTiming>());
	DataStorageUi.RegisterWidgetFactory<FMegaMeshBuildCostWidgetConstructor>(DataStorageUi.FindPurpose(BuildCostPurpose.GeneratePurposeID()), TColumn<FMegaMeshModifierTiming>() && TColumn<FIsMegaMeshModifierTag>());
	DataStorageUi.RegisterWidgetFactory<FMegaMeshBuildCostAggregateWidgetConstructor>(DataStorageUi.FindPurpose(BuildCostPurpose.GeneratePurposeID()), TColumn<FMegaMeshModifierTiming>() && TColumn<FIsMegaMeshDefinitionLayerTag>());

	DataStorageUi.RegisterWidgetFactory<FParentActorWidgetHeaderConstructor>(DataStorageUi.FindPurpose(HeaderPurpose.GeneratePurposeID()), TColumn<FParentActorRefColumn>());
	DataStorageUi.RegisterWidgetFactory<FParentActorWidgetConstructor>(DataStorageUi.FindPurpose(OutlinerPurpose.GeneratePurposeID()), TColumn<FParentActorRefColumn>());

}

void UMegaMeshTedsFactory::RegisterWidgetPurposes(Editor::DataStorage::IUiProvider& DataStorageUi) const
{
}

void UMegaMeshTedsFactory::PreShutdown(Editor::DataStorage::ICoreProvider& DataStorage)
{
#if WITH_EDITOR
	if (UTedsOutlinerWidgetFactory* Factory = DataStorage.FindFactory<UTedsOutlinerWidgetFactory>())
	{
		Factory->UnregisterExternalFilterProvider(FName("MeshPartition"));
	}
#endif

	if (UMeshPartitionEditorSubsystem* EditorSubsystem = UMeshPartitionEditorSubsystem::Get())
	{
		EditorSubsystem->OnMegaMeshChanged().RemoveAll(this);
	}

	DataStorage.OnUpdateCompleted().RemoveAll(this);

#if WITH_EDITOR
	FEditorDelegates::MapChange.RemoveAll(this);
#endif
}

void UMegaMeshTedsFactory::OnMegaMeshChanged(const FOnChangedEventInfo& Info)
{
	bShouldResolveBoundsFiltering.store(true);
}

void UMegaMeshTedsFactory::GenerateModifierSortKeys()
{
	using namespace UE::Editor::DataStorage;

	if (!bShouldUpdateModifierSortKeys.exchange(false))
	{
		return;
	}

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	FRowHandleArray ModifierRows;
	DataStorage->RunQuery(MegaMeshModifierQueryRW, Queries::CreateDirectQueryCallbackBinding(
		[&ModifierRows](IDirectQueryContext& Context, const RowHandle* Rows)
		{
			ModifierRows.Append(FRowHandleArrayView(Rows, Context.GetRowCount(), FRowHandleArrayView::EFlags::IsUnique));
		}));

	TMap<TWeakObjectPtr<AMeshPartition>, TArray<RowHandle>> ModifiersPerMegaMesh;

	for (RowHandle ModifierRow : ModifierRows.GetRows())
	{
		FMegaMeshReferenceColumn* RefColumn = DataStorage->GetColumn<FMegaMeshReferenceColumn>(ModifierRow);
		if (RefColumn && RefColumn->Mesh.IsValid() )
		{
			ModifiersPerMegaMesh.FindOrAdd(RefColumn->Mesh).Add(ModifierRow);
		}
	}

	for (TTuple< TWeakObjectPtr<AMeshPartition>, TArray<RowHandle> > MapEntry : ModifiersPerMegaMesh)
	{
		UMeshPartitionDefinition* Definition = MapEntry.Key.Get()->GetMeshPartitionDefinition();

		TArray<FModifierDesc> ModifierDescriptors;
		TMap<FSoftObjectPath, UModifierComponent*> DescriptorToModifier;

		for (RowHandle ModifierRow : MapEntry.Value)
		{
			FTypedElementUObjectColumn* Object = DataStorage->GetColumn<FTypedElementUObjectColumn>(ModifierRow);
			if (UModifierComponent* Modifier = Cast<UModifierComponent>(Object->Object); Modifier != nullptr)
			{
				FModifierDesc Descriptor(*Modifier);
				ModifierDescriptors.Add(Descriptor);			
				DescriptorToModifier.Add(Descriptor.ModifierPath, Modifier);
			}
		}

		if (Definition)
		{
			SortModifierDescriptors(Definition->GetModifierTypePriorities(), ModifierDescriptors);
		}
		else
		{
			SortModifierDescriptors(TConstArrayView<FName>(), ModifierDescriptors);
		}
		
		TArray<FModifierDesc>::TConstIterator Iterator = ModifierDescriptors.CreateConstIterator();
		while (Iterator)
		{
			RowHandle ModifierRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(*DescriptorToModifier.Find(Iterator->ModifierPath)));

			FMegaMeshModifierSortColumn* SortColumn = DataStorage->GetColumn<FMegaMeshModifierSortColumn>(ModifierRow);
			if (SortColumn)
			{
				SortColumn->SortKey = Iterator.GetIndex();
			}
			Iterator++;
		}

	}
}

void UMegaMeshTedsFactory::ResolveBoundsFiltering()
{
	using namespace Editor::DataStorage;
	using namespace Editor::DataStorage::Queries;

	if (!bShouldResolveBoundsFiltering.exchange(false))
	{
		return;
	}

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);


	TArray<RowHandle> MegaMeshRows;
	TArray<RowHandle> ModifierRows;
	RowHandle BoundsFilterObjectRow = InvalidRowHandle;

	DataStorage->RunQuery(MegaMeshQuery,
		CreateDirectQueryCallbackBinding(
			[&MegaMeshRows](IDirectQueryContext& Context, RowHandle Row)
			{
				MegaMeshRows.Add(Row);
			})
	);

	DataStorage->RunQuery(MegaMeshModifierQueryRW,
		CreateDirectQueryCallbackBinding(
			[&ModifierRows](IDirectQueryContext& Context, RowHandle Row)
			{
				ModifierRows.Add(Row);
			})
	);

	for (RowHandle ModifierRow : ModifierRows)
	{
		DataStorage->RemoveColumns<FMegaMeshAffectsFilterBounds>(ModifierRow);
	}

	DataStorage->RunQuery(BoundsFilterObjectQuery,
		CreateDirectQueryCallbackBinding(
			[&BoundsFilterObjectRow](IDirectQueryContext& Context, RowHandle Row)
			{
				BoundsFilterObjectRow = Row;
			})
	);

	FBox TargetBounds;
	bool bHasBounds = false;

	if (BoundsFilterObjectRow != InvalidRowHandle)
	{
		FTypedElementUObjectColumn* ObjCol = DataStorage->GetColumn<FTypedElementUObjectColumn>(BoundsFilterObjectRow);
		if (ObjCol)
		{
			if (AActor* Actor = Cast<AActor>(ObjCol->Object); Actor != nullptr)
			{
				TargetBounds = Actor->GetComponentsBoundingBox();
				bHasBounds = true;
			}
			
		}
	}

	if (bHasBounds)
	{
		for (RowHandle MegaMeshRow : MegaMeshRows)
		{
			FTypedElementUObjectColumn* ObjectCol = DataStorage->GetColumn<FTypedElementUObjectColumn>(MegaMeshRow);
			if (!ObjectCol)
			{
				continue;
			}
			if (AMeshPartition* MegaMesh = Cast<AMeshPartition>(ObjectCol->Object); MegaMesh != nullptr)
			{
				UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent());
				if (!EditorComponent)
				{
					continue;
				}

				TArray<UModifierComponent*> AffectingModifiers;
				TArray<FBox> BoundsToCheck;
				BoundsToCheck.Add(TargetBounds);
				EditorComponent->GetModifiersAffectingBounds(AffectingModifiers, BoundsToCheck);

				for (UModifierComponent* Modifier : AffectingModifiers)
				{
					if (Modifier->IsBase())
					{
						continue;
					}

					RowHandle ModifierRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(Modifier));
					if (ModifierRow != InvalidRowHandle)
					{
						DataStorage->AddColumn<FMegaMeshAffectsFilterBounds>(ModifierRow);
					}
				}
			}
		}
	}
}

void UMegaMeshTedsFactory::ResolveLayerRows()
{
	using namespace Editor::DataStorage;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	FRowHandleArray UnresolvedRows;
	// Pull all unresolved rows out of TEDS as the context can't directly manipulate them. This is akin to not being allowed to modify an array while iterating over it.
	DataStorage->RunQuery(UnresolvedRetrievalQuery, Queries::CreateDirectQueryCallbackBinding(
		[&UnresolvedRows](IDirectQueryContext& Context, const RowHandle* Rows)
		{
			UnresolvedRows.Append(FRowHandleArrayView(Rows, Context.GetRowCount(), FRowHandleArrayView::EFlags::IsUnique));
		}));

	// This is the same value the Builder uses... TODO: Makes sure these are kept in sync in case the builder changes
	constexpr uint32 UngroupedLayerIndex = TNumericLimits<uint32>::Max();

	for (RowHandle ComponentRow : UnresolvedRows.GetRows())
	{
		if (const FUnresolvedMegaMeshLayer* UnresolvedLayer = DataStorage->GetColumn<FUnresolvedMegaMeshLayer>(ComponentRow))
		{
			FString LayerKey = "";
			RowHandle ParentRow = InvalidRowHandle;
			FName LayerName = UnresolvedLayer->Layer;
			uint32 LayerStackIndex = 0;

			if (FTypedElementUObjectColumn* Object = DataStorage->GetColumn<FTypedElementUObjectColumn>(ComponentRow))
			{
				if (MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(Object->Object); Modifier != nullptr)
				{
					if (FMegaMeshReferenceColumn* MeshRef = DataStorage->GetColumn<FMegaMeshReferenceColumn>(ComponentRow))
					{
						TWeakObjectPtr<AMeshPartition> MegaMeshInstance = MeshRef->Mesh;

						if (MegaMeshInstance.IsValid())
						{
							ParentRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(MegaMeshInstance.Get())); // Actors are automatically registered in TEDS Mapping Table.
							if (!DataStorage->IsRowAvailable(ParentRow))
							{
								// Parent has been deleted so move on to the next ColumnRow.
								continue;
							}

							// If there is no EditorComponent, fall back to ungrouped. If one is set later, the per-tick update processor will detect the change and
							// re-sort the modifier into the correct layer.
							const UMeshPartitionEditorComponent* const EditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMeshInstance->GetMeshPartitionComponent());
							LayerStackIndex = EditorComponent ? EditorComponent->GetModifierLayerIndex(UnresolvedLayer->Layer) : UngroupedLayerIndex;

							if (LayerStackIndex == UngroupedLayerIndex)
							{
								LayerName = TEXT("Unassigned Modifiers");
							}
							LayerKey = UMegaMeshEditorUISubsystem::Local::GenerateLayerMapKey(LayerStackIndex, *MegaMeshInstance);
						}
					}
				}
			}

			 

			// First check if it's not already added as that could have happened in a previous iteration of this loop.
			RowHandle LayerRow = DataStorage->LookupMappedRow(UMegaMeshEditorUISubsystem::Local::MegaMeshMappingDomain, FMapKeyView(LayerKey));
			if (DataStorage->IsRowAvailable(LayerRow))
			{
				// Layer row already exists, reuse it.
			}
			else if (LayerRow != InvalidRowHandle)
			{
				// A mapping exists but the row is no longer available (deleted by
				// HandlePotentialLayerRemoval in an observer context, with mapping
				// cleanup deferred). Skip this modifier and let it resolve on the
				// next cycle when the stale mapping has been cleaned up.
				continue;
			}
			else
			{

				

				LayerRow = DataStorage->AddRow(MegaMeshDefinitionLayerTable);

				DataStorage->AddColumn(LayerRow, FTypedElementLabelColumn{ .Label = LayerName.ToString() });
				DataStorage->AddColumn(LayerRow, FMegaMeshLayerNameColumn{ .Name = LayerName });
				DataStorage->AddColumn(LayerRow, FTypedElementRowReferenceColumn{ .Row = ParentRow });
				DataStorage->AddColumn(LayerRow, FMegaMeshRowParentColumn{ .Parent = ParentRow });
				DataStorage->SetParentRow(MegaMeshHierarchy, LayerRow, ParentRow);
				DataStorage->AddColumn(LayerRow, FMegaMeshPriorityColumn{ .Priority = LayerStackIndex });
				DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(LayerRow);

				if (DataStorage->HasColumns<FIsMegaMeshActiveInOutlinerTag>(ParentRow))
				{
					DataStorage->AddColumns<FIsMegaMeshActiveInOutlinerTag>(LayerRow);
				}

				// These can be stored in FUnresolvedMegaMeshLayer or the data can be retrieved again.
				//Context.AddColumn(LayerRow, MoveTemp(LayerHierarchyData));

				FMegaMeshRowParentColumn* MeshHierarchyData = DataStorage->GetColumn< FMegaMeshRowParentColumn>(ParentRow);
				if (MeshHierarchyData != nullptr)
				{
					MeshHierarchyData->ChildrenSet.Add(LayerRow, 0);
					DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(ParentRow);

				}

				FTedsOutlinerContextMenuColumn PlaceholderSection;

				if (LayerStackIndex != UngroupedLayerIndex)
				{
					PlaceholderSection.OnCreateContextMenu = MakeShared<FTedsOutlinerContextMenuDelegate>();
					PlaceholderSection.OnCreateContextMenu->BindLambda([LayerRow, LayerName](UToolMenu* ToolMenu, SSceneOutliner& SceneOutliner)
						{
							ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
							if (DataStorage == nullptr)
							{
								return;
							}

							FToolMenuSection& Section = ToolMenu->AddSection("Layer Actions", LOCTEXT("LayerActionsTitle", "Layer Actions"));
							Section.AddMenuEntry
							(
								"LayerActionMakeActiveLayer",
								LOCTEXT("LayerActionMakeActive_Title", "Make Layer Active"),
								LOCTEXT("LayerActionMakeActive_ToolTip", "Make the selected layer active, assign newly created modifiers with place modifier tool to this layer."),
								FSlateIcon(),
								FUIAction
								(
									FExecuteAction::CreateLambda([LayerRow, LayerName, DataStorage]()
										{
											const UMegaMeshTedsFactory* Factory = DataStorage->FindFactory<UMegaMeshTedsFactory>();
											if (!Factory)
											{
												return;
											}

											RowHandle PreviousActiveLayer = InvalidRowHandle;
											DataStorage->RunQuery(Factory->ActiveLayerQuery, Queries::CreateDirectQueryCallbackBinding(
												[&PreviousActiveLayer](IDirectQueryContext& Context, const RowHandle Row)
												{
													ensure(PreviousActiveLayer == InvalidRowHandle);
													PreviousActiveLayer = Row;
												}));

											if (PreviousActiveLayer != InvalidRowHandle)
											{
												DataStorage->RemoveColumn<FMegaMeshActiveLayerTag>(PreviousActiveLayer);
											}
											DataStorage->AddColumns<FMegaMeshActiveLayerTag>(LayerRow);
										}),

									FCanExecuteAction::CreateLambda([LayerRow, LayerName, DataStorage]() -> bool
										{
											return !DataStorage->HasColumns<FMegaMeshActiveLayerTag>(LayerRow);
										})
								)
							);
						});

					DataStorage->AddColumn(LayerRow, MoveTemp(PlaceholderSection));
				}

				// Don't forget the register the layer for retrieval in the future. Also, don't worry about deleting this, if the row gets deleted then the mapping
				// will be deleted automatically as well.
				DataStorage->MapRow(UMegaMeshEditorUISubsystem::Local::MegaMeshMappingDomain, FMapKey(LayerKey), LayerRow);
			}

			DataStorage->AddColumn(ComponentRow, FTypedElementRowReferenceColumn{ .Row = LayerRow });

			if (DataStorage->HasColumns<FIsMegaMeshActiveInOutlinerTag>(LayerRow))
			{
				DataStorage->AddColumns<FIsMegaMeshActiveInOutlinerTag>(ComponentRow);
			}

			FMegaMeshRowParentColumn* LayerHierarchyData = DataStorage->GetColumn< FMegaMeshRowParentColumn>(LayerRow);
			if (LayerHierarchyData != nullptr)
			{
				LayerHierarchyData->ChildrenSet.Add(ComponentRow, 0);
			}
			DataStorage->AddColumn(ComponentRow, FMegaMeshRowParentColumn{ .Parent = LayerRow });
			DataStorage->SetParentRow(MegaMeshHierarchy, ComponentRow, LayerRow);
			DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(ComponentRow);
			DataStorage->RemoveColumns<FUnresolvedMegaMeshLayer>(ComponentRow);
		}
	}

}

void UMegaMeshTedsFactory::CleanMegaMeshRows(uint32 MapFlags)
{
	using namespace Editor::DataStorage;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	DataStorage->RemoveAllRowsWith<FIsMegaMeshDefinitionLayerTag>();
}
}
#undef LOCTEXT_NAMESPACE
