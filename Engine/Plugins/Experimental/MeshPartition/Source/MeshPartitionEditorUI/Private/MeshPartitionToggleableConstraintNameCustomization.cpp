// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionToggleableConstraintNameCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailPropertyRow.h"
#include "MeshPartitionChannel.h"
#include "MeshPartitionEditorUIStyle.h"
#include "PropertyEditorUtils.h"
#include "PropertyHandle.h"
#include "Widgets/Input/SSearchableComboBox.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SCheckBox.h"

#define LOCTEXT_NAMESPACE "MeshVertexSculptToolCustomizations"

TSharedRef<IPropertyTypeCustomization> UE::MeshPartition::FToggleableConstraintNameCustomization::MakeInstance()
{
	return MakeShared<UE::MeshPartition::FToggleableConstraintNameCustomization>();
}

void UE::MeshPartition::FToggleableConstraintNameCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> NameHandle = StructHandle->GetChildHandle(TEXT("Name"));
	if (!ensure(NameHandle))
	{
		return;
	}

	IDetailPropertyRow& Row = ChildBuilder.AddProperty(NameHandle.ToSharedRef());

	// Get the default widgets so that we can reuse the default value widget
	//  when we're not creating an options box
	TSharedPtr<SWidget> DefaultNameWidget, DefaultValueWidget;
	Row.GetDefaultWidgets(DefaultNameWidget, DefaultValueWidget);

	// Customize the value widget
	TSharedPtr<SHorizontalBox> ValueContent;
	Row.CustomWidget()
		.NameContent()
		[
			// Use the name for the struct, not the channel property, so that user can rename it
			//  if they want.
			StructHandle->CreatePropertyNameWidget()
		]
		.ValueContent()
		[
			SAssignNew(ValueContent, SHorizontalBox)
		];

	// Holds the state of the toggle button
	TSharedPtr<bool> ConstrainToDefinition = MakeShared<bool>();

	ValueContent->AddSlot()
	[
		DefaultValueWidget.ToSharedRef()
	];

	DefaultValueWidget->SetVisibility(MakeAttributeLambda([ConstrainToDefinition]()->EVisibility
	{
		return *ConstrainToDefinition ? EVisibility::Collapsed : EVisibility::Visible;
	}));

	
	ValueContent->AddSlot()
	[
		SAssignNew(ComboBox, SSearchableComboBox)
		[
			SNew(STextBlock)
			.Text_Lambda([NameHandle]()->FText
			{
				FName NameOut;
				NameHandle->GetValue(NameOut);
				return FText::FromName(NameOut);
			})
			.Font(CustomizationUtils.GetRegularFont())
		]
		.Visibility_Lambda([ConstrainToDefinition]()->EVisibility 
		{
			return *ConstrainToDefinition ? EVisibility::Visible : EVisibility::Collapsed;
		})
		.OptionsSource(&ChannelOptions)
		.OnComboBoxOpening(FOnComboBoxOpening::CreateSPLambda(StructHandle, [StructHandle, this]()
		{
			bool bCurrentNameIsValidOption;
			ReinitializeOptions(StructHandle, bCurrentNameIsValidOption);
		}))
		// Gets called to generate each of the dropdown entries
		.OnGenerateWidget(SSearchableComboBox::FOnGenerateWidget::CreateSPLambda(StructHandle,
			[](TSharedPtr<FString> InComboString) ->TSharedRef<SWidget> 
		{
			return
				SNew(STextBlock)
				.Text(FText::FromString(*InComboString));
		}))
		.OnSelectionChanged(SSearchableComboBox::FOnSelectionChanged::CreateSPLambda(StructHandle,
			[NameHandle](TSharedPtr<FString> InSelectedItem, ESelectInfo::Type SelectInfo)
		{
			// Ignore programmatic selection changes; only write back on
			// genuine user input from the dropdown.
			if (SelectInfo == ESelectInfo::Direct)
			{
				return;
			}
			FName NewName(**InSelectedItem);
			NameHandle->SetValue(NewName);
		}))
		.SearchVisibility(EVisibility::Visible)
	];

	// Add toggle for constraining to definition
	ValueContent->AddSlot()
		.AutoWidth()
		[
			SNew(SCheckBox)
			.Style(FAppStyle::Get(), "DetailsView.SectionButton")
			.Padding(FMargin(4, 2))
			.ToolTipText(LOCTEXT("ConstrainToDefinitionTooltip", "Constrain options to ones defined in the Mesh Partition Definition."))
			.HAlign(HAlign_Center)
			.OnCheckStateChanged_Lambda([ConstrainToDefinition](const ECheckBoxState NewState)
			{
				*ConstrainToDefinition = NewState == ECheckBoxState::Checked;
			})
			.IsChecked_Lambda([ConstrainToDefinition]() -> ECheckBoxState
			{
				return *ConstrainToDefinition ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
			})
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(0))
					.AutoWidth()
					[
						SNew(SImage)
							.Image(FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("MeshPartitionDefinition")))
					]
			]
		];

	// The reinitialization call should be made once the above widgets are created.
	bool bCurrentNameIsValidOption = false;
	ReinitializeOptions(StructHandle, bCurrentNameIsValidOption);
	*ConstrainToDefinition = bCurrentNameIsValidOption;
}

void UE::MeshPartition::FToggleableConstraintNameCustomization::ReinitializeOptions(TSharedRef<IPropertyHandle> StructHandle, bool& bCurrentNameIsValidOption)
{
	bCurrentNameIsValidOption = false;

	TSharedPtr<IPropertyHandle> NameHandle = StructHandle->GetChildHandle(TEXT("Name"));
	if (!ensure(NameHandle))
	{
		return;
	}

	FName CurrentName;
	const FPropertyAccess::Result GetValueResult = NameHandle->GetValue(CurrentName);

	// When multiple objects are selected with different values, GetValue returns
	// MultipleValues and leaves CurrentName at its default (NAME_None). We must
	// not propagate that bogus None into the combo box or it will appear as if
	// the property is unset.
	if (GetValueResult == FPropertyAccess::MultipleValues)
	{
		if (ComboBox)
		{
			ComboBox->RefreshOptions();
		}
		return;
	}

	// None is always a valid option (we'll look for others later)
	bCurrentNameIsValidOption = CurrentName.IsNone();

	// Call the GetOptions function to get our options
	ChannelOptions.Reset();
	FString GetOptionsFunctionName = StructHandle->GetProperty()->GetMetaData("GetOptions");
	if (ensureMsgf(!GetOptionsFunctionName.IsEmpty(), TEXT("Need a GetOptions MetaData tag on a "
		"FToggleableConstraintNameCustomization pointing to a function that gives valid options.")))
	{
		TArray<UObject*> OuterObjects;
		StructHandle->GetOuterObjects(OuterObjects);

		TArray<FString> OptionStrings;
		PropertyEditorUtils::GetPropertyOptions(OuterObjects, GetOptionsFunctionName, OptionStrings, nullptr);
		Algo::Transform(OptionStrings, ChannelOptions, [](const FString& Str) { return MakeShared<FString>(Str); });
	}

	// Update the currently selected item in the combo box internal state
	if (ComboBox)
	{
		ComboBox->RefreshOptions();
		FString CurrentNameString = CurrentName.ToString();
		TSharedPtr<FString>* ExistingOption = ChannelOptions.FindByPredicate([CurrentNameString](TSharedPtr<FString> String) { return String && *String == CurrentNameString; });
		if (ExistingOption)
		{
			bCurrentNameIsValidOption = true;
			ComboBox->SetSelectedItem(*ExistingOption, ESelectInfo::Direct);
		}
		else
		{
			ComboBox->SetSelectedItem(MakeShared<FString>(CurrentNameString), ESelectInfo::Direct);
		}
	}
}

#undef LOCTEXT_NAMESPACE