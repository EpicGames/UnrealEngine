// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "Containers/Array.h"

class FString;
class SSearchableComboBox;
class SWidget;

namespace UE::MeshPartition
{
/**
* Creates a widget that has a toggle that either uses GetOptions metadata function for
*  constraining the options, or allows text entry if the toggle is off.
*/
class FToggleableConstraintNameCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	// Does nothing, so that the child row is placed inline where the channel name selector is placed.
	virtual void CustomizeHeader(
		TSharedRef<IPropertyHandle> PropertyHandle,
		FDetailWidgetRow& HeaderRow,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override {}

	virtual void CustomizeChildren(
		TSharedRef<IPropertyHandle> PropertyHandle,
		IDetailChildrenBuilder& ChildBuilder,
		IPropertyTypeCustomizationUtils& CustomizationUtils) override;

private:
	TSharedPtr<SSearchableComboBox> ComboBox;
	TArray<TSharedPtr<FString>> ChannelOptions;
	void ReinitializeOptions(TSharedRef<IPropertyHandle> StructHandle, bool& bCurrentNameIsValidOption);
};
} // namespace UE::MeshPartition