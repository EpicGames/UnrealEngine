// Copyright Epic Games, Inc. All Rights Reserved.

#include "SculptLayersModifierController.h"

#include "MeshPartitionEditorUIModule.h"
#include "ModelingWidgets/SMeshLayersStack.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"

#define LOCTEXT_NAMESPACE "SculptLayersModifierController"

namespace UE::MeshPartition
{
FSculptLayersModifiersController::FSculptLayersModifiersController() { }

FName FSculptLayersModifiersController::GetLayerName(const int32 InLayerIndex) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return NAME_None;
	}
	
	return Properties->GetLayerName(InLayerIndex);
}

void FSculptLayersModifiersController::SetLayerName(const int32 InLayerIndex, const FName InName) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return;
	}
	
	Properties->SetLayerName(InLayerIndex, InName);
}

double FSculptLayersModifiersController::GetLayerWeight(const int32 InLayerIndex) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return 0.0;
	}
	
	// ensure the provided index is valid
	if (!Properties->LayerWeights.IsValidIndex(InLayerIndex))
	{
		UE_LOGF(LogMegaMeshEditorUI, Warning, "Sculpt Layer not found. Invalid index: %d.", InLayerIndex);
		return INDEX_NONE;
	}
	
	return Properties->LayerWeights[InLayerIndex];
}

void FSculptLayersModifiersController::SetLayerWeight(const int32 InLayerIndex, const double InWeight, const EPropertyChangeType::Type ChangeType) const
{
	// ensure the properties are valid
	if (!ensure(Properties))
	{
		return;
	}
	
	Properties->SetLayerWeight(InLayerIndex, InWeight, ChangeType);
}

int32 FSculptLayersModifiersController::GetNumMeshLayers() const
{
	if (!ensure(Properties))
	{
		return INDEX_NONE;
	}
	return Properties->LayerWeights.Num();
}

void FSculptLayersModifiersController::RefreshLayersStackView() const
{
	if (LayersStackView.IsValid())
	{
		LayersStackView->RefreshStackView();
	}
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE