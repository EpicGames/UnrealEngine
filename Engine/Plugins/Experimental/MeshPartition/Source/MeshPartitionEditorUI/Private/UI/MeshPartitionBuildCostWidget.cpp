// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionBuildCostWidget.h"

#include "MeshPartition.h"

#include "MeshPartitionEditorUIStyle.h"

#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Layout/Visibility.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Columns/TypedElementLabelColumns.h"

#include "Columns/LayerOutlinerColumns.h"
#include "Columns/SlateHeaderColumns.h"

#define LOCTEXT_NAMESPACE "MegaMeshBuildCostWidget"

namespace UE::MeshPartition
{
//
// FMegaMeshBuildCostWidgetHeaderConstructor
//

FMegaMeshBuildCostWidgetHeaderConstructor::FMegaMeshBuildCostWidgetHeaderConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshBuildCostWidgetHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Fixed,
			.Width = 24.0f
		});

	return SNew(SImage)
		.DesiredSizeOverride(FVector2D(16.f, 16.f))
		.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Scalability")))
		.ToolTipText(LOCTEXT("MegaMeshBuildCostWidgetHeader", "Build Cost"));
}

//
// FMegaMeshBuildCostWidgetConstructor
//

namespace MegaMeshBuildCostWidgetLocals
{
	FColor GetColorForTime(double Time, const FMegaMeshTimingStatistics& AggregateStatistics)
	{
		FColor Color;

		// Red if we're greater than 2 standard deviations away from the mean
		if (Time > (AggregateStatistics.TotalTimeMean + 2.0 * AggregateStatistics.TotalTimeStandardDeviation))
		{
			Color = FColor(162, 49, 49);
		}
		// Yellow if we're greater than 1 standard deviation away from the mean
		else if (Time > (AggregateStatistics.TotalTimeMean + AggregateStatistics.TotalTimeStandardDeviation))
		{
			Color = FColor(189, 155, 68);
		}
		// Green if we're less than a standard deviation away from the mean
		else
		{
			Color = FColor(57, 136, 75);
		}
		return Color;
	}

	const FSlateBrush* GetBrushForTime(double Time, const FMegaMeshTimingStatistics& AggregateStatistics)
	{	
		return FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildCostAggregate"));
	}
}

//
//
//

FMegaMeshBuildCostWidgetConstructorDelegator::FMegaMeshBuildCostWidgetConstructorDelegator()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshBuildCostWidgetConstructorDelegator::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	if (DataStorage->IsRowAvailable(TargetRow))
	{
		Editor::DataStorage::IUiProvider::FWidgetConstructorPtr OutWidgetConstructorPtr;

		auto AssignWidgetToColumn = [&OutWidgetConstructorPtr](Editor::DataStorage::IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				OutWidgetConstructorPtr = WidgetConstructor;
				return false;
			};


		// List all columns on the row, then use these columns to refine our widget matching
		TArray<TWeakObjectPtr<const UScriptStruct>> RowColumns;
		DataStorage->ListColumns(TargetRow, [&RowColumns](const UScriptStruct& ColumnType)
			{
				RowColumns.Emplace(&ColumnType);
				return true;
			});

		const UE::Editor::DataStorage::IUiProvider::FPurposeID PurposeID = UE::Editor::DataStorage::IUiProvider::FPurposeInfo("MegaMesh", "Outliner", "BuildCost").GeneratePurposeID();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(PurposeID), UE::Editor::DataStorage::IUiProvider::EMatchApproach::LongestMatch, RowColumns, Arguments, AssignWidgetToColumn);
		

		if (OutWidgetConstructorPtr)
		{
			TSharedPtr<SWidget> Widget = DataStorageUi->ConstructWidget(WidgetRow, *OutWidgetConstructorPtr, Arguments);

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					Widget.ToSharedRef()
				];
		}
	}

	return SNew(SBox);
}

//
//
//

FMegaMeshBuildCostWidgetConstructor::FMegaMeshBuildCostWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshBuildCostWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SMegaMeshBuildCostWidget, TargetRow, WidgetRow)
				.UseAggregateView(false)
				.ToolTip(SNew(SBuildStatsToolTip).DataStorage(DataStorage).TargetRow(TargetRow))				
		];
}

//
//
//


void SMegaMeshBuildCostWidget::Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow)
{
	TargetRow = InTargetRow;
	WidgetRow = InWidgetRow;

	bool bUseAggregateView = InArgs._UseAggregateView.Get();

	if (!bUseAggregateView)
	{
		ChildSlot
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()					
					[
						SNew(STextBlock)
							.Text(this, &SMegaMeshBuildCostWidget::GetLabel)
							.ColorAndOpacity(this, &SMegaMeshBuildCostWidget::GetColor)
					]

			];
	}
	else
	{
		ChildSlot
			[
				SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
							.DesiredSizeOverride(FVector2D(16.f, 16.f))
							.Image(this, &SMegaMeshBuildCostWidget::GetAggregateImageBrush)
							.ColorAndOpacity(this, &SMegaMeshBuildCostWidget::GetAggregateColor)
					]

			];
	}
}

FText SMegaMeshBuildCostWidget::GetLabel() const
{
	UE::Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage();
	FNumberFormattingOptions Options;
	Options.MinimumFractionalDigits = 3;

	if (DataStorage)
	{
		FMegaMeshModifierTiming* TimingData = DataStorage->GetColumn<FMegaMeshModifierTiming>(TargetRow);
		if (TimingData)
		{
			return FText::AsNumber(TimingData->TotalTime, &Options);
		}
	}

	return LOCTEXT("MegaMeshLayerNameLabel", "N/A");
}

FSlateColor SMegaMeshBuildCostWidget::GetColor() const
{
	FColor Color = FColor::Green;
	using namespace UE::Editor::DataStorage;
	UE::Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage();
	if (DataStorage)
	{
		FMegaMeshReferenceColumn* MegaMeshRef = DataStorage->GetColumn<FMegaMeshReferenceColumn>(TargetRow);
		if (!MegaMeshRef || !MegaMeshRef->Mesh.IsValid())
		{
			return FSlateColor();
		}

		AMeshPartition* MegaMeshInstance = Cast<AMeshPartition>(MegaMeshRef->Mesh.Get());
		RowHandle MegaMeshRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(MegaMeshInstance));

		FMegaMeshTimingStatistics* TimingStats = DataStorage->GetColumn<FMegaMeshTimingStatistics>(MegaMeshRow);
		FMegaMeshModifierTiming* TimingData = DataStorage->GetColumn<FMegaMeshModifierTiming>(TargetRow);
		if (TimingData && TimingStats)
		{
			Color = MegaMeshBuildCostWidgetLocals::GetColorForTime(TimingData->TotalTime, *TimingStats);
		}
	}

	return FSlateColor(Color);
}

const FSlateBrush* SMegaMeshBuildCostWidget::GetAggregateImageBrush() const
{
	using namespace UE::Editor::DataStorage;
	UE::Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage();
	if (DataStorage)
	{
		FMegaMeshReferenceColumn* MegaMeshRef = DataStorage->GetColumn<FMegaMeshReferenceColumn>(TargetRow);
		if (!MegaMeshRef || !MegaMeshRef->Mesh.IsValid())
		{
			return nullptr;
		}

		AMeshPartition* MegaMeshInstance = Cast<AMeshPartition>(MegaMeshRef->Mesh.Get());
		RowHandle MegaMeshRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(MegaMeshInstance));

		FMegaMeshTimingStatistics* TimingStats = DataStorage->GetColumn<FMegaMeshTimingStatistics>(MegaMeshRow);
		FMegaMeshModifierTiming* TimingData = DataStorage->GetColumn<FMegaMeshModifierTiming>(TargetRow);
		if (TimingData && TimingStats)
		{
			return MegaMeshBuildCostWidgetLocals::GetBrushForTime(TimingData->MaxTime, *TimingStats);
		}
	}

	ensure(false);
	return nullptr;
}

FSlateColor SMegaMeshBuildCostWidget::GetAggregateColor() const
{
	FColor Color = FColor::Green;
	using namespace UE::Editor::DataStorage;
	UE::Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage();
	if (DataStorage)
	{
		FMegaMeshReferenceColumn* MegaMeshRef = DataStorage->GetColumn<FMegaMeshReferenceColumn>(TargetRow);
		if (!MegaMeshRef || !MegaMeshRef->Mesh.IsValid())
		{
			return FSlateColor();
		}

		AMeshPartition* MegaMeshInstance = Cast<AMeshPartition>(MegaMeshRef->Mesh.Get());
		RowHandle MegaMeshRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(MegaMeshInstance));

		FMegaMeshTimingStatistics* TimingStats = DataStorage->GetColumn<FMegaMeshTimingStatistics>(MegaMeshRow);
		FMegaMeshModifierTiming* TimingData = DataStorage->GetColumn<FMegaMeshModifierTiming>(TargetRow);
		if (TimingData && TimingStats)
		{
			Color = MegaMeshBuildCostWidgetLocals::GetColorForTime(TimingData->MaxTime, *TimingStats);
		}
	}

	return FSlateColor(Color);
}

UE::Editor::DataStorage::ICoreProvider* SMegaMeshBuildCostWidget::GetDataStorage()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

UE::Editor::DataStorage::IUiProvider* SMegaMeshBuildCostWidget::GetDataStorageUI()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
}

UE::Editor::DataStorage::ICompatibilityProvider* SMegaMeshBuildCostWidget::GetDataStorageCompatibility()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}


//
//
//

FMegaMeshBuildCostAggregateWidgetConstructor::FMegaMeshBuildCostAggregateWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshBuildCostAggregateWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SMegaMeshBuildCostWidget, TargetRow, WidgetRow)
				.UseAggregateView(true)

		];
}


//
//
//

void SBuildStatsToolTip::Construct(const FArguments& InArgs)
{
	TargetRowHandle = InArgs._TargetRow;
	DataStorage = InArgs._DataStorage;

	SToolTip::Construct(
		SToolTip::FArguments()
		.TextMargin(0.f));
}

// IToolTip interface
bool SBuildStatsToolTip::IsEmpty() const 
{
	return !DataStorage || !DataStorage->IsRowAvailable(TargetRowHandle);
}

void SBuildStatsToolTip::OnOpening() 
{
	using namespace UE::Editor::DataStorage;
	FNumberFormattingOptions OptionsFloating;
	OptionsFloating.MinimumFractionalDigits = 3;

	FNumberFormattingOptions Options;
	Options.SetRoundingMode(ERoundingMode::ToZero);
	Options.MaximumFractionalDigits = 0;

	TSharedPtr<SBox> ToolTipBox = SNew(SBox)
		.Padding(2);

	TSharedPtr<SVerticalBox> ToolTipWidget = SNew(SVerticalBox);

	


	if (DataStorage)
	{
		FMegaMeshReferenceColumn* MegaMeshRef = DataStorage->GetColumn<FMegaMeshReferenceColumn>(TargetRowHandle);
		if (!MegaMeshRef || !MegaMeshRef->Mesh.IsValid())
		{
			return;
		}
		AMeshPartition* MegaMeshInstance = Cast<AMeshPartition>(MegaMeshRef->Mesh.Get());
		RowHandle MegaMeshRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(MegaMeshInstance));

		FMegaMeshTimingStatistics* TimingStats = DataStorage->GetColumn<FMegaMeshTimingStatistics>(MegaMeshRow);
		FMegaMeshModifierTiming* TimingData = DataStorage->GetColumn<FMegaMeshModifierTiming>(TargetRowHandle);
		if (TimingData && TimingStats)
		{
			ToolTipWidget->AddSlot()
				.AutoHeight()
				[
					SNew(SBorder)
					.Padding(3,5)
					.BorderImage(FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildCostToolTipTitle")))
					[
						SNew(STextBlock)
							.Text(LOCTEXT("BuildCostWidgetToolTipTitle", "Build Time Statistics:"))
					]
				];

			ToolTipWidget->AddSlot()
				.Padding(8,3)
				[
					SNew(SHorizontalBox)						
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.MinWidth(200)
						.HAlign(EHorizontalAlignment::HAlign_Left)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("BuildCostWidgetToolTip_InstanceCount", "Sections Modified"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.HAlign(EHorizontalAlignment::HAlign_Right)
						[
							SNew(STextBlock)
								.Text(FText::AsNumber(TimingData->InstanceCount, &Options))
						]

				];

			ToolTipWidget->AddSlot()
				.Padding(8, 3)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.MinWidth(200)
						.HAlign(EHorizontalAlignment::HAlign_Left)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("BuildCostWidgetToolTip_TotalTime", "Total Time (s)"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.HAlign(EHorizontalAlignment::HAlign_Right)
						[
							SNew(STextBlock)
								.Text(FText::AsNumber(TimingData->TotalTime, &OptionsFloating))
								.ColorAndOpacity(this, &SBuildStatsToolTip::GetColor, TimingData->TotalTime, *TimingStats)
						]

				];

			ToolTipWidget->AddSlot()
				.Padding(8, 3)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.MinWidth(200)
						.HAlign(EHorizontalAlignment::HAlign_Left)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("BuildCostWidgetToolTip_MaxTime", "Max Instance Time (s)"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.HAlign(EHorizontalAlignment::HAlign_Right)
						[
							SNew(STextBlock)
								.Text(FText::AsNumber(TimingData->MaxTime, &OptionsFloating))
								.ColorAndOpacity(this, &SBuildStatsToolTip::GetColor, TimingData->MaxTime, *TimingStats)
						]

				];

			ToolTipWidget->AddSlot()
				.Padding(8, 3)
				[
					SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.MinWidth(200)
						.HAlign(EHorizontalAlignment::HAlign_Left)
						[
							SNew(STextBlock)
								.Text(LOCTEXT("BuildCostWidgetToolTip_MinTime", "Min Instance Time (s)"))
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1)
						.HAlign(EHorizontalAlignment::HAlign_Right)
						[
							SNew(STextBlock)
								.Text(FText::AsNumber(TimingData->MinTime, &OptionsFloating))
								.ColorAndOpacity(this, &SBuildStatsToolTip::GetColor, TimingData->MinTime, *TimingStats)
						]

				];
		}
	}

	ToolTipBox->SetContent(ToolTipWidget.ToSharedRef());
	SetContentWidget(ToolTipBox.ToSharedRef());
}

FSlateColor SBuildStatsToolTip::GetColor(double Time, FMegaMeshTimingStatistics TimingStats) const
{
	return FSlateColor(MegaMeshBuildCostWidgetLocals::GetColorForTime(Time, TimingStats));
}


bool SBuildStatsToolTip::IsInteractive() const 
{
	return false;
}

void SBuildStatsToolTip::OnClosed() 
{
	ResetContentWidget();
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
