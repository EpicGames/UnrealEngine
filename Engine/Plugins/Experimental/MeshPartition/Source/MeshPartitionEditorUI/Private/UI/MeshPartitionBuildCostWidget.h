// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SToolTip.h"

#include "UObject/StrongObjectPtr.h"
#include "Delegates/Delegate.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "ScopedTransaction.h"

#include "MeshPartitionBuildCostWidget.generated.h"

namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;
struct FMegaMeshTimingStatistics;

USTRUCT()
struct FMegaMeshBuildCostWidgetHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshBuildCostWidgetHeaderConstructor();
	~FMegaMeshBuildCostWidgetHeaderConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};

USTRUCT()
struct FMegaMeshBuildCostWidgetConstructorDelegator : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshBuildCostWidgetConstructorDelegator();
	~FMegaMeshBuildCostWidgetConstructorDelegator() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


USTRUCT()
struct FMegaMeshBuildCostWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshBuildCostWidgetConstructor();
	~FMegaMeshBuildCostWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


/**  */
class SMegaMeshBuildCostWidget : public SCompoundWidget
{
	using RowHandle = UE::Editor::DataStorage::RowHandle;

public:
	SLATE_BEGIN_ARGS(SMegaMeshBuildCostWidget) {}
		SLATE_ATTRIBUTE(bool, UseAggregateView);
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow);

protected:


	FText GetLabel() const;
	FSlateColor GetColor() const;

	const FSlateBrush* GetAggregateImageBrush() const;
	FSlateColor GetAggregateColor() const;

	RowHandle TargetRow;
	RowHandle WidgetRow;

private:

	static UE::Editor::DataStorage::ICoreProvider* GetDataStorage();
	static UE::Editor::DataStorage::IUiProvider* GetDataStorageUI();
	static UE::Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();
};

USTRUCT()
struct FMegaMeshBuildCostAggregateWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshBuildCostAggregateWidgetConstructor();
	~FMegaMeshBuildCostAggregateWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


class SBuildStatsToolTip : public SToolTip
{
public:
	SLATE_BEGIN_ARGS(SBuildStatsToolTip)
		: _TargetRow(UE::Editor::DataStorage::InvalidRowHandle)
		, _DataStorage(nullptr)
		{
		}

		SLATE_ARGUMENT(UE::Editor::DataStorage::RowHandle, TargetRow)
		SLATE_ARGUMENT(UE::Editor::DataStorage::ICoreProvider*, DataStorage)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	FSlateColor GetColor(double Time,  FMegaMeshTimingStatistics TimingStats) const;

	// IToolTip interface
	virtual bool IsEmpty() const override;
	virtual void OnOpening() override;
	virtual bool IsInteractive() const override;	
	virtual void OnClosed() override;

private:
	UE::Editor::DataStorage::ICoreProvider* DataStorage = nullptr;
	UE::Editor::DataStorage::RowHandle TargetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
};
} // namespace UE::MeshPartition