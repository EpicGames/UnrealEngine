// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionHierarchyWidget.h"

#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"

#include "MeshPartitionEditorUIStyle.h"

#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "Columns/TedsOutlinerColumns.h"
#include "ISceneOutliner.h"
#include "TedsOutlinerItem.h"
#include "ActorTreeItem.h"


#include "Columns/LayerOutlinerColumns.h"

#define LOCTEXT_NAMESPACE "FHierarchyWidget"

//
// MeshPartition::FHierarchyWidget
//

namespace UE::MeshPartition
{
FHierarchyWidget::FHierarchyWidget()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FHierarchyWidget::CreateWidget(Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	Editor::DataStorage::RowHandle TargetRow, Editor::DataStorage::RowHandle WidgetRow,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(MeshPartition::SHierarchyWidget, TargetRow, WidgetRow)
				.ToolTipText(LOCTEXT("MegaMeshVisibilityToggleTooltip", "Toggles the visibility."))
		];
}

//
// MeshPartition::SHierarchyWidget
//

void SHierarchyWidget::Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow)
{
	TargetRow = InTargetRow;
	WidgetRow = InWidgetRow;

	FNumberFormattingOptions Options;
	Options.UseGrouping = false;
	FText ParentString;

	using namespace Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		FMegaMeshRowParentColumn* HierarchyColumn = DataStorage->GetColumn<FMegaMeshRowParentColumn>(TargetRow);
		if (HierarchyColumn)
		{
			ParentString = FText::AsNumber(HierarchyColumn->Parent, &Options);
		}
	}

	ChildSlot
		[
			SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
						.Text(FText::Format(LOCTEXT("SMegaMeshHierarchyWidget_Parent", "Parent: {0}"), ParentString))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
						.Text(LOCTEXT("SMegaMeshHierarchyWidget_Children", "Children"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SAssignNew(ChildrenListView,SListView< TSharedPtr<FText> >)
					.ListItemsSource(&ChildrenLabels)
					.OnGenerateRow(this, &SHierarchyWidget::GenerateItemRow)
				]
		];

}

void SHierarchyWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	ChildrenLabels.Empty();
	FNumberFormattingOptions Options;
	Options.UseGrouping = false;

	using namespace Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		FMegaMeshRowParentColumn* HierarchyColumn = DataStorage->GetColumn<FMegaMeshRowParentColumn>(TargetRow);
		if (HierarchyColumn)
		{
			for (TTuple<RowHandle, int32> Child : HierarchyColumn->ChildrenSet)
			{
				ChildrenLabels.Add(MakeShared<FText>(FText::AsNumber(Child.Key, &Options)));
			}
		}
	}

	ChildrenListView->RequestListRefresh();
}


TSharedRef<ITableRow> SHierarchyWidget::GenerateItemRow(TSharedPtr<FText> Entry, const TSharedRef<STableViewBase>& OwnerTable) const
{
	if (Entry.IsValid())
	{
		return SNew(STableRow<TSharedPtr<FText>>, OwnerTable)
			[
				SNew(STextBlock)
					.Text(*Entry)
			];
	}
	else
	{
		return SNew(STableRow<TSharedPtr<FText>>, OwnerTable);
			
	}
}

Editor::DataStorage::ICoreProvider* SHierarchyWidget::GetDataStorage()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

Editor::DataStorage::ICoreProvider* SHierarchyWidget::GetDataStorageUI()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(UiFeatureName);
}

Editor::DataStorage::ICompatibilityProvider* SHierarchyWidget::GetDataStorageCompatibility()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE