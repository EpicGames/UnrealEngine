// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"

#include "UObject/StrongObjectPtr.h"
#include "Delegates/Delegate.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "SceneOutlinerFwd.h"
#include "Widgets/Views/SListView.h"

#include "ScopedTransaction.h"

#include "MeshPartitionHierarchyWidget.generated.h"

#define UE_API MESHPARTITIONEDITORUI_API


namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;

USTRUCT(MinimalAPI)
struct FHierarchyWidget : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FHierarchyWidget();
	~FHierarchyWidget() override = default;

	UE_API virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


/** Widget responsible for managing the visibility for a single item */
class SHierarchyWidget : public SCompoundWidget
{
	using RowHandle = Editor::DataStorage::RowHandle;

public:
	SLATE_BEGIN_ARGS(MeshPartition::SHierarchyWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	UE_API void Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow);

	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	RowHandle TargetRow;
	RowHandle WidgetRow;

	TArray<TSharedPtr<FText>> ChildrenLabels;
	TSharedPtr<SListView<TSharedPtr<FText>>> ChildrenListView;

	UE_API TSharedRef<ITableRow> GenerateItemRow(TSharedPtr<FText> Entry, const TSharedRef<STableViewBase>& OwnerTable) const;

private:
	static UE_API Editor::DataStorage::ICoreProvider* GetDataStorage();
	static UE_API Editor::DataStorage::ICoreProvider* GetDataStorageUI();
	static UE_API Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();
};
} // namespace UE::MeshPartition

#undef UE_API
