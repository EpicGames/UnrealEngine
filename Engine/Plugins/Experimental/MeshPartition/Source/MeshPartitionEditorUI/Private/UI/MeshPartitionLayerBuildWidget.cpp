// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionLayerBuildWidget.h"

#include "MeshPartition.h"
#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionModifierDescriptors.h"

#include "MeshPartitionEditorUIStyle.h"

#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "Columns/SceneOutlinerColumns.h"
#include "ISceneOutliner.h"
#include "TedsOutlinerItem.h"
#include "ActorTreeItem.h"
#include "TedsOutlinerHelpers.h"

#include "Columns/SlateHeaderColumns.h"
#include "Columns/LayerOutlinerColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"

#define LOCTEXT_NAMESPACE "MegaMeshLayerVisibilityWidget"

//
// Locals
//

namespace UE::MeshPartition
{
namespace MegaMeshLayerBuildWidgetLocals
{
	using namespace Editor::DataStorage;

	FMegaMeshBuildToStatus* GetBuildStatus(RowHandle TargetRow, RowHandle* MegaMeshRowOut = nullptr, RowHandle* LayerRowOut = nullptr, RowHandle* ModifierRowOut = nullptr)
	{
		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			RowHandle MegaMeshRow = InvalidRowHandle;
			RowHandle LayerRow = InvalidRowHandle;
			RowHandle ModifierRow = InvalidRowHandle;
			if (DataStorage->HasColumns<FIsMegaMeshDefinitionLayerTag>(TargetRow))
			{
				LayerRow = TargetRow;
				FMegaMeshRowParentColumn* LayerHierarchyColumn = DataStorage->GetColumn<FMegaMeshRowParentColumn>(TargetRow);
				if (LayerHierarchyColumn)
				{
					MegaMeshRow = LayerHierarchyColumn->Parent;
				}
			}
			if (DataStorage->HasColumns<FIsMegaMeshModifierTag>(TargetRow))
			{
				ModifierRow = TargetRow;
				FMegaMeshRowParentColumn* ModifierHierarchyColumn = DataStorage->GetColumn<FMegaMeshRowParentColumn>(TargetRow);
				if (ModifierHierarchyColumn)
				{
					LayerRow = ModifierHierarchyColumn->Parent;
					FMegaMeshRowParentColumn* LayerHierarchyColumn = DataStorage->GetColumn<FMegaMeshRowParentColumn>(LayerRow);
					if (LayerHierarchyColumn)
					{
						MegaMeshRow = LayerHierarchyColumn->Parent;
					}
				}
			}

			if (!DataStorage->IsRowAvailable(MegaMeshRow))
			{
				return nullptr;
			}

			if (MegaMeshRowOut)
			{
				*MegaMeshRowOut = MegaMeshRow;
			}
			if (LayerRowOut && DataStorage->IsRowAvailable(LayerRow))
			{
				*LayerRowOut = LayerRow;
			}
			if (ModifierRowOut && DataStorage->IsRowAvailable(ModifierRow))
			{
				*ModifierRowOut = ModifierRow;
			}

			return DataStorage->GetColumn<FMegaMeshBuildToStatus>(MegaMeshRow);
		}

		return nullptr;
	}

}

//
// FMegaMeshLayerBuildWidgetHeaderConstructor
//

FMegaMeshLayerBuildWidgetHeaderConstructor::FMegaMeshLayerBuildWidgetHeaderConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshLayerBuildWidgetHeaderConstructor::CreateWidget(Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	Editor::DataStorage::RowHandle TargetRow, Editor::DataStorage::RowHandle WidgetRow, const Editor::DataStorage::FMetaDataView& Arguments)
{
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Fixed,
			.Width = 24.0f
		});

	return SNew(SImage)
		.DesiredSizeOverride(FVector2D(16.f, 16.f))
		.Image(FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildToHeader")))
		.ToolTipText(LOCTEXT("MegaMeshLayerBuildWidgetHeader", "Build To"));
}

//
// FMegaMeshLayerBuildWidget
//

FMegaMeshLayerBuildWidget::FMegaMeshLayerBuildWidget()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshLayerBuildWidget::CreateWidget(Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	Editor::DataStorage::RowHandle TargetRow, Editor::DataStorage::RowHandle WidgetRow,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SMegaMeshLayerBuildWidget, TargetRow, WidgetRow)
				.ToolTipText(LOCTEXT("MegaMeshVisibilityToggleTooltip", "Toggles the visibility."))
		];
}

//
// SMegaMeshLayerBuildWidget
//

void SMegaMeshLayerBuildWidget::Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow)
{
	TargetRow = InTargetRow;
	WidgetRow = InWidgetRow;

	SImage::Construct(
		SImage::FArguments()
		.IsEnabled(this, &SMegaMeshLayerBuildWidget::IsEnabled)
		.ColorAndOpacity(this, &SMegaMeshLayerBuildWidget::GetForegroundColor)
		.Image(this, &SMegaMeshLayerBuildWidget::GetBrush)
	);

	static const FName NAME_AutoVisibleHoveredBrush = TEXT("AutoVisibleHighlightIcon16x");
	static const FName NAME_AutoVisibleNotHoveredBrush = TEXT("AutoVisibleIcon16x");
	static const FName NAME_VisibleHoveredBrush = TEXT("VisibleHighlightIcon16x");
	static const FName NAME_VisibleNotHoveredBrush = TEXT("VisibleIcon16x");
	static const FName NAME_NotVisibleHoveredBrush = TEXT("NotVisibleHighlightIcon16x");
	static const FName NAME_NotVisibleNotHoveredBrush = TEXT("NotVisibleIcon16x");

	AutoVisibleHoveredBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(NAME_AutoVisibleHoveredBrush);
	AutoVisibleNotHoveredBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(NAME_AutoVisibleNotHoveredBrush);

	VisibleHoveredBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(NAME_VisibleHoveredBrush);
	VisibleNotHoveredBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(NAME_VisibleNotHoveredBrush);

	NotVisibleHoveredBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(NAME_NotVisibleHoveredBrush);
	NotVisibleNotHoveredBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(NAME_NotVisibleNotHoveredBrush);

	DisabledBuildToBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildTo_Disabled"));
	TargetBuildToBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildTo_Target"));
	LayerBuildToBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildTo_Layer_Targeted"));
	LayerAutoBuildToBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildTo_Layer_Auto"));
	ModifierBuildToBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildTo_Modifier_Targeted"));
	ModifierAutoBuildToBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BuildTo_Modifier_Auto"));

}

FReply SMegaMeshLayerBuildWidget::HandleClick()
{
	if (!IsEnabled())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction
	UndoTransaction.Reset(new FScopedTransaction(LOCTEXT("SetMegaMeshLayerVisibility", "Set Layer Visibility")));

	SetIsVisible(true);

	return FReply::Handled();
}

FReply SMegaMeshLayerBuildWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

/** Called when the mouse button is pressed down on this widget */
FReply SMegaMeshLayerBuildWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}

/** Process a mouse up message */
FReply SMegaMeshLayerBuildWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
void SMegaMeshLayerBuildWidget::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

/** Get the brush for this widget */
const FSlateBrush* SMegaMeshLayerBuildWidget::GetBrush() const
{
	if (IsHovered())
	{
		return TargetBuildToBrush;
	}
	else {
		BuildCondition Status = GetBuildConditionForRow();
		switch (Status)
		{
		case BuildCondition::NotBuilt:
			return DisabledBuildToBrush;
		case BuildCondition::LayerAutoBuilt:
			return LayerAutoBuildToBrush;
		case BuildCondition::LayerTargetedForBuild:
			return LayerBuildToBrush;
		case BuildCondition::ModifierTargetedForBuild:
			return ModifierBuildToBrush;
		case BuildCondition::ModifierAutoBuilt:
			return ModifierAutoBuildToBrush;
		}
	}

	return nullptr;
}

FSlateColor SMegaMeshLayerBuildWidget::GetForegroundColor() const
{
	const bool bIsHovered = IsHovered();

	if (bIsHovered)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
	}
	else
	{
		return FAppStyle::Get().GetSlateColor("Colors.Foreground");
	}
}

SMegaMeshLayerBuildWidget::BuildCondition SMegaMeshLayerBuildWidget::GetBuildConditionForRow() const
{
	using namespace Editor::DataStorage;

	BuildCondition Status = BuildCondition::NotBuilt;


	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		RowHandle MegaMeshRow = InvalidRowHandle;
		RowHandle LayerRow = InvalidRowHandle;
		RowHandle ModifierRow = InvalidRowHandle;

		TConstArrayView<const FName> Layers;

		FMegaMeshBuildToStatus* BuildToStatus = MegaMeshLayerBuildWidgetLocals::GetBuildStatus(TargetRow, &MegaMeshRow, &LayerRow, &ModifierRow);

		if (BuildToStatus)
		{
			bool bIsLayerRow = (LayerRow != InvalidRowHandle);
			bool bIsModifierRow = (ModifierRow != InvalidRowHandle);
			bool bIsTargetingALayer = !DataStorage->IsRowAvailable(BuildToStatus->ModiferToBuildTo);
			bool bIsTargetingAModifier = !bIsTargetingALayer;


			FTypedElementUObjectColumn* MegaMeshObjectCol = DataStorage->GetColumn<FTypedElementUObjectColumn>(MegaMeshRow);
			if (AMeshPartition* MegaMeshInstance = Cast<AMeshPartition>(MegaMeshObjectCol->Object); MegaMeshInstance != nullptr)
			{
				UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMeshInstance->GetMeshPartitionComponent());
				if (UMeshPartitionDefinition* Definition = EditorComponent->GetMegaMeshDefinition(); Definition)
				{
					Layers = Definition->GetModifierTypePriorities();
				}
			}

			FMegaMeshPriorityColumn* LayerPriority = nullptr;
			FMegaMeshPriorityColumn* ModifierPriority = nullptr;

			if (bIsLayerRow)
			{
				LayerPriority = DataStorage->GetColumn<FMegaMeshPriorityColumn>(LayerRow);
				if (LayerPriority)
				{
					if (bIsTargetingALayer)
					{
						Status = BuildToStatus->LayerBuildToIndex > LayerPriority->Priority ? BuildCondition::LayerAutoBuilt : BuildCondition::NotBuilt;
						Status = BuildToStatus->LayerBuildToIndex == LayerPriority->Priority ? BuildCondition::LayerTargetedForBuild : Status;
					}
					else
					{
						Status = BuildToStatus->LayerBuildToIndex > LayerPriority->Priority ? BuildCondition::LayerAutoBuilt : BuildCondition::NotBuilt;						
					}
				}
			}

			if (bIsModifierRow && bIsTargetingAModifier)
			{
				if (ModifierRow != BuildToStatus->ModiferToBuildTo)
				{

					FTypedElementUObjectColumn* TestModifierObjectCol = DataStorage->GetColumn<FTypedElementUObjectColumn>(ModifierRow);
					FTypedElementUObjectColumn* BuildToModifierObjectCol = DataStorage->GetColumn<FTypedElementUObjectColumn>(BuildToStatus->ModiferToBuildTo);

					MeshPartition::UModifierComponent* TestModifier = Cast<MeshPartition::UModifierComponent>(TestModifierObjectCol->Object);
					MeshPartition::UModifierComponent* BuildToModifier = Cast<MeshPartition::UModifierComponent>(BuildToModifierObjectCol->Object);

					Status = MeshPartition::FModifierGroup::ShouldApplyModifierBefore(Layers,
						MeshPartition::FModifierDesc(*TestModifier),
						MeshPartition::FModifierDesc(*BuildToModifier)) ? BuildCondition::ModifierAutoBuilt : BuildCondition::NotBuilt;
				}
				else
				{
					Status = BuildCondition::ModifierTargetedForBuild;
				}
			}

			if (bIsModifierRow && bIsTargetingALayer && (Status & BuildCondition::Layer))
			{
				Status = BuildCondition::ModifierAutoBuilt;
			}
		}
	}

	return Status;
}

/** Set the item this widget is responsible for to be hidden or shown */
void SMegaMeshLayerBuildWidget::SetIsVisible(const bool bVisible)
{
	// This is the same value the Builder uses... TODO: Makes sure these are kept in sync in case the builder changes
	constexpr uint32 UngroupedLayerIndex = TNumericLimits<uint32>::Max();

	using namespace Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		RowHandle MegaMeshRow = InvalidRowHandle;
		RowHandle LayerRow = InvalidRowHandle;
		RowHandle ModifierRow = InvalidRowHandle;

		FMegaMeshBuildToStatus* BuildToStatus = MegaMeshLayerBuildWidgetLocals::GetBuildStatus(TargetRow, &MegaMeshRow, &LayerRow, &ModifierRow);

		if (BuildToStatus)
		{
			BuildToStatus->LayerBuildToIndex = UngroupedLayerIndex;
			BuildToStatus->ModiferToBuildTo = InvalidRowHandle;
			if (ModifierRow != InvalidRowHandle)
			{
				BuildToStatus->ModiferToBuildTo = ModifierRow;
			}

			if (LayerRow != InvalidRowHandle)
			{
				FMegaMeshPriorityColumn* LayerPriority = DataStorage->GetColumn<FMegaMeshPriorityColumn>(LayerRow);
				if (LayerPriority)
				{
					BuildToStatus->LayerBuildToIndex = LayerPriority->Priority;
				}
			}
		}

		DataStorage->AddColumns<FTypedElementSyncBackToWorldTag>(MegaMeshRow);
	}
}

FSceneOutlinerTreeItemPtr SMegaMeshLayerBuildWidget::GetTreeItem(RowHandle InRow) const
{
	using namespace Editor::DataStorage;
	ICoreProvider* DataStorage = GetDataStorage();
	if (!DataStorage)
	{
		return nullptr;
	}

	const FSceneOutlinerColumn* TEDSOutlinerColumn = DataStorage->GetColumn<FSceneOutlinerColumn>(WidgetRow);
	if (!TEDSOutlinerColumn)
	{
		return nullptr;
	}

	TSharedPtr<ISceneOutliner> Outliner = TEDSOutlinerColumn->Outliner.IsValid() ? TEDSOutlinerColumn->Outliner.Pin() : nullptr;
	if (!Outliner.IsValid())
	{
		return nullptr;
	}

	return Editor::Outliner::Helpers::GetTreeItemFromRowHandle(DataStorage, Outliner.ToSharedRef(), InRow);
}

Editor::DataStorage::ICoreProvider* SMegaMeshLayerBuildWidget::GetDataStorage()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

Editor::DataStorage::ICoreProvider* SMegaMeshLayerBuildWidget::GetDataStorageUI()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(UiFeatureName);
}

Editor::DataStorage::ICompatibilityProvider* SMegaMeshLayerBuildWidget::GetDataStorageCompatibility()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE