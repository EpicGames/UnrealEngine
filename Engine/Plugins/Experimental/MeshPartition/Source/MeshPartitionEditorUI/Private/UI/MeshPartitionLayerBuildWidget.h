// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"

#include "UObject/StrongObjectPtr.h"
#include "Delegates/Delegate.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "SceneOutlinerFwd.h"

#include "ScopedTransaction.h"

#include "MeshPartitionLayerBuildWidget.generated.h"

namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;

USTRUCT()
struct FMegaMeshLayerBuildWidgetHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshLayerBuildWidgetHeaderConstructor();
	~FMegaMeshLayerBuildWidgetHeaderConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


USTRUCT()
struct FMegaMeshLayerBuildWidget : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshLayerBuildWidget();
	~FMegaMeshLayerBuildWidget() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


/** Widget responsible for managing the visibility for a single item */
class SMegaMeshLayerBuildWidget : public SImage
{
	using RowHandle = Editor::DataStorage::RowHandle;

	enum BuildCondition
	{
		NotBuilt = 0,
		TargetedForBuild = 1,
		AutoBuilt = 2,
		Modifier = 4,
		Layer = 8,

		ModifierTargetedForBuild = 5,
		ModifierAutoBuilt = 6,
		LayerTargetedForBuild = 9,
		LayerAutoBuilt = 10,
	};

public:
	SLATE_BEGIN_ARGS(SMegaMeshLayerBuildWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow);

protected:

	/** Returns whether the widget is enabled or not */
	virtual bool IsEnabled() const { return true; }

	/** Get the brush for this widget */
	virtual const FSlateBrush* GetBrush() const;

	FReply HandleClick();

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	/** Called when the mouse button is pressed down on this widget */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Process a mouse up message */
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	virtual FSlateColor GetForegroundColor() const;

	/** Set the item this widget is responsible for to be hidden or shown */
	void SetIsVisible(const bool bVisible);

	RowHandle TargetRow;
	RowHandle WidgetRow;

	/** Scoped undo transaction */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** Visibility brushes for the various states */
	const FSlateBrush* AutoVisibleHoveredBrush;
	const FSlateBrush* AutoVisibleNotHoveredBrush;
	const FSlateBrush* VisibleHoveredBrush;
	const FSlateBrush* VisibleNotHoveredBrush;
	const FSlateBrush* NotVisibleHoveredBrush;
	const FSlateBrush* NotVisibleNotHoveredBrush;

	const FSlateBrush* DisabledBuildToBrush;
	const FSlateBrush* TargetBuildToBrush;
	const FSlateBrush* LayerBuildToBrush;
	const FSlateBrush* LayerAutoBuildToBrush;
	const FSlateBrush* ModifierBuildToBrush;
	const FSlateBrush* ModifierAutoBuildToBrush;

private:
	BuildCondition GetBuildConditionForRow() const;

	FSceneOutlinerTreeItemPtr GetTreeItem(RowHandle InRow) const;

	static Editor::DataStorage::ICoreProvider* GetDataStorage();
	static Editor::DataStorage::ICoreProvider* GetDataStorageUI();
	static Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();
};
} // namespace UE::MeshPartition
