// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionLayerNameWidget.h"

#include "MeshPartitionEditorUIStyle.h"

#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "Layout/Visibility.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"

#include "Modifiers/MeshPartitionBooleanModifier.h"
#include "Modifiers/MeshPartitionMeshProjectModifier.h"
#include "Modifiers/MeshPartitionNoiseModifier.h"
#include "Modifiers/MeshPartitionPatchModifier.h"
#include "Modifiers/MeshPartitionInstancedPatchModifier.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"
#include "Modifiers/MeshPartitionRemeshModifier.h"
#include "Modifiers/MeshPartitionSplineModifier.h"
#include "Modifiers/MeshPartitionTexturePatchModifier.h"


#include "Columns/LayerOutlinerColumns.h"

#define LOCTEXT_NAMESPACE "MegaMeshLayerVisibilityWidget"

namespace UE::MeshPartition
{
//
// Locals
//

namespace MegaMeshNameWidgetLocals
{
	const TMap<UClass*, FName> ModifierIconMap = { 
		{MeshPartition::UBooleanModifier::StaticClass(), TEXT("ModifierBoolean")},
		//{MeshPartition::ULatticeModifier::StaticClass(), TEXT("Icons.Star")},
		{MeshPartition::UMeshProjectModifier::StaticClass(), TEXT("ModifierProjection")},
		{MeshPartition::UNoiseModifier::StaticClass(), TEXT("ModifierNoise")},
		{MeshPartition::UPatchModifier::StaticClass(), TEXT("ModifierPatch")},
		{MeshPartition::UInstancedPatchModifier::StaticClass(), TEXT("ModifierPatchInstance")},
		{MeshPartition::UProjectMeshLayersModifier::StaticClass(), TEXT("ModifierSculpt")},
		{MeshPartition::URemeshModifier::StaticClass(), TEXT("ModifierRemesh")},
		{MeshPartition::USplineModifier::StaticClass(), TEXT("ModifierPath")},
		{MeshPartition::UTexturePatchModifier::StaticClass(), TEXT("ModifierTexturePatch")},
	};

}

//
// FNameWidgetHeaderConstructor
//

FNameWidgetHeaderConstructor::FNameWidgetHeaderConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FNameWidgetHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{

	return SNew(STextBlock)
		.Text(LOCTEXT("NameWidget_Header", "Modifier"));
}

TArray<TSharedPtr<const Editor::DataStorage::FColumnSorterInterface>> FNameWidgetHeaderConstructor::ConstructColumnSorters(
	Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	TArray<TSharedPtr<const Editor::DataStorage::FColumnSorterInterface>> Sorters;
	Sorters.Add(MakeShared<FMegaMeshNameWidgetSorter>());
	return Sorters;
}

//
// FMeshPartitionTypeInfoHeaderConstructor
//

FMeshPartitionTypeInfoHeaderConstructor::FMeshPartitionTypeInfoHeaderConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMeshPartitionTypeInfoHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(STextBlock)
		.Text(LOCTEXT("TypeInfoWidget_Header", "Type"));
}

TArray<TSharedPtr<const Editor::DataStorage::FColumnSorterInterface>> FMeshPartitionTypeInfoHeaderConstructor::ConstructColumnSorters(
	Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	return {};
}

//
// FMeshPartitionTypeInfoCellConstructor
//

FMeshPartitionTypeInfoCellConstructor::FMeshPartitionTypeInfoCellConstructor()
	: Super(StaticStruct())
{
}

TArray<TSharedPtr<const Editor::DataStorage::FColumnSorterInterface>> FMeshPartitionTypeInfoCellConstructor::ConstructColumnSorters(
	Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	return {};
}

//
// FMegaMeshNameWidgetConstructorDelegator
//

FMegaMeshNameWidgetConstructorDelegator::FMegaMeshNameWidgetConstructorDelegator()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshNameWidgetConstructorDelegator::CreateWidget(Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	Editor::DataStorage::RowHandle TargetRow,
	Editor::DataStorage::RowHandle WidgetRow,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	if (DataStorage->IsRowAvailable(TargetRow))
	{
		Editor::DataStorage::IUiProvider::FWidgetConstructorPtr OutWidgetConstructorPtr;

		auto AssignWidgetToColumn = [&OutWidgetConstructorPtr](Editor::DataStorage::IUiProvider::FWidgetConstructorPtr Constructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				OutWidgetConstructorPtr = Constructor;
				return false;
			};


		// List all columns on the row, then use these columns to refine our widget matching
		TArray<TWeakObjectPtr<const UScriptStruct>> RowColumns;
		DataStorage->ListColumns(TargetRow, [&RowColumns](const UScriptStruct& ColumnType)
			{
				RowColumns.Emplace(&ColumnType);
				return true;
			});

		const Editor::DataStorage::IUiProvider::FPurposeID PurposeID = Editor::DataStorage::IUiProvider::FPurposeInfo("MegaMesh", "Outliner", "LayerName").GeneratePurposeID();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(PurposeID), Editor::DataStorage::IUiProvider::EMatchApproach::LongestMatch, RowColumns, Arguments, AssignWidgetToColumn);


		if (OutWidgetConstructorPtr)
		{
			TSharedPtr<SWidget> Widget = DataStorageUi->ConstructWidget(WidgetRow, *OutWidgetConstructorPtr, Arguments);

			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					Widget.ToSharedRef()
				];
		}
	}

	return SNew(SBox);
}

//
// FMegaMeshNameWidgetSorter
//

Editor::DataStorage::FColumnSorterInterface::ESortType FMegaMeshNameWidgetSorter::GetSortType() const
{
	return Editor::DataStorage::FColumnSorterInterface::ESortType::FixedSize64;
}

FText FMegaMeshNameWidgetSorter::GetShortName() const
{
	return LOCTEXT("FMegaMeshNameWidgetSorter", "Sorter");
}

int32 FMegaMeshNameWidgetSorter::Compare(const Editor::DataStorage::ICoreProvider& Storage, Editor::DataStorage::RowHandle Left, Editor::DataStorage::RowHandle Right) const
{
	return 0;
}

Editor::DataStorage::FPrefixInfo FMegaMeshNameWidgetSorter::CalculatePrefix(const Editor::DataStorage::ICoreProvider& Storage, Editor::DataStorage::RowHandle Row, uint32 ByteIndex) const
{
	Editor::DataStorage::FPrefixInfo Prefix;

	Prefix.bHasRemainingBytes = ByteIndex < 8;

	bool HasSortKey = Storage.HasColumns<FMegaMeshModifierSortColumn>(Row);
	if (!HasSortKey)
	{
		bool HasPriority = Storage.HasColumns<FMegaMeshPriorityColumn>(Row);
		if (!HasPriority)
		{
			Prefix.Prefix = 0;
		}
		else
		{
			uint32 Priority = Storage.GetColumn<FMegaMeshPriorityColumn>(Row)->Priority;
			Prefix.Prefix = Priority;
		}
	}
	else
	{
		uint32 Priority = Storage.GetColumn<FMegaMeshModifierSortColumn>(Row)->SortKey;
		Prefix.Prefix = Priority;
	}



	return Prefix;
}

//
// FMegaMeshLayerNameWidgetConstructor
//

FMegaMeshLayerNameWidgetConstructor::FMegaMeshLayerNameWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshLayerNameWidgetConstructor::CreateWidget(Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	Editor::DataStorage::RowHandle TargetRow,
	Editor::DataStorage::RowHandle WidgetRow,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	FTypedElementLabelColumn* LabelCol = DataStorage->GetColumn<FTypedElementLabelColumn>(TargetRow);
	FText Label = LabelCol != nullptr ? FText::FromString(LabelCol->Label) : LOCTEXT("MegaMeshLayerNameWidget_NoLabel", "NO LABEL");


	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SMegaMeshLayerNameWidget, TargetRow, WidgetRow)
				.Label(Label)
		];
}


//
// FMegaMeshModifierNameWidgetConstructor
//

FMegaMeshModifierNameWidgetConstructor::FMegaMeshModifierNameWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshModifierNameWidgetConstructor::CreateWidget(Editor::DataStorage::ICoreProvider* DataStorage,
	Editor::DataStorage::IUiProvider* DataStorageUi,
	Editor::DataStorage::RowHandle TargetRow,
	Editor::DataStorage::RowHandle WidgetRow,
	const Editor::DataStorage::FMetaDataView& Arguments)
{
	FTypedElementLabelColumn* LabelCol = DataStorage->GetColumn<FTypedElementLabelColumn>(TargetRow);
	FText Label = LabelCol != nullptr ? FText::FromString(LabelCol->Label) : LOCTEXT("MegaMeshLayerNameWidget_NoLabel", "NO LABEL");


	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SMegaMeshLayerNameWidget, TargetRow, WidgetRow)
				.Label(Label)
		];
}

//
// SMegaMeshLayerNameWidget
//

void SMegaMeshLayerNameWidget::Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow)
{
	TargetRow = InTargetRow;
	WidgetRow = InWidgetRow;

	Label = InArgs._Label;

	ChildSlot 
	[
		SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0, 0, 3, 0)
			[
				SNew(SImage)
				.Visibility(this, &SMegaMeshLayerNameWidget::GetIconVisibility)
				.Image(this, &SMegaMeshLayerNameWidget::GetBrush)					
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(STextBlock)
					.Text(Label)
			]
	];
}



EVisibility SMegaMeshLayerNameWidget::GetIconVisibility() const
{
	Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage();
	if(DataStorage)
	{
		bool bIsLayer = DataStorage->HasColumns<FIsMegaMeshDefinitionLayerTag>(TargetRow);
		if (bIsLayer)
		{
			bool bIsActive = DataStorage->HasColumns<FMegaMeshActiveLayerTag>(TargetRow);
			return bIsActive ? EVisibility::Visible : EVisibility::Hidden;
		}
		else
		{
			return EVisibility::Visible;
		}
	}

	return EVisibility::Hidden;
}

const FSlateBrush* SMegaMeshLayerNameWidget::GetBrush() const
{
	Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage();
	if (DataStorage)
	{
		bool bIsLayer = DataStorage->HasColumns<FIsMegaMeshDefinitionLayerTag>(TargetRow);
		if (bIsLayer)
		{
			static const FName ActiveIcon = TEXT("Icons.Star");
			const FSlateBrush* ActiveIconBrush = FAppStyle::Get().GetBrush(ActiveIcon);
			return ActiveIconBrush;
		}
		else
		{
			if(DataStorage->HasColumns<FTypedElementClassTypeInfoColumn>(TargetRow))
			{
				FTypedElementClassTypeInfoColumn* TypeInfo = DataStorage->GetColumn<FTypedElementClassTypeInfoColumn>(TargetRow);
				const FSlateBrush* ModifierIconBrush = FMegaMeshEditorUIStyle::Get()->GetBrush(FName(TypeInfo->TypeInfo.Get()->GetAuthoredName()));
				return ModifierIconBrush;
			}
		}
	}
	return nullptr;
}

FText SMegaMeshLayerNameWidget::GetLabel() const
{
	return LOCTEXT("MegaMeshLayerNameLabel", "Foo");
}

Editor::DataStorage::ICoreProvider* SMegaMeshLayerNameWidget::GetDataStorage()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

Editor::DataStorage::IUiProvider* SMegaMeshLayerNameWidget::GetDataStorageUI()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
}

Editor::DataStorage::ICompatibilityProvider* SMegaMeshLayerNameWidget::GetDataStorageCompatibility()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE