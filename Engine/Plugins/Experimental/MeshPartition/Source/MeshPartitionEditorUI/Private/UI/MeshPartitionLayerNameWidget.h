// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"

#include "UObject/StrongObjectPtr.h"
#include "Delegates/Delegate.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Framework/TypedElementSorter.h"
#include "Widgets/TypeInfoWidget.h"

#include "ScopedTransaction.h"

#include "MeshPartitionLayerNameWidget.generated.h"

namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;


USTRUCT()
struct FNameWidgetHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FNameWidgetHeaderConstructor();
	~FNameWidgetHeaderConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TArray<TSharedPtr<const Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};

// Header constructor for the Type column in the Mesh Partition outliner.
// Explicitly returns no sorter, so it won't interfere with the primary sort column.
USTRUCT()
struct FMeshPartitionTypeInfoHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMeshPartitionTypeInfoHeaderConstructor();
	~FMeshPartitionTypeInfoHeaderConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TArray<TSharedPtr<const Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		const Editor::DataStorage::FMetaDataView& Arguments) override;
};

// Cell constructor for the Type column in the Mesh Partition outliner.
// Inherits display behavior from FTypeInfoWidgetConstructor but returns no sorter.
USTRUCT()
struct FMeshPartitionTypeInfoCellConstructor : public FTypeInfoWidgetConstructor
{
	GENERATED_BODY()

public:
	FMeshPartitionTypeInfoCellConstructor();
	~FMeshPartitionTypeInfoCellConstructor() override = default;

	virtual TArray<TSharedPtr<const Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		const Editor::DataStorage::FMetaDataView& Arguments) override;
};

class FMegaMeshNameWidgetSorter : public Editor::DataStorage::FColumnSorterInterface
{
	virtual ESortType GetSortType() const override;
	virtual FText GetShortName() const override;
	virtual int32 Compare(const Editor::DataStorage::ICoreProvider& Storage, Editor::DataStorage::RowHandle Left, Editor::DataStorage::RowHandle Right) const override;
	virtual Editor::DataStorage::FPrefixInfo CalculatePrefix(const Editor::DataStorage::ICoreProvider& Storage, Editor::DataStorage::RowHandle Row, uint32 ByteIndex) const override;
};


USTRUCT()
struct FMegaMeshNameWidgetConstructorDelegator : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshNameWidgetConstructorDelegator();
	~FMegaMeshNameWidgetConstructorDelegator() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};

USTRUCT()
struct FMegaMeshLayerNameWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshLayerNameWidgetConstructor();
	~FMegaMeshLayerNameWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};

USTRUCT()
struct FMegaMeshModifierNameWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshModifierNameWidgetConstructor();
	~FMegaMeshModifierNameWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		Editor::DataStorage::ICoreProvider* DataStorage,
		Editor::DataStorage::IUiProvider* DataStorageUi,
		Editor::DataStorage::RowHandle TargetRow,
		Editor::DataStorage::RowHandle WidgetRow,
		const Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


/** Widget responsible for managing the visibility for a single item */
class SMegaMeshLayerNameWidget : public SCompoundWidget
{
	using RowHandle = Editor::DataStorage::RowHandle;

public:
	SLATE_BEGIN_ARGS(SMegaMeshLayerNameWidget) {}
		SLATE_ATTRIBUTE(FText, Label);
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow);

protected:

	EVisibility GetIconVisibility() const;
	const FSlateBrush* GetBrush() const;
	FText GetLabel() const;

	TAttribute<FText> Label;

	RowHandle TargetRow;
	RowHandle WidgetRow;

private:

	static Editor::DataStorage::ICoreProvider* GetDataStorage();
	static Editor::DataStorage::IUiProvider* GetDataStorageUI();
	static Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();
};
} // namespace UE::MeshPartition
