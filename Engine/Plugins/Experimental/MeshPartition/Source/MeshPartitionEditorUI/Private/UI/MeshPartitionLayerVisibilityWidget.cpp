// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPartitionLayerVisibilityWidget.h"

#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"

#include "MeshPartitionEditorUIStyle.h"

#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Text/STextBlock.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "Columns/LayerOutlinerColumns.h"
#include "Columns/SlateHeaderColumns.h"

#define LOCTEXT_NAMESPACE "MegaMeshLayerVisibilityWidget"

//
// FMegaMeshLayerBuildWidgetHeaderConstructor
//

FMegaMeshVisibilityWidgetHeaderConstructor::FMegaMeshVisibilityWidgetHeaderConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshVisibilityWidgetHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Fixed,
			.Width = 24.0f
		});

	return SNew(SImage)
		.DesiredSizeOverride(FVector2D(16.f, 16.f))
		.Image(FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BoundingBoxHeader")))
		.ToolTipText(LOCTEXT("MegaMeshVisibilityWidgetHeader", "Bounding Box Visibility"));
}

 //
// FMegaMeshVisibilityFlagWidget
//

FMegaMeshVisibilityFlagWidget::FMegaMeshVisibilityFlagWidget()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FMegaMeshVisibilityFlagWidget::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SMegaMeshVisibilityWidget, TargetRow, WidgetRow)
				.ToolTipText(LOCTEXT("MegaMeshVisibilityToggleTooltip", "Toggles the visibility."))
		];
}

//
// SMegaMeshVisibilityWidget
//

void SMegaMeshVisibilityWidget::Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow)
{
	TargetRow = InTargetRow;
	WidgetRow = InWidgetRow;

	SImage::Construct(
		SImage::FArguments()
		.IsEnabled(this, &SMegaMeshVisibilityWidget::IsEnabled)
		.ColorAndOpacity(this, &SMegaMeshVisibilityWidget::GetForegroundColor)
		.Image(this, &SMegaMeshVisibilityWidget::GetBrush)
	);


	static const FName NAME_VisibleHoveredBrush = TEXT("Level.VisibleHighlightIcon16x");
	static const FName NAME_VisibleNotHoveredBrush = TEXT("Level.VisibleIcon16x");
	static const FName NAME_NotVisibleHoveredBrush = TEXT("Level.NotVisibleHighlightIcon16x");
	static const FName NAME_NotVisibleNotHoveredBrush = TEXT("Level.NotVisibleIcon16x");

	VisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleHoveredBrush);
	VisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleNotHoveredBrush);

	NotVisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleHoveredBrush);
	NotVisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleNotHoveredBrush);

}

FReply SMegaMeshVisibilityWidget::HandleClick()
{
	if (!IsEnabled())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction
	UndoTransaction.Reset(new FScopedTransaction(LOCTEXT("SetMegaMeshLayerVisibility", "Set Layer Visibility")));

	const bool bVisible = !IsVisible();

	SetIsVisible(bVisible);

	return FReply::Handled();
}

FReply SMegaMeshVisibilityWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

/** Called when the mouse button is pressed down on this widget */
FReply SMegaMeshVisibilityWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}

/** Process a mouse up message */
FReply SMegaMeshVisibilityWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
void SMegaMeshVisibilityWidget::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

/** Get the brush for this widget */
const FSlateBrush* SMegaMeshVisibilityWidget::GetBrush() const
{
	if (IsVisible())
	{
		return IsHovered() ? VisibleHoveredBrush : VisibleNotHoveredBrush;
	}
	else
	{
		return IsHovered() ? NotVisibleHoveredBrush : NotVisibleNotHoveredBrush;
	}
}

FSlateColor SMegaMeshVisibilityWidget::GetForegroundColor() const
{
	const bool bIsHovered = IsHovered();

	if (bIsHovered)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
	}

	return FSlateColor::UseForeground();
}

/** Check if our wrapped tree item is visible */
bool SMegaMeshVisibilityWidget::IsVisible() const
{
	using namespace UE::Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		UE::MeshPartition::FMegaMeshDrawBoundsColumn* DrawBoundsColumn = DataStorage->GetColumn<UE::MeshPartition::FMegaMeshDrawBoundsColumn>(TargetRow);
		if (DrawBoundsColumn)
		{
			return DrawBoundsColumn->bEnabled;
		}
	}

	return true;
}

/** Set the item this widget is responsible for to be hidden or shown */
void SMegaMeshVisibilityWidget::SetIsVisible(const bool bVisible)
{
	using namespace UE::Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		UE::MeshPartition::FMegaMeshDrawBoundsColumn* DrawBoundsColumn = DataStorage->GetColumn<UE::MeshPartition::FMegaMeshDrawBoundsColumn>(TargetRow);
		if (DrawBoundsColumn)
		{
			DrawBoundsColumn->bEnabled = bVisible;
		}
		DataStorage->AddColumn<UE::MeshPartition::FMegaMeshUpdatedTag>(TargetRow);
	}
}

UE::Editor::DataStorage::ICoreProvider* SMegaMeshVisibilityWidget::GetDataStorage()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

UE::Editor::DataStorage::ICoreProvider* SMegaMeshVisibilityWidget::GetDataStorageUI()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(UiFeatureName);
}

UE::Editor::DataStorage::ICompatibilityProvider* SMegaMeshVisibilityWidget::GetDataStorageCompatibility()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}

#undef LOCTEXT_NAMESPACE