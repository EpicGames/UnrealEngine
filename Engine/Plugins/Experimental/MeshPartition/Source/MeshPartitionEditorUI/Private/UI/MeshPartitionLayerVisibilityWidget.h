// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"

#include "UObject/StrongObjectPtr.h"
#include "Delegates/Delegate.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "ScopedTransaction.h"

#include "MeshPartitionLayerVisibilityWidget.generated.h"

USTRUCT()
struct FMegaMeshVisibilityWidgetHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshVisibilityWidgetHeaderConstructor();
	~FMegaMeshVisibilityWidgetHeaderConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};

USTRUCT()
struct FMegaMeshVisibilityFlagWidget : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FMegaMeshVisibilityFlagWidget();
	~FMegaMeshVisibilityFlagWidget() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};


/** Widget responsible for managing the visibility for a single item */
class SMegaMeshVisibilityWidget : public SImage
{
	using RowHandle = UE::Editor::DataStorage::RowHandle;

public:
	SLATE_BEGIN_ARGS(SMegaMeshVisibilityWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	void Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow);

protected:

	/** Returns whether the widget is enabled or not */
	virtual bool IsEnabled() const { return true; }

	/** Get the brush for this widget */
	virtual const FSlateBrush* GetBrush() const;

	FReply HandleClick();

	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	/** Called when the mouse button is pressed down on this widget */
	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Process a mouse up message */
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
	virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	virtual FSlateColor GetForegroundColor() const;

	/** Check if the TargetRow Object is Visible */
	bool IsVisible() const;

	/** Set the item this widget is responsible for to be hidden or shown */
	void SetIsVisible(const bool bVisible);

	RowHandle TargetRow;
	RowHandle WidgetRow;

	/** Scoped undo transaction */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** Visibility brushes for the various states */
	const FSlateBrush* VisibleHoveredBrush;
	const FSlateBrush* VisibleNotHoveredBrush;
	const FSlateBrush* NotVisibleHoveredBrush;
	const FSlateBrush* NotVisibleNotHoveredBrush;

private:

	static UE::Editor::DataStorage::ICoreProvider* GetDataStorage();
	static UE::Editor::DataStorage::ICoreProvider* GetDataStorageUI();
	static UE::Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();
};
