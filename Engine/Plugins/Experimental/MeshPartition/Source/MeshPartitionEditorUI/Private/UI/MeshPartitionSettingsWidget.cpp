// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionSettingsWidget.h"

#include "UI/MeshPartitionLayerVisibilityWidget.h"
#include "Columns/LayerOutlinerColumns.h"
#include "MeshPartitionTedsFactory.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartition.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionEditorUIStyle.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Widgets/STedsTableViewer.h"
#include "Widgets/Input/SButton.h"
#include "TedsRowViewNode.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "DataStorage/Features.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"

#include "MeshPartitionDefinition.h"

#include "Widgets/Layout/SScaleBox.h"

#include "Algo/Compare.h"

#include "SceneOutlinerPublicTypes.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementQueryContext.h"

#include "TedsHierarchyNode.h"
#include "TedsRowArrayNode.h"
#include "TedsQueryNode.h"
#include "TedsRowQueryResultsNode.h"
#include "TedsRowMergeNode.h"

#include "Modules/ModuleManager.h"

#include "TedsOutliner/Public/Compatibility/SceneOutlinerTedsBridge.h"
#include "TedsOutliner/Public/Compatibility/SceneOutlinerRowHandleColumn.h"
#include "TedsOutliner/Public/TedsOutlinerImpl.h"
#include "TedsOutliner/Public/TedsOutlinerMode.h"
#include "TedsOutlinerItem.h"
#include "TedsOutlinerFilter.h"

#include "Widgets/STedsHierarchyViewer.h"
#include "Widgets/Composite/STedsCompositeHierarchyViewer.h"

#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "LevelEditor.h"

#include "PropertyEditorModule.h"

#include "Editor/EditorEngine.h"
#include "Styling/ToolBarStyle.h"

#include "Selection.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Columns/LayerOutlinerColumns.h"


#define LOCTEXT_NAMESPACE "MegaMeshSettingsWidget"

static FAutoConsoleVariable CVarMeshPartitionOutlinerEnableTedsHierarchyWidget(
	TEXT("MeshPartitionOutliner.EnableTedsHierarchyWidget"),
	true,
	TEXT("Enables new Teds Hierarchy Widget instead of the Teds Outliner Widget in the Mesh Partition Outliner"),
	ECVF_Default
);

#if WITH_EDITOR
void UFilterSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	OnModified.Broadcast(this, PropertyChangedEvent.Property);
}
#endif

namespace UE::MeshPartition
{



SLATE_IMPLEMENT_WIDGET(SMegaMeshSettingsWidget)
void SMegaMeshSettingsWidget::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{

}

SMegaMeshSettingsWidget::SMegaMeshSettingsWidget()
{
	USelection::SelectionChangedEvent.AddRaw(this, &SMegaMeshSettingsWidget::OnLevelSelectionChanged);
	USelection::SelectObjectEvent.AddRaw(this, &SMegaMeshSettingsWidget::OnLevelSelectionChanged);
}

SMegaMeshSettingsWidget::~SMegaMeshSettingsWidget()
{
	USelection::SelectionChangedEvent.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);
}

namespace SMegaMeshSettingsWidgetLocals
{
	TSharedRef<IDetailsView> MakeDetailsView()
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::ObjectsUseNameArea;
		DetailsViewArgs.bShowObjectLabel = true;
		DetailsViewArgs.bAllowSearch = true;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bShowCustomFilterOption = true;
		DetailsViewArgs.bShowSectionSelector = false;
		DetailsViewArgs.bAllowFavoriteSystem = true;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
		DetailsViewArgs.bShowPropertyMatrixButton = true;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.InitialSections.Add(FName(TEXT("MeshPartition")));
		

		TSharedRef<IDetailsView> DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);
		
		return DetailsView;
	}

	TSharedRef<IDetailsView> MakeFilterDetailsView()
	{
		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		FDetailsViewArgs DetailsViewArgs;
		DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bShowCustomFilterOption = false;
		DetailsViewArgs.bShowSectionSelector = false;
		DetailsViewArgs.bAllowFavoriteSystem = false;
		DetailsViewArgs.bAllowMultipleTopLevelObjects = false;
		DetailsViewArgs.bShowPropertyMatrixButton = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bUpdatesFromSelection = false;		


		TSharedRef<IDetailsView> DetailsView = PropertyEditor.CreateDetailView(DetailsViewArgs);

		return DetailsView;
	}

	UActorComponent* GetModifierFromOutlinerItem(ISceneOutlinerTreeItem& TreeItemRef)
	{
		using namespace Editor::Outliner;
		using namespace Editor::DataStorage;

		if (FTedsOutlinerTreeItem* TEDSItem = TreeItemRef.CastTo<FTedsOutlinerTreeItem>())
		{
			ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			if (TEDSItem->IsValid() && Storage)
			{
				const FTypedElementUObjectColumn* RawObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(TEDSItem->GetRowHandle());
				return RawObjectColumn ? Cast<UActorComponent>(RawObjectColumn->Object) : nullptr;
			}
		}

		return nullptr;
	}

	FText GenerateDefinitionPathString(UMeshPartitionDefinition* Definition)
	{
		if (Definition)
		{
			FString Path = Definition->GetPathName();
			TArray<FString> PathParts;
			Path.ParseIntoArray(PathParts, TEXT("/"));
			FString TruncatedPath = "..";
			for (int Index = FMath::Max(0, PathParts.Num() - 2); Index < PathParts.Num(); Index++)
			{
				if (Index == PathParts.Num() - 1)
				{
					FString AssetName;
					PathParts[Index].Split(".", &AssetName, nullptr);
					TruncatedPath.PathAppend(*AssetName, AssetName.Len());
				}
				else
				{
					TruncatedPath.PathAppend(*PathParts[Index], PathParts[Index].Len());
				}
			}
			return FText::FromString(TruncatedPath);
		}
		else
		{
			return LOCTEXT("MegaMeshSettingsWidget_NoDefinition", "No Definition Assigned");
		}
	}
}

void SMegaMeshSettingsWidget::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(FilterSettings);
}

void SMegaMeshSettingsWidget::Construct(const FArguments& InArgs)
{
	static const FName NAME_VisibleHoveredBrush = TEXT("Level.VisibleHighlightIcon16x");



	FSlimHorizontalToolBarBuilder Toolbar(nullptr, FMultiBoxCustomization::None);
	Toolbar.SetStyle(&FMegaMeshEditorUIStyle::Get().Get(), "OutlinerToolbar");

	PreviousBoundsFilterActor = nullptr;
	FilterSettings = NewObject<UFilterSettings>();	
	FilterSettings->GetOnModified().AddSP(this, &SMegaMeshSettingsWidget::OnFilterSettingsModified);
	DetailsView = SMegaMeshSettingsWidgetLocals::MakeDetailsView();
	FilterDetailsView = SMegaMeshSettingsWidgetLocals::MakeFilterDetailsView();

	if (FilterSettings)
	{
		FilterDetailsView->SetObject(FilterSettings);
	}

	Toolbar.AddComboButton(
		FUIAction(),
		FOnGetContent::CreateLambda(
			[this]()
			{
				FMenuBuilder Menu(true, nullptr);

				Menu.AddMenuEntry(LOCTEXT("SMegaMeshSettingsWidget_FindInContentBrowser", "Find in Content Browser"),
					FText::GetEmpty(),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.TabIcon"),
					FUIAction(
						FExecuteAction::CreateRaw(this, &SMegaMeshSettingsWidget::OpenDefinitionInContentBrowser),
						FCanExecuteAction::CreateRaw(this, &SMegaMeshSettingsWidget::CanOpenDefinitionInContentBrowser)
					));

				return Menu.MakeWidget();
			}),
		TAttribute<FText>(),
		TAttribute<FText>(),
		TAttribute<FSlateIcon>(),
		true
	);

	OnMegaMeshSelectorOpening();

	bool bUseHierarchyViewer = CVarMeshPartitionOutlinerEnableTedsHierarchyWidget->GetBool();

	TSharedPtr<SWidget> OutlinerWidget;
	using namespace Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		if (!AreEditorDataStorageFeaturesEnabled())
		{
			OutlinerWidget = SNew(STextBlock)
				.Text(LOCTEXT("TEDSPluginNotEnabledText", "You need to enable the Typed Element Data Storage plugin to see the table viewer!"));
		}
		else
		{
			if (!bUseHierarchyViewer)
			{
				OutlinerWidget = Outliner = CreateOutlinerWidget().ToSharedPtr();
			}
			else
			{
				OutlinerWidget = HierarchyViewer = CreateHierarchyWidget().ToSharedPtr();
			}
		}
	}

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FMegaMeshEditorUIStyle::Get()->GetBrush(TEXT("BlackBorderFillBrush")))
		[
			SNew(SVerticalBox)

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.Padding(5,2,5,7)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					[
						SAssignNew(MegaMeshSelector, SComboBox<TSharedPtr<FMegaMeshEntry>>)
						.OptionsSource(&MegaMeshList)
						.InitiallySelectedItem(ActiveMegaMesh)
						.OnSelectionChanged(this, &SMegaMeshSettingsWidget::OnComboBoxSelectionChanged)
						.OnGenerateWidget(this, &SMegaMeshSettingsWidget::GenerateMegaMeshEntry)
						.OnComboBoxOpening(this, &SMegaMeshSettingsWidget::OnMegaMeshSelectorOpening)
						.ToolTipText(LOCTEXT("SMegaMeshSettingsWidget_SelectMegaMeshTooltip", "Select which Mesh Partition Actor and its modifiers to view."))
						[
							SNew(STextBlock)
							.Text(this, &SMegaMeshSettingsWidget::GetActiveMegaMeshLabel)
						]
						
						
					]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					[						
						SNew(SButton)
						.OnClicked(this, &SMegaMeshSettingsWidget::OnMegaMeshSelectClicked)
						.ButtonStyle(&FMegaMeshEditorUIStyle::Get()->GetWidgetStyle<FButtonStyle>("OutlinerButton"))
						.IsEnabled(this, &SMegaMeshSettingsWidget::IsActiveMegaMesh)
						.ToolTipText(LOCTEXT("SMegaMeshSettingsWidget_SelectInEditorMegaMeshButtonTooltip","Select this Mesh Partition in Editor."))
						[
							SNew(SBox)
							.HAlign(EHorizontalAlignment::HAlign_Center)
							.VAlign(EVerticalAlignment::VAlign_Center)
							[
								SNew(SImage)							
								.Image(FAppStyle::GetBrush("Icons.SelectInViewport"))								
							]
						]
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0)
					.HAlign(EHorizontalAlignment::HAlign_Center)
					[
						SNew(SBox)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Right)
					[
						SNew(SBorder)
						.BorderImage(FStyleDefaults::GetNoBrush())
						.VAlign(EVerticalAlignment::VAlign_Center)
						.Padding(0,0,10,0)
						[
							SAssignNew(DefinitionPath, STextBlock)
							.Text(this, &SMegaMeshSettingsWidget::GetActiveDefinitionLabel)
							.Justification(ETextJustify::InvariantLeft)
							.ToolTipText(LOCTEXT("SMegaMeshSettingsWidget_DefinitionTooltip", "Currently assigned Mesh Partition definition asset, if available."))
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					.HAlign(EHorizontalAlignment::HAlign_Left)
					[
						Toolbar.MakeWidget()
					]
				]
			]

			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
					.Visibility_Lambda([bUseHierarchyViewer]() -> EVisibility
					{
						return bUseHierarchyViewer ? EVisibility::Visible : EVisibility::Collapsed;
					})
				[
					FilterDetailsView.ToSharedRef()
				]
			]
			+SVerticalBox::Slot()
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Vertical)

				+SSplitter::Slot()
				[
					OutlinerWidget.IsValid() ? OutlinerWidget.ToSharedRef() : SNew(SBorder)
				]
				+ SSplitter::Slot()
				[
					SNew(SBox)
					.Padding(0,5)
					[
						DetailsView.ToSharedRef()
					]
				]
			]
		]

	];
}

void SMegaMeshSettingsWidget::OnFilterSettingsModified(UObject*, FProperty*)
{
	using namespace Editor::DataStorage;
	using namespace Editor::Outliner;

	if (FilterSettings)
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName); Storage != nullptr)
		{
			if (PreviousBoundsFilterActor.IsValid())
			{
				RowHandle Row = Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(PreviousBoundsFilterActor.Get()));
				if (Row != InvalidRowHandle)
				{
					Storage->RemoveColumn<FMegaMeshBoundsFilterSourceTag>(Row);
				}
			}

			if (FilterSettings->BoundsFilterActor.LoadSynchronous())
			{
				RowHandle Row = Storage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(FilterSettings->BoundsFilterActor.Get()));
				if (Row != InvalidRowHandle)
				{
					Storage->AddColumn<FMegaMeshBoundsFilterSourceTag>(Row);
				}
				PreviousBoundsFilterActor = FilterSettings->BoundsFilterActor.LoadSynchronous();
			}
		}
	}
}

void SMegaMeshSettingsWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	Super::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);


	if (!ActiveMegaMesh)
	{
		RebuildMegaMeshSelectionList();
		if (MegaMeshList.Num() > 0)
		{
			ActiveMegaMesh = MegaMeshList[0];
			MegaMeshSelector->SetSelectedItem(ActiveMegaMesh);
		}
	}

	if (ActiveMegaMesh)
	{
		if (ActiveMegaMesh->MegaMesh.IsValid())
		{
			ActiveMegaMesh->Definition = ActiveMegaMesh->MegaMesh.Pin()->GetMeshPartitionDefinition();
		}
		else
		{
			ActiveMegaMesh = nullptr;
		}
	}
	
}


TSharedRef<SSceneOutliner> SMegaMeshSettingsWidget::CreateOutliner(const FSceneOutlinerInitializationOptions& InInitOptions, const Editor::Outliner::FTedsOutlinerParams& InInitTedsOptions) const
{
	using namespace Editor::DataStorage;
	using namespace Editor::Outliner;

	ensureMsgf(AreEditorDataStorageFeaturesEnabled(), TEXT("Unable to initialize the Teds-Outliner before TEDS itself is initialized."));

	FSceneOutlinerInitializationOptions InitOptions(InInitOptions);
	FTedsOutlinerParams InitTedsOptions(InInitTedsOptions);

	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([&InitTedsOptions](SSceneOutliner* Outliner)
		{
			InitTedsOptions.SceneOutliner = Outliner;

			return new FTedsOutlinerMode(InitTedsOptions);
		});
#if 0
	// Add the custom column that displays row handles
	if (InInitTedsOptions.bShowRowHandleColumn)
	{
		InitOptions.ColumnMap.Add(FSceneOutlinerRowHandleColumn::GetID(),
			FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 2,
				FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner)
					{
						return MakeShareable(new FSceneOutlinerRowHandleColumn(InSceneOutliner));
					})));
	}
#endif

	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10));
	
	TSharedRef<SSceneOutliner> TedsOutlinerShared = SNew(SSceneOutliner, InitOptions);
	return TedsOutlinerShared;
}

TSharedRef<SSceneOutliner> SMegaMeshSettingsWidget::CreateOutlinerWidget()
{
	using namespace Editor;

	using namespace DataStorage::Queries;
	using namespace Editor::Outliner;
 
	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	// The Outliner is populated with Actors and Entities
	DataStorage::FQueryDescription OutlinerQueryDescription =
		Select()
		.Where( ( (TColumn<FIsMegaMeshModifierTag>() && TColumn<FMegaMeshNotHiddenInOutlinerTag>()) || TColumn<FIsMegaMeshDefinitionLayerTag>()) && TColumn<FIsMegaMeshActiveInOutlinerTag>() )
		.Compile();

	FMetaData GenericMetaData = FMetaData();
	GenericMetaData.AddOrSetMutableData("bAllowSorting", false);
	GenericMetaData.AddOrSetMutableData("bColumnInitialSortIsAscending", false);
	
	const FTedsOutlinerColumnDescription ColumnDescription ({
		FTypedElementLabelColumn::StaticStruct(),
		FTypedElementClassTypeInfoColumn::StaticStruct(),
		FMegaMeshDrawBoundsColumn::StaticStruct(),
		FMegaMeshBuildUpToThisLayerColumn::StaticStruct(),
		FMegaMeshModifierTiming::StaticStruct(),
		FParentActorRefColumn::StaticStruct()
	}, GenericMetaData);

	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowHeaderRow = true;
	InitOptions.FilterBarOptions.bHasFilterBar = true;
	InitOptions.bShowTransient = true;
	InitOptions.OutlinerIdentifier = "MegaMeshOutliner";
	InitOptions.PrimaryColumnName = FName(FTypedElementLabelColumn::StaticStruct()->GetDisplayNameText().ToString());


	FTedsOutlinerParams Params(nullptr);
	Params.QueryDescription = OutlinerQueryDescription;
	Params.bShowRowHandleColumn = false;
	Params.ColumnDescription = ColumnDescription;
	Params.CellWidgetPurpose = IUiProvider::FPurposeInfo(TEXT("MegaMesh"), TEXT("Outliner"), NAME_None).GeneratePurposeID();
	Params.LabelWidgetPurpose = IUiProvider::FPurposeInfo(TEXT("MegaMesh"), TEXT("Outliner"), TEXT("LayerName")).GeneratePurposeID();
	Params.HeaderWidgetPurpose = IUiProvider::FPurposeInfo(TEXT("MegaMesh"), TEXT("OutlinerHeader"), NAME_None).GeneratePurposeID();

	const FTedsOutlinerHierarchyData::FGetParentRowHandle RowHandleGetter = FTedsOutlinerHierarchyData::FGetParentRowHandle::CreateLambda([](const void* InColumnData)
		{
			if (const FTableRowParentColumn* ParentColumn = static_cast<const FTableRowParentColumn*>(InColumnData))
			{
				return ParentColumn->Parent;
			}
			return InvalidRowHandle;
		});

	const FTedsOutlinerHierarchyData::FGetChildrenRowsHandles ChildrenRowHandlesGetter = FTedsOutlinerHierarchyData::FGetChildrenRowsHandles::CreateLambda([](void* InColumnData)
		{
			struct MegaMeshOutlinerSorter
			{
				MegaMeshOutlinerSorter(TMap<RowHandle, int32> OrderingMap)
					: Ordering(OrderingMap)
				{}

				bool operator()(const RowHandle& A, const RowHandle& B) const
				{
					return Ordering[A] < Ordering[B];
				}

				TMap<RowHandle, int32> Ordering;
			};


			if (FMegaMeshRowParentColumn* ParentColumn = static_cast<FMegaMeshRowParentColumn*>(InColumnData))
			{
				ParentColumn->Children.Reset(ParentColumn->ChildrenSet.Num());
				for (TTuple<RowHandle, int32> Handle : ParentColumn->ChildrenSet)
				{
					ParentColumn->Children.Emplace(Handle.Key);
				}
				ParentColumn->Children.Sort<MegaMeshOutlinerSorter>(MegaMeshOutlinerSorter(ParentColumn->ChildrenSet));
				return TArrayView<RowHandle>(ParentColumn->Children);
			}

			return TArrayView<RowHandle>();
		});

	const FTedsOutlinerHierarchyData::FSetParentRowHandle RowHandleSetter = FTedsOutlinerHierarchyData::FSetParentRowHandle::CreateLambda([](void* InColumnData,
		RowHandle InRowHandle)
		{
			if (FMegaMeshRowParentColumn* ParentColumn = static_cast<FMegaMeshRowParentColumn*>(InColumnData))
			{
				ParentColumn->Parent = InRowHandle;
			}
		});

	Params.HierarchyData = MakeShared<FTedsOutlinerLegacyHierarchyInterface>(FTedsOutlinerHierarchyData(FMegaMeshRowParentColumn::StaticStruct(), RowHandleGetter, RowHandleSetter, ChildrenRowHandlesGetter));

	const FLevelEditorModule* const LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(TEXT("LevelEditor"));
	if (LevelEditorModule)
	{
		if (auto LevelEditor = LevelEditorModule->GetLevelEditorInstance().Pin())
		{
			Params.SelectionSet = LevelEditor->GetMutableElementSelectionSet();
		}
	}

	TSharedRef<SSceneOutliner> TEDSOutlinerShared = CreateOutliner(InitOptions, Params);

	DataStorage::RowHandle OutlinerRow = Storage->LookupMappedRow(MappingDomain, DataStorage::FMapKey(TEDSOutlinerShared->GetOutlinerIdentifier()));

	if (FTedsOutlinerSelectionChangeColumn* SelectionChangeColumn = Storage->GetColumn<FTedsOutlinerSelectionChangeColumn>(OutlinerRow))
	{
		if (!SelectionChangeColumn->OnSelectionChanged)
		{
			SelectionChangeColumn->OnSelectionChanged = MakeShared<UE::Editor::Outliner::FOnTedsOutlinerSelectionChanged>();
		}
		SelectionChangeColumn->OnSelectionChanged->AddLambda([this](ESelectInfo::Type SelectionType)
		{
			OnOutlinerSelectionChanged(SelectionType);
		});
	}

	return TEDSOutlinerShared;
}

TSharedRef<UE::Editor::DataStorage::STedsCompositeHierarchyViewer> SMegaMeshSettingsWidget::CreateHierarchyWidget()
{

	using namespace Editor;

	using namespace DataStorage::Queries;
	using namespace Editor::Outliner;

	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	DataStorage::FQueryDescription OutlinerQueryDescription =
		Select()
		.Where(((TColumn<FIsMegaMeshModifierTag>() && TColumn<FMegaMeshNotHiddenInOutlinerTag>()) || TColumn<FIsMegaMeshDefinitionLayerTag>()) && TColumn<FIsMegaMeshActiveInOutlinerTag>())
		.Compile();

	TSharedRef<QueryStack::FQueryNode> ReferenceQueryNode =
		MakeShared<QueryStack::FQueryNode>(*Storage, OutlinerQueryDescription);
	TSharedPtr<QueryStack::IRowNode> FinalNode = nullptr;

	if (ReferenceQueryNode->GetQuery() != InvalidQueryHandle)
	{
		FinalNode =
			MakeShared<QueryStack::FRowQueryResultsNode>(*Storage,
				ReferenceQueryNode, QueryStack::FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate);
	}
	else
	{
		FinalNode = MakeShared<QueryStack::FRowArrayNode>(FRowHandleArray());
	}
	UMegaMeshTedsFactory* Factory = Storage->FindFactory<UMegaMeshTedsFactory>();
	TSharedPtr<FHierarchyViewerData> HierarchyData = MakeShared<FHierarchyViewerData>(*Storage, Factory->MegaMeshHierarchy);
	IUiProvider::FPurposeID CellWidgetPurpose = IUiProvider::FPurposeInfo(TEXT("MegaMesh"), TEXT("Outliner"), NAME_None).GeneratePurposeID();
	IUiProvider::FPurposeID HeaderWidgetPurpose = IUiProvider::FPurposeInfo(TEXT("MegaMesh"), TEXT("OutlinerHeader"), NAME_None).GeneratePurposeID();
	TSharedPtr<FMetaData> GenericMetaData = MakeShared<FMetaData>();
	GenericMetaData->AddOrSetMutableData("bAllowSorting", false);
	GenericMetaData->AddOrSetMutableData("bColumnInitialSortIsAscending", false);

	TSharedPtr<FColumnMetaData> TypeInfoColumnMetaData = MakeShared<FColumnMetaData>(FTypedElementClassTypeInfoColumn::StaticStruct(), FColumnMetaData::EFlags::None);
	TypeInfoColumnMetaData->AddOrSetMutableData("bColumnDisableSorting", true);

	TSharedPtr<FFilterCategory> MeshPartitionFilterCategory = MakeShared<FFilterCategory>(
		LOCTEXT("MeshPartitionFilterCategory","Mesh Partition"),
		LOCTEXT("MeshPartitionFilterCategory_Tooltip","Filters related to the Mesh Partition")
	);
	
	TArray<TSharedPtr<DataStorage::FTedsFilter>> Filters;

	{
		TConstQueryFunction<bool> FilterQuery =
			BuildConstQueryFunction<bool>([](TConstQueryContext<CurrentTableInfo> Context)
			{
				if (Context.CurrentTableHasColumns<FMegaMeshAffectsFilterBounds>())
				{
					return true;
				}
				return false;
			});

		Filters.Add(MakeShared<DataStorage::FTedsFilter>(
			FName(TEXT("ActorBoundsFilter")),
			LOCTEXT("ActorBoundsFilter", "Filter By Actor Bounds"),
			LOCTEXT("ActorBoundsFilter_Tooltip", "Use the selected Actor's bonuds to filter only those modifiers affecting that region."),
			FName(),
			MeshPartitionFilterCategory,
			MoveTemp(FilterQuery)
			));
	}


	TSharedPtr<STedsCompositeHierarchyViewer> ReferenceView = SNew(STedsCompositeHierarchyViewer, HierarchyData)
		.HierarchyViewerArgs(SHierarchyViewer::FArguments()	
			.AllNodeProvider(FinalNode)
			.Columns({
				FTypedElementLabelColumn::StaticStruct(),
				FTypedElementClassTypeInfoColumn::StaticStruct(),
				FMegaMeshDrawBoundsColumn::StaticStruct(),
				FMegaMeshBuildUpToThisLayerColumn::StaticStruct(),
				FMegaMeshModifierTiming::StaticStruct(),
				FParentActorRefColumn::StaticStruct()
				})
			.GenericMetaData(GenericMetaData)
			.ColumnMetaData({TypeInfoColumnMetaData})
			.CellWidgetPurpose(CellWidgetPurpose)
			.HeaderWidgetPurpose(HeaderWidgetPurpose)
			.PrimaryColumn(FTypedElementLabelColumn::StaticStruct())
			.OnSelectionChanged_Lambda([this](RowHandle Row, ESelectInfo::Type SelectInfo)
				{
					OnOutlinerSelectionChanged(SelectInfo);
				})
			.EmptyRowsMessage(LOCTEXT("MegaMeshSettings_NoRows", "No modifiers found for this Mesh Partition instance.")))
			.Filters(Filters)
		;

	ReferenceView->ExpandAll();

	return ReferenceView.ToSharedRef();
}

void SMegaMeshSettingsWidget::OnComboBoxSelectionChanged(TSharedPtr<FMegaMeshEntry> InNewSelection, ESelectInfo::Type SelectInfo)
{
	using namespace Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		if (ActiveMegaMesh && InNewSelection != ActiveMegaMesh)
		{
			DataStorage->RemoveColumns<FIsMegaMeshActiveInOutlinerTag>(ActiveMegaMesh->MegaMeshRow);
		}

		if (InNewSelection)
		{
			ActiveMegaMesh = InNewSelection;
			DataStorage->AddColumns<FIsMegaMeshActiveInOutlinerTag>(ActiveMegaMesh->MegaMeshRow);

			if (ActiveMegaMesh->MegaMesh.IsValid())
			{
				ActiveMegaMesh->Definition = ActiveMegaMesh->MegaMesh.Pin()->GetMeshPartitionDefinition();
			}

			return;
		}
	}

	DefinitionPath->SetText(LOCTEXT("MegaMeshSettingsWidget_NoMegaMesh", "No Mesh Partition Selected"));
	ActiveMegaMesh.Reset();
}

TSharedRef<SWidget> SMegaMeshSettingsWidget::GenerateMegaMeshEntry(TSharedPtr<FMegaMeshEntry> InItem)
{
	if (InItem)
	{
		return SNew(STextBlock)
			.Text(FText::FromString(InItem->MegaMeshLabel));
	}
	else
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MegaMeshSettingsWidget_InvalidMegaMeshRef", "Invalid Mesh Partition Reference"));
	}
}

void SMegaMeshSettingsWidget::RebuildMegaMeshSelectionList()
{
	TArray<TSharedPtr<FMegaMeshEntry>> MegaMeshListTemp;

	using namespace Editor::DataStorage;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		if (UMegaMeshTedsFactory* Factory = DataStorage->FindFactory<UMegaMeshTedsFactory>(); Factory)
		{
			DataStorage->RunQuery(Factory->MegaMeshQuery, Queries::CreateDirectQueryCallbackBinding(
				[this, &MegaMeshListTemp](IDirectQueryContext& Context, const RowHandle Row, const FTypedElementUObjectColumn& ObjectRef)
				{
					TSharedPtr<FMegaMeshEntry> Entry = MakeShared<FMegaMeshEntry>();
					Entry->MegaMesh = Cast<AMeshPartition>(ObjectRef.Object);
					Entry->MegaMeshRow = Row;
					if (Entry->MegaMesh.IsValid())
					{
						Entry->MegaMeshLabel = Entry->MegaMesh.Pin()->GetName();
					}
					MegaMeshListTemp.Add(Entry);

				}));
		}
	}

	MegaMeshListTemp.Sort([](const TSharedPtr<FMegaMeshEntry>& A, const TSharedPtr<FMegaMeshEntry>& B)
		{
			if (A.IsValid() && B.IsValid())
			{
				return A->MegaMeshLabel.Compare(B->MegaMeshLabel) < 0;
			}
			else
			{
				return A.IsValid() ? true : false;
			}
		});

	if (!Algo::Compare(MegaMeshListTemp, MegaMeshList, [](const TSharedPtr<FMegaMeshEntry>& A, const TSharedPtr<FMegaMeshEntry>& B)
		{
			if (A.IsValid() && B.IsValid())
			{
				return A->MegaMeshRow == B->MegaMeshRow;
			}
			else
			{
				return false;
			}
		}))
	{
		RowHandle LastSelectedMegaMeshRow = InvalidRowHandle;
		if (ActiveMegaMesh)
		{
			LastSelectedMegaMeshRow = ActiveMegaMesh->MegaMeshRow;
		}

		MegaMeshList = MoveTemp(MegaMeshListTemp);

		if (MegaMeshSelector)
		{
			MegaMeshSelector->RefreshOptions();
			ActiveMegaMesh.Reset();
			if (LastSelectedMegaMeshRow != InvalidRowHandle)
			{
				TSharedPtr<FMegaMeshEntry>* Entry = MegaMeshList.FindByPredicate([LastSelectedMegaMeshRow](const TSharedPtr<FMegaMeshEntry>& Entry)
					{
						return Entry->MegaMeshRow == LastSelectedMegaMeshRow;
					});
				if (Entry)
				{
					ActiveMegaMesh = *Entry;
				}

			}
			MegaMeshSelector->SetSelectedItem(ActiveMegaMesh);
		}

	}
}

void SMegaMeshSettingsWidget::OnMegaMeshSelectorOpening()
{
	RebuildMegaMeshSelectionList();
}

FText SMegaMeshSettingsWidget::GetActiveMegaMeshLabel() const
{
	if (ActiveMegaMesh)
	{
		return FText::FromString(ActiveMegaMesh->MegaMeshLabel);
	}
	else
	{
		return LOCTEXT("MegaMeshSettingsWidget_NoMegaMeshSelected", "No Mesh Partition Selected");
	}

}
FText SMegaMeshSettingsWidget::GetActiveDefinitionLabel() const
{
	if (ActiveMegaMesh)
	{
		return SMegaMeshSettingsWidgetLocals::GenerateDefinitionPathString(ActiveMegaMesh->Definition.IsValid() ? ActiveMegaMesh->Definition.Get() : nullptr);
	}
	else
	{
		return LOCTEXT("MegaMeshSettingsWidget_NoMegaMesh", "No Mesh Partition Selected");
	}
}

void SMegaMeshSettingsWidget::OnOutlinerSelectionChanged(ESelectInfo::Type SelectInfo)
{
	UpdateDetailsViewFromOutliner();
	if (SelectInfo != ESelectInfo::Direct)
	{
		UpdateLevelSelectionFromOutliner();
	}
}

void SMegaMeshSettingsWidget::UpdateDetailsViewFromOutliner()
{
	TArray<UObject*> Modifiers;

	if (Outliner.IsValid())
	{
		TArray<FSceneOutlinerTreeItemPtr> SelectedItems = Outliner->GetSelectedItems();
		Modifiers.Reserve(SelectedItems.Num());
		for (FSceneOutlinerTreeItemPtr& SelectedItem : SelectedItems)
		{
			UActorComponent* Modifier = SMegaMeshSettingsWidgetLocals::GetModifierFromOutlinerItem(*SelectedItem);
			if (Modifier)
			{
				Modifiers.Add(Modifier);
			}
		}


		if (Modifiers.Num() > 0)
		{
			DetailsView->SetObjects(Modifiers);
			return;
		}
	}

	if (HierarchyViewer)
	{
		using namespace Editor::DataStorage;
		ICoreProvider* DataStorage = GetDataStorage();
		if (!DataStorage)
		{
			return;
		}

		HierarchyViewer->ForEachSelectedRow([this, DataStorage, &Modifiers](RowHandle SelectedRow)
			{
				const FTypedElementUObjectColumn* RawObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(SelectedRow);
				UActorComponent* Modifier = RawObjectColumn ? Cast<UActorComponent>(RawObjectColumn->Object) : nullptr;

				if(Modifier != nullptr)
				{
					Modifiers.Add(Modifier);
				}

			});
			
			
		if (Modifiers.Num() > 0)
		{
			DetailsView->SetObjects(Modifiers);
			return;
		}

	}

	DetailsView->SetObject(nullptr);
}

void SMegaMeshSettingsWidget::UpdateLevelSelectionFromOutliner()
{
	TArray<UObject*> Modifiers;

	if (Outliner.IsValid())
	{
		TArray<FSceneOutlinerTreeItemPtr> SelectedItems = Outliner->GetSelectedItems();
		Modifiers.Reserve(SelectedItems.Num());
		for (FSceneOutlinerTreeItemPtr& SelectedItem : SelectedItems)
		{
			UActorComponent* Modifier = SMegaMeshSettingsWidgetLocals::GetModifierFromOutlinerItem(*SelectedItem);
			if (Modifier)
			{
				Modifiers.Add(Modifier);
			}
		}

		if (Modifiers.Num() > 0)
		{
			for (UObject* Modifier : Modifiers)
			{
				GEditor->SelectComponent(Cast<UActorComponent>(Modifier), true, true);
			}
		}
	}

	if (HierarchyViewer)
	{
		using namespace Editor::DataStorage;
		ICoreProvider* DataStorage = GetDataStorage();
		if (!DataStorage)
		{
			return;
		}

		HierarchyViewer->ForEachSelectedRow([this, DataStorage, &Modifiers](RowHandle SelectedRow)
			{
				const FTypedElementUObjectColumn* RawObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(SelectedRow);
				UActorComponent* Modifier = RawObjectColumn ? Cast<UActorComponent>(RawObjectColumn->Object) : nullptr;

				if (Modifier != nullptr)
				{
					Modifiers.Add(Modifier);
				}

			});


		if (Modifiers.Num() > 0)
		{
			GEditor->SelectNone(true, true);
			for (UObject* Modifier : Modifiers)
			{
				GEditor->SelectComponent(Cast<UActorComponent>(Modifier), true, true);
			}
		}

	}
}


void SMegaMeshSettingsWidget::OnLevelSelectionChanged(UObject* InObject)
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage = GetDataStorage();

	if (Outliner.IsValid())
	{
		if (DataStorage)
		{
			UMegaMeshTedsFactory* Factory = DataStorage->FindFactory<UMegaMeshTedsFactory>();
			if (!Factory)
			{
				return;
			}

			TArray<RowHandle> ModifierRows;
			DataStorage->RunQuery(Factory->MegaMeshModifierQueryRW, Queries::CreateDirectQueryCallbackBinding(
				[&ModifierRows](IDirectQueryContext& Context, RowHandle Row)
				{
					ModifierRows.Add(Row);
				}));
			for (RowHandle ModifierRow : ModifierRows)
			{
				DataStorage->RemoveColumns<FTypedElementSelectionColumn>(ModifierRow);
			}

			USelection* Selection = Cast<USelection>(InObject);

			if (!Selection)
			{
				return;
			}
			TArray<USceneComponent*> SelectedComponents;
			TArray<UModifierComponent*> SelectedModifiers;
			TArray<AActor*> SelectedActors;
			Selection->GetSelectedObjects<USceneComponent>(SelectedComponents);
			Selection->GetSelectedObjects<UModifierComponent>(SelectedModifiers);
			Selection->GetSelectedObjects<AActor>(SelectedActors);

			if (SelectedComponents.Num() > SelectedModifiers.Num())
			{
				SelectedModifiers.RemoveAll([&SelectedComponents](UModifierComponent* Modifier)
					{
						return SelectedComponents.Contains(Modifier);
					});
			}

			for (UModifierComponent* Modifier : SelectedModifiers)
			{
				RowHandle ModifierRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain,
					FMapKeyView(Modifier));

				if (Modifier->IsSelected() && ModifierRow != InvalidRowHandle)
				{
					DataStorage->AddColumn(ModifierRow, FTypedElementSelectionColumn{ .SelectionSet = FName() });
				}
			}

			for (AActor* Actor : SelectedActors)
			{
				TInlineComponentArray< UModifierComponent*, 10> OwnedModifiers;
				Actor->GetComponents<UModifierComponent>(OwnedModifiers, false);

				for (UModifierComponent* Modifier : OwnedModifiers)
				{
					RowHandle ModifierRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain,
						FMapKeyView(Modifier));
					if (ModifierRow != InvalidRowHandle)
					{
						DataStorage->AddColumn(ModifierRow, FTypedElementSelectionColumn{ .SelectionSet = FName() });
					}
				}
			}
		}
	}

	if (HierarchyViewer && DataStorage)
	{

		USelection* Selection = Cast<USelection>(InObject);

		if (!Selection)
		{
			return;
		}
		TArray<USceneComponent*> SelectedComponents;
		TArray<UModifierComponent*> SelectedModifiers;
		TArray<AActor*> SelectedActors;
		Selection->GetSelectedObjects<USceneComponent>(SelectedComponents);
		Selection->GetSelectedObjects<UModifierComponent>(SelectedModifiers);
		Selection->GetSelectedObjects<AActor>(SelectedActors);

		if (SelectedComponents.Num() > SelectedModifiers.Num())
		{
			SelectedModifiers.RemoveAll([&SelectedComponents](UModifierComponent* Modifier)
				{
					return SelectedComponents.Contains(Modifier);
				});
		}

		HierarchyViewer->ClearSelection();

		for (UModifierComponent* Modifier : SelectedModifiers)
		{
			RowHandle ModifierRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain,
				FMapKeyView(Modifier));

			if (ModifierRow != InvalidRowHandle)
			{
				HierarchyViewer->SetSelection(ModifierRow, true, ESelectInfo::Direct);					
			}
		}

		for (AActor* Actor : SelectedActors)
		{
			TInlineComponentArray< UModifierComponent*, 10> OwnedModifiers;
			Actor->GetComponents<UModifierComponent>(OwnedModifiers, false);

			for (UModifierComponent* Modifier : OwnedModifiers)
			{
				RowHandle ModifierRow = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain,
					FMapKeyView(Modifier));
				if (ModifierRow != InvalidRowHandle)
				{
					HierarchyViewer->SetSelection(ModifierRow, true, ESelectInfo::Direct);
				}
			}
		}

	}

}


void SMegaMeshSettingsWidget::OpenDefinitionInContentBrowser()
{
	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
	if (ActiveMegaMesh.IsValid() && ActiveMegaMesh->Definition.IsValid())
	{
		TArray<UObject*> ObjectsToBrowseTo;
		ObjectsToBrowseTo.Add(ActiveMegaMesh->Definition.Get());
		ContentBrowserModule.Get().SyncBrowserToAssets(ObjectsToBrowseTo);
	}
}

bool SMegaMeshSettingsWidget::CanOpenDefinitionInContentBrowser() const
{
	return ActiveMegaMesh.IsValid() && ActiveMegaMesh->Definition.IsValid();	
}

FReply SMegaMeshSettingsWidget::OnMegaMeshSelectClicked()
{
	if (ActiveMegaMesh.IsValid())
	{
		GEditor->SelectNone(true, true);
		GEditor->SelectActor(ActiveMegaMesh->MegaMesh.Get(), true, true, true, true);
	}

	return FReply::Handled();
}

bool SMegaMeshSettingsWidget::IsActiveMegaMesh() const
{
	return ActiveMegaMesh.IsValid();
}

Editor::DataStorage::ICoreProvider* SMegaMeshSettingsWidget::GetDataStorage()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

Editor::DataStorage::IUiProvider* SMegaMeshSettingsWidget::GetDataStorageUI()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
}

Editor::DataStorage::ICompatibilityProvider* SMegaMeshSettingsWidget::GetDataStorageCompatibility()
{
	using namespace Editor::DataStorage;
	return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE
