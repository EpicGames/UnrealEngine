// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SComboBox.h"
#include "DataStorage/Handles.h"
#include "SceneOutlinerFwd.h"
#include "IDetailsView.h"
#include "UObject/GCObject.h"

#include "MeshPartitionSettingsWidget.generated.h"

#define UE_API MESHPARTITIONEDITORUI_API

class ISceneOutliner;
struct FSceneOutlinerInitializationOptions;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class IUiProvider;
	class ICompatibilityProvider;	
	class SHierarchyViewer;
	class STedsCompositeHierarchyViewer;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::Outliner
{
	struct FTedsOutlinerParams;
}

DECLARE_MULTICAST_DELEGATE_TwoParams(FFilterSettingsModifiedSignature, UObject*, FProperty*);


UCLASS(MinimalAPI)
class UFilterSettings : public UObject
{
	GENERATED_BODY()

public:

	FFilterSettingsModifiedSignature& GetOnModified()
	{
		return OnModified;
	}

	UPROPERTY(EditAnywhere, Category = "FilterData")
	TSoftObjectPtr<AActor> BoundsFilterActor;


#if WITH_EDITOR
	/**
	  * Posts a message to the OnModified delegate with the modified FProperty
	  * @warning Please consider listening to OnModified instead of overriding this function
	  * @warning this function is currently only called in Editor (not at runtime)
	  */
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent);
#endif

protected:

	FFilterSettingsModifiedSignature OnModified;

};

namespace UE::MeshPartition
{
	
class UMeshPartitionEditorComponent;
class AMeshPartition;
class UMeshPartitionDefinition;



/**
* Widget for displaying a single group tag.
*/
class SMegaMeshSettingsWidget : public SCompoundWidget, public FGCObject
{
	SLATE_DECLARE_WIDGET(SMegaMeshSettingsWidget, SCompoundWidget)

public:
	SLATE_BEGIN_ARGS(SMegaMeshSettingsWidget)
		{}

	SLATE_END_ARGS();

	UE_API SMegaMeshSettingsWidget();
	UE_API ~SMegaMeshSettingsWidget();
	UE_API void Construct(const FArguments& InArgs);
	UE_API void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override {	return TEXT("SMegaMeshSettingsWidget");	}

private:

	struct FMegaMeshEntry {
		TWeakObjectPtr<AMeshPartition> MegaMesh;
		TWeakObjectPtr<UMeshPartitionDefinition> Definition;
		FString MegaMeshLabel;
		Editor::DataStorage::RowHandle MegaMeshRow;
	};

	TSharedPtr<IDetailsView> FilterDetailsView;
	TObjectPtr<UFilterSettings> FilterSettings;
	TWeakObjectPtr<AActor> PreviousBoundsFilterActor;

	TArray<TSharedPtr<FMegaMeshEntry>> MegaMeshList;
	TSharedPtr<SComboBox<TSharedPtr<FMegaMeshEntry>>> MegaMeshSelector;
	TSharedPtr<FMegaMeshEntry> ActiveMegaMesh;
	TSharedPtr<STextBlock> DefinitionPath;
	TSharedPtr<SSceneOutliner> Outliner;
	TSharedPtr<UE::Editor::DataStorage::STedsCompositeHierarchyViewer> HierarchyViewer;

	// Refresh the rows in the current view by syncing the the items source
	UE_API TSharedRef<SSceneOutliner> CreateOutliner(const FSceneOutlinerInitializationOptions& InInitOptions, const Editor::Outliner::FTedsOutlinerParams& InInitTedsOptions) const;
	UE_API TSharedRef<SSceneOutliner> CreateOutlinerWidget();
	UE_API TSharedRef<UE::Editor::DataStorage::STedsCompositeHierarchyViewer> CreateHierarchyWidget();
	UE_API void OnComboBoxSelectionChanged(TSharedPtr<FMegaMeshEntry> InNewSelection, ESelectInfo::Type SelectInfo);
	UE_API TSharedRef<SWidget> GenerateMegaMeshEntry(TSharedPtr<FMegaMeshEntry> InItem);
	TSharedPtr<IDetailsView> DetailsView;
	UE_API void OnMegaMeshSelectorOpening();
	UE_API FText GetActiveMegaMeshLabel() const;
	UE_API FText GetActiveDefinitionLabel() const;
	UE_API void OpenDefinitionInContentBrowser();
	UE_API bool CanOpenDefinitionInContentBrowser() const;
	UE_API FReply OnMegaMeshSelectClicked();
	UE_API bool IsActiveMegaMesh() const;
	UE_API void RebuildMegaMeshSelectionList();
	UE_API void OnOutlinerSelectionChanged(ESelectInfo::Type SelectInfo);
	UE_API void UpdateDetailsViewFromOutliner();
	UE_API void UpdateLevelSelectionFromOutliner();

	UE_API void OnFilterSettingsModified(UObject*, FProperty*);
	UE_API void OnLevelSelectionChanged(UObject* InObject);

	UE_API Editor::DataStorage::ICoreProvider* GetDataStorage();
	UE_API Editor::DataStorage::IUiProvider* GetDataStorageUI();
	UE_API Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();

};
} // namespace UE::MeshPartition

#undef UE_API
