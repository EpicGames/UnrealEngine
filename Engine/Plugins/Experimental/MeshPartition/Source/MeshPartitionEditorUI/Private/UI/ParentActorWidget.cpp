// Copyright Epic Games, Inc. All Rights Reserved.


#include "ParentActorWidget.h"

#include "MeshPartitionEditorSubsystem.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierComponent.h"

#include "MeshPartitionEditorUIStyle.h"

#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "Columns/LayerOutlinerColumns.h"
#include "Columns/SlateHeaderColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"

#define LOCTEXT_NAMESPACE "ParentActorWidget"

namespace UE::MeshPartition
{

	//
	// FParentActorWidgetHeaderConstructor
	//

	FParentActorWidgetHeaderConstructor::FParentActorWidgetHeaderConstructor()
		: Super(StaticStruct())
	{
	}

	TSharedPtr<SWidget> FParentActorWidgetHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
	{
		DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
			{
				.ColumnSizeMode = EColumnSizeMode::Manual,
				.Width = 100.0f
			});

		return SNew(STextBlock)
			.Text(LOCTEXT("ParentActorWidget_Header", "Actor"));
	}

	//
   // FParentActorWidgetConstructor
   //

	FParentActorWidgetConstructor::FParentActorWidgetConstructor()
		: Super(StaticStruct())
	{
	}

	TSharedPtr<SWidget> FParentActorWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments)
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SParentActorWidget, TargetRow, WidgetRow)
					.ToolTipText(LOCTEXT("ParentActorWidget_Tooltip", "Owning actor for this Modifier"))
			];
	}

	//
	// SParentActorWidget
	//

	void SParentActorWidget::Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow)
	{
		TargetRow = InTargetRow;
		WidgetRow = InWidgetRow;

		ChildSlot
			[
				SNew(STextBlock)
				.Text(this, &SParentActorWidget::GetParentActorLabel)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			];
	}

	FText SParentActorWidget::GetParentActorLabel() const
	{
		Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage();
		if (DataStorage)
		{
			FParentActorRefColumn* ParentActorRef = DataStorage->GetColumn<FParentActorRefColumn>(TargetRow);
			if (ParentActorRef)
			{
				FTypedElementLabelColumn* Label = DataStorage->GetColumn<FTypedElementLabelColumn>(ParentActorRef->ParentActor);
				if (Label)
				{
					return FText::FromString(Label->Label);
				}
			}
		}

		return LOCTEXT("SParentActorWidget_UnknownActor", "Unknown Actor");
	}

	UE::Editor::DataStorage::ICoreProvider* SParentActorWidget::GetDataStorage()
	{
		using namespace UE::Editor::DataStorage;
		return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	}

	UE::Editor::DataStorage::ICoreProvider* SParentActorWidget::GetDataStorageUI()
	{
		using namespace UE::Editor::DataStorage;
		return GetMutableDataStorageFeature<ICoreProvider>(UiFeatureName);
	}

	UE::Editor::DataStorage::ICompatibilityProvider* SParentActorWidget::GetDataStorageCompatibility()
	{
		using namespace UE::Editor::DataStorage;
		return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	}

}

#undef LOCTEXT_NAMESPACE