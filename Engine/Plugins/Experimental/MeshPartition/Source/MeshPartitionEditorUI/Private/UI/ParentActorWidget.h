// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Images/SImage.h"

#include "UObject/StrongObjectPtr.h"
#include "Delegates/Delegate.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "ScopedTransaction.h"

#include "ParentActorWidget.generated.h"

namespace UE::MeshPartition
{

	USTRUCT()
	struct FParentActorWidgetHeaderConstructor : public FSimpleWidgetConstructor
	{
		GENERATED_BODY()

	public:
		FParentActorWidgetHeaderConstructor();
		~FParentActorWidgetHeaderConstructor() override = default;

		virtual TSharedPtr<SWidget> CreateWidget(
			UE::Editor::DataStorage::ICoreProvider* DataStorage,
			UE::Editor::DataStorage::IUiProvider* DataStorageUi,
			UE::Editor::DataStorage::RowHandle TargetRow,
			UE::Editor::DataStorage::RowHandle WidgetRow,
			const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	protected:
	};

	USTRUCT()
	struct FParentActorWidgetConstructor : public FSimpleWidgetConstructor
	{
		GENERATED_BODY()

	public:
		FParentActorWidgetConstructor();
		~FParentActorWidgetConstructor() override = default;

		virtual TSharedPtr<SWidget> CreateWidget(
			UE::Editor::DataStorage::ICoreProvider* DataStorage,
			UE::Editor::DataStorage::IUiProvider* DataStorageUi,
			UE::Editor::DataStorage::RowHandle TargetRow,
			UE::Editor::DataStorage::RowHandle WidgetRow,
			const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	protected:
	};


	/** Widget responsible for displaying the owning actor of a modifier */
	class SParentActorWidget : public SCompoundWidget
	{
		using RowHandle = UE::Editor::DataStorage::RowHandle;

	public:
		SLATE_BEGIN_ARGS(SParentActorWidget) {}
		SLATE_END_ARGS()

		/** Construct this widget */
		void Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow);

	protected:

		RowHandle TargetRow;
		RowHandle WidgetRow;

	private:


		FText GetParentActorLabel() const;

		static UE::Editor::DataStorage::ICoreProvider* GetDataStorage();
		static UE::Editor::DataStorage::ICoreProvider* GetDataStorageUI();
		static UE::Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();
	};
}