// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "LayerOutlinerColumns.generated.h"

namespace UE::MeshPartition
{
class AMeshPartition;

USTRUCT(meta = (DisplayName = "Parent"))
struct FMegaMeshRowParentColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle Parent;		

	TArray<UE::Editor::DataStorage::RowHandle> Children;
	TMap<UE::Editor::DataStorage::RowHandle, int32> ChildrenSet;
};

USTRUCT(meta = (DisplayName = "Unresolved Modifier Layer"))
struct FUnresolvedMegaMeshLayer : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FString LayerName;
	FName Layer;
};

USTRUCT(meta = (DisplayName = "Is MeshPartition Bounds Filter Source"))
struct FMegaMeshBoundsFilterSourceTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Parent Actor Reference"))
struct FParentActorRefColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle ParentActor;
};

USTRUCT(meta = (DisplayName = "Is this Object an AMeshPartition"))
struct FIsMegaMeshObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Is this Object a Mesh Partition Modifier"))
struct FIsMegaMeshModifierTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Layer tags need updating"))
struct FMegaMeshLayerUpdatedTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Modifier Affects a Selected Base Section"))
struct FMegaMeshAffectsFilterBounds final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Is this Mesh Partition Modifier assigned to a Mesh Partition?"))
struct FIsMegaMeshModifierAssignedTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Not Hidden in Mesh Partition Outliner"))
struct FMegaMeshNotHiddenInOutlinerTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
}; 

USTRUCT(meta = (DisplayName = "Mesh Partition Timing Statistics"))
struct FMegaMeshTimingStatistics : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	double TotalTimeMean;
	double TotalTimeStandardDeviation;
};

USTRUCT(meta = (DisplayName = "Modifier Timing"))
struct FMegaMeshModifierTiming : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	int InstanceCount;
	double TotalTime;
	double MinTime;
	double MaxTime;
};

USTRUCT(meta = (DisplayName = "Is this Object an UMeshPartitionDefinition"))
struct FIsMegaMeshDefinitionObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Is Active Layer"))
struct FMegaMeshActiveLayerTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Is this a UMeshPartitionDefinition Layer"))
struct FIsMegaMeshDefinitionLayerTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Was this row updated since the last frame?"))
struct FMegaMeshUpdatedTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Mark for build to here"))
struct FMegaMeshBuildToHereTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Priority"))
struct FMegaMeshPriorityColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 Priority = 0;
};

USTRUCT(meta = (DisplayName = "Sort Key"))
struct FMegaMeshModifierSortColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 SortKey = 0;
};

USTRUCT(meta = (DisplayName = "Mesh Partition Reference"))
struct FMegaMeshReferenceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	TWeakObjectPtr<AMeshPartition> Mesh = nullptr;
};

USTRUCT(meta = (DisplayName = "Auto Build Visibility"))
struct FAutoMegaMeshBuildVisibilityTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Build Visibility Status"))
struct FMegaMeshBuildToStatus final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	uint32 LayerBuildToIndex = 0;

	UE::Editor::DataStorage::RowHandle ModiferToBuildTo;
};

USTRUCT(meta = (DisplayName = "Build Up To This Layer"))
struct FMegaMeshBuildUpToThisLayerColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	bool Enabled = true;
};

USTRUCT(meta = (DisplayName = "Draw Bounds"))
struct FMegaMeshDrawBoundsColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	bool bEnabled = false;
};


USTRUCT(meta = (DisplayName = "Active Mesh Partition for Visualizing"))
struct FMegaMeshVisualizationsActiveTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMegaMeshLayerNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FName Name;
};

USTRUCT(meta = (DisplayName = "This row is visible in the Mesh Partition outliner"))
struct FIsMegaMeshActiveInOutlinerTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
} // namespace UE::MeshPartition