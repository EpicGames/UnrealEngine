// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "WorkspaceMenuStructure.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SWidget.h"

DECLARE_LOG_CATEGORY_EXTERN(LogMegaMeshEditorUI, Log, All);

struct FToolMenuContext;
class AStaticMeshActor;
class FExtender;

namespace UE::MeshPartition
{
class SMegaMeshSettingsWidget;

class FMegaMeshEditorUIModule : public IModuleInterface
{
public:
	void RegisterTabSpawners(const TSharedPtr<FWorkspaceItem>& WorkspaceGroup);
	void UnregisterTabSpawners();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void OnPostEngineInit();
	TArray<FName> VisualizersToUnregisterOnShutdown;
	TArray<FName> ClassesToUnregisterOnShutdown;
	TArray<FName> PropertiesToUnregisterOnShutdown;

	void RegisterPropertyEditorCategories();
	TSharedRef<SDockTab> MakeMegaMeshSettingsTab(const FSpawnTabArgs&);
	TSharedRef<SWidget> GetMegaMeshSettings(const TSharedRef<SDockTab>& InParentTab);

	bool bHasRegisteredTabSpawners = false;
	TWeakPtr<SMegaMeshSettingsWidget> MegaMeshSettingsPtr;

	TSharedPtr<FExtender> ScalabilityMenuExtender;
	void AddScalabilitySubMenu(FMenuBuilder& InMenuBuilder);
	void FillScalabilitySubmenu(FMenuBuilder& InMenuBuilder);

	TSharedRef<SWidget> CreateEnablePreviewSectionSimplificationWidget();
	TSharedRef<SWidget> CreatePreviewSectionSimplificationEdgeLengthWidget();
	TSharedRef<SWidget> CreatePreviewSectionSimplificationMinVertexNumberWidget();
};
} // namespace UE::MeshPartition
