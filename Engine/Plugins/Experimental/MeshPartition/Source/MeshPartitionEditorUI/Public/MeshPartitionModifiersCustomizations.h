// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SharedPointer.h"

class SCheckBox;
class IPropertyHandle;
class IMeshLayersController;
class USplineComponent;

namespace UE::MeshPartition
{
class UProjectMeshLayersModifier;
class USplineModifier;

class FMegaMeshProjectSculptLayersModifierDetails : public IDetailCustomization
{
public:
	FMegaMeshProjectSculptLayersModifierDetails();
	virtual ~FMegaMeshProjectSculptLayersModifierDetails();
	
	static TSharedRef<IDetailCustomization> MakeInstance();
	void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	TSharedRef<IMeshLayersController> Controller;
	FDelegateHandle ModifierUpdatesHandle;
	TWeakObjectPtr<MeshPartition::UProjectMeshLayersModifier> Modifier;
};

class FMegaMeshTexturePatchDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	
private:
	void ReparentHeightDisplacement(IDetailLayoutBuilder& DetailBuilder);

	void AddWeightChannelCopyButtons(IDetailLayoutBuilder& DetailBuilder);

	void AddTessellationCheckbox(IDetailLayoutBuilder& DetailBuilder);

	TSharedPtr<IPropertyHandle> TessellationModeProperty;
	TSharedPtr<SCheckBox> CheckBoxTessellation;
	TSharedPtr<SCheckBox> CheckBoxTessellationFast;
	bool bTessellationFast { false };
};

class FSplineSoftObjectPointerDetails : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	void AddComboBox(IDetailLayoutBuilder& DetailBuilder);

	TWeakObjectPtr<MeshPartition::USplineModifier> SplineModifier;
	TArray<TWeakObjectPtr<USplineComponent>> Splines;
	TArray<TSharedPtr<FString>> SplineNames;
	TSharedPtr<FString> SelectedName;
};


} // namespace UE::MeshPartition
