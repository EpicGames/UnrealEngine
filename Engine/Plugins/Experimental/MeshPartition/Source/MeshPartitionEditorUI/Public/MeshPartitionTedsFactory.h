// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "MeshPartitionTedsFactory.generated.h"

#define UE_API MESHPARTITIONEDITORUI_API

namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;
class AMeshPartition;
class UMeshPartitionDefinition;
struct FOnChangedEventInfo;

UCLASS(MinimalAPI)
class UMegaMeshTedsFactory : public UEditorDataStorageFactory
{

public:
	GENERATED_BODY()

public:

	UE_API virtual void PreRegister(Editor::DataStorage::ICoreProvider& DataStorage) override;
	UE_API virtual void PostRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	UE_API virtual void RegisterTables(Editor::DataStorage::ICoreProvider& DataStorage,
		Editor::DataStorage::ICompatibilityProvider& CompatibilityDataStorage) override;
	UE_API virtual void RegisterQueries(Editor::DataStorage::ICoreProvider& DataStorage) override;
	UE_API virtual void RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	UE_API virtual void RegisterWidgetConstructors(Editor::DataStorage::ICoreProvider& DataStorage,
		Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	UE_API virtual void RegisterWidgetPurposes(Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	UE_API virtual void PreShutdown(Editor::DataStorage::ICoreProvider& DataStorage) override;

public:

	Editor::DataStorage::QueryHandle MegaMeshQuery;
	Editor::DataStorage::QueryHandle DefinitionLayerQueryRO;
	Editor::DataStorage::QueryHandle ActiveLayerQuery;
	Editor::DataStorage::QueryHandle MegaMeshModifierQueryRW;
	Editor::DataStorage::FHierarchyHandle MegaMeshHierarchy;

private:


	std::atomic<bool> bShouldUpdateModifierSortKeys = false;
	std::atomic<bool> bShouldResolveBoundsFiltering = false;

	void ResolveLayerRows();
	void GenerateModifierSortKeys();
	void ResolveBoundsFiltering();
	void CleanMegaMeshRows(uint32 MapFlags);

	void OnMegaMeshChanged(const FOnChangedEventInfo&);

	Editor::DataStorage::TableHandle MegaMeshTable;
	Editor::DataStorage::TableHandle MegaMeshDefinitionTable;
	Editor::DataStorage::TableHandle MegaMeshDefinitionLayerTable;
	Editor::DataStorage::TableHandle MegaMeshModifierTable;

	Editor::DataStorage::QueryHandle DefinitionLayerQueryRW;
	Editor::DataStorage::QueryHandle DefinitionQuery;
	Editor::DataStorage::QueryHandle MegaMeshQueryRW;
	Editor::DataStorage::QueryHandle UnresolvedRetrievalQuery;
	Editor::DataStorage::QueryHandle BoundsFilterObjectQuery;
};
} // namespace UE::MeshPartition

#undef UE_API
