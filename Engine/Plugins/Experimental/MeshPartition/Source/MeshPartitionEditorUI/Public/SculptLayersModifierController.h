// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ModelingWidgets/MeshLayersController.h"

namespace UE::MeshPartition
{
class FSculptLayersModifiersController : public IMeshLayersController
{
public:
	FSculptLayersModifiersController();

	// get access to the Properties.
	// Warning: do not make modifications to the asset directly, use the controller API
	MeshPartition::UProjectMeshLayersModifier* GetProperties() const { return Properties;};
	virtual void SetProperties(MeshPartition::UProjectMeshLayersModifier* InProperties) override
	{
		Properties = const_cast<MeshPartition::UProjectMeshLayersModifier*>(InProperties);
	};

private:
	TObjectPtr<MeshPartition::UProjectMeshLayersModifier> Properties = nullptr;

public:
	
	virtual FName GetLayerName(const int32 InLayerIndex) const override;
	virtual void SetLayerName(const int32 InLayerIndex, const FName InName) const override;
	virtual double GetLayerWeight(const int32 InLayerIndex) const override;
	virtual void SetLayerWeight(const int32 InLayerIndex, const double InWeight, EPropertyChangeType::Type ChangeType) const override;
	virtual int32 GetNumMeshLayers() const override;
	virtual void RefreshLayersStackView() const override;
};
} // namespace UE::MeshPartition