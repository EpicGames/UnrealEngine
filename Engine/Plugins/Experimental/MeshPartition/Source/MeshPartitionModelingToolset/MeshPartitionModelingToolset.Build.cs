// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MeshPartitionModelingToolset : ModuleRules
{
	public MeshPartitionModelingToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
		bAllowUETypesInNamespaces = true;
		
		PublicIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PrivateIncludePaths.AddRange(
			new string[] {
			}
			);
		
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"MeshPartition", // UE::MeshPartition::FChannelName
				"MeshPartitionEditor",
			}
			);
			
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"ToolMenus",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"ImageCore",

				"TypedElementFramework",
				"TypedElementRuntime",

				"MeshPartitionEditorUI",

				"MathCore",

				"DynamicMesh",
				"GeometryCore",
				"GeometryFramework",
				"GeometryAlgorithms",
				"MeshConversion",
				"MeshDescription",
                "StaticMeshDescription",
				"ModelingComponents",
				"ModelingComponentsEditorOnly", // UDynamicMeshComponentToolTarget
				"ModelingOperators",
				"ModelingOperatorsEditorOnly",
				"MeshModelingToolsExp",
				"MeshModelingTools",
				"HairStrandsCore",
				"ModelingToolsEditorMode",
                "InteractiveToolsFramework",
				"Projects",
				"SourceControl"
			}
			);
		
		DynamicallyLoadedModuleNames.AddRange(
			new string[]
			{
			}
			);
	}
}
