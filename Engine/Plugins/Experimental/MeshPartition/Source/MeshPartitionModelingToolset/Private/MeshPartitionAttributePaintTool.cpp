// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionAttributePaintTool.h"

#include "InteractiveToolManager.h" // for NewObject parent
#include "MeshPartitionModifierComponent.h"
#include "ModelingToolTargetUtil.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"
#include "MeshPartitionModifierToolTarget.h"

#define LOCTEXT_NAMESPACE "UAttributePaintTool"

namespace UE::MeshPartition
{
namespace MegaMeshAttributePaintToolLocals
{
	/**
	* Adapter that allows us to only allow painting some weight attributes and not others, used
	*  for layer modifiers, where we want to keep certain inactive layers around but not paintable,
	*  in case we add them through the tool.
	*/
	//~ TODO: This is mostly a direct copy of FDynamicMeshVertexAttributeSource and FDynamicMeshVertexAttributeSource,
	//~  which are currently private in MeshAttributePaintTool.cpp. Consider exposing those as protected in the
	//~  base paint tool and subclassing here (though it would be quite messy to have all those classes in header...)
	class FFilteredDynamicMeshVertexAttributeSource : public IMeshVertexAttributeSource
	{
	public:
		// Copy of FDynamicMeshVertexAttributeSource
		class FAttributeAdapter : public IMeshVertexAttributeAdapter
		{
		public:
			FDynamicMesh3* Mesh;
			Geometry::FDynamicMeshWeightAttribute* WeightAttribute;

			FAttributeAdapter(FDynamicMesh3* InMesh, Geometry::FDynamicMeshWeightAttribute* InWeightAttribute)
				: Mesh(InMesh), WeightAttribute(InWeightAttribute)
			{
			}

			virtual int32 ElementNum() const override
			{
				return Mesh->MaxVertexID();
			}

			virtual float GetValue(int32 Index) const override
			{
				float Wt;
				WeightAttribute->GetValue(Index, &Wt);
				return Wt;
			}

			virtual void SetValue(int32 Index, float Value) override
			{
				WeightAttribute->SetScalarValue(Index, Value);
			}

			virtual Geometry::FInterval1f GetValueRange() override
			{
				return Geometry::FInterval1f(0.0f, 1.0f);
			}
		};

		Geometry::FDynamicMesh3* Mesh = nullptr;
		TSet<FName> AllowedAttributes;

		FFilteredDynamicMeshVertexAttributeSource(Geometry::FDynamicMesh3* MeshIn)
		{
			Mesh = MeshIn;
		}

		virtual int32 GetAttributeElementNum() override
		{
			return Mesh->MaxVertexID();
		}

		virtual TArray<FName> GetAttributeList() override
		{
			TArray<FName> Result;
		
			if (Geometry::FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
			{
				const int32 NumLayers = Attributes->NumWeightLayers();
				Result.Reserve(NumLayers);
				for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
				{
					FName LayerName = Attributes->GetWeightLayer(LayerIdx)->GetName();
					if (AllowedAttributes.Contains(LayerName))
					{
						Result.Add(LayerName);
					}
				}
			}
			return Result;
		}

		virtual TUniquePtr<IMeshVertexAttributeAdapter> GetAttribute(FName AttributeName) override
		{
			if (Geometry::FDynamicMeshAttributeSet* Attributes = Mesh->Attributes())
			{
				const int32 NumLayers = Attributes->NumWeightLayers();
				for (int32 LayerIdx = 0; LayerIdx < NumLayers; ++LayerIdx)
				{
					Geometry::FDynamicMeshWeightAttribute* WeightLayer = Attributes->GetWeightLayer(LayerIdx);
					if (WeightLayer->GetName() == AttributeName)
					{
						return MakeUnique<FAttributeAdapter>(Mesh, WeightLayer);
					}
				}
			}
			return nullptr;
		}
	};//end FFilteredDynamicMeshVertexAttributeSource
}//end namespace MegaMeshAttributePaintToolLocals


void UAttributePaintTool::Setup()
{
	using namespace MegaMeshAttributePaintToolLocals;

	bPreferMeshDescription = false;

	Super::Setup();
	
	// Update default brush size mode to Radius
	BrushProperties->bSpecifyRadius = true;
	BrushProperties->BrushRadius = 500.f;
	BrushProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	// Recalculate with the updated radius
	RecalculateBrushRadius();
	
	// Update default material mode to ExistingMaterial.
	ViewProperties->MaterialMode = EMeshAttributePaintMaterialMode::ExistingMaterial;
	ViewProperties->RestoreProperties(this, GetPropertyCacheIdentifier());

	// This tool works on dynamic meshes only, currently
	ensure(EditedDynamicMesh);
	bFilteredAttributeSourceInitialized = false;


	// Layer modifiers need special handling, because we need to only allow painting on active layers,
	//  but keep around any that we might have built up in case a user adds that channel.
	if (MeshPartition::UProjectMeshLayersModifier* LayersModifier =
		Cast<MeshPartition::UProjectMeshLayersModifier>(ToolTarget::GetTargetSceneComponent(Target)))
	{
		if (ensure(EditedDynamicMesh))
		{
			AttributeSource = MakeUnique<FFilteredDynamicMeshVertexAttributeSource>(EditedDynamicMesh.Get());
			FFilteredDynamicMeshVertexAttributeSource* CastAttributeSource = static_cast<FFilteredDynamicMeshVertexAttributeSource*>(AttributeSource.Get());
			bFilteredAttributeSourceInitialized = true;

			for (const FSculptLayerModifierWeightAttributeEntry& Entry : LayersModifier->GetChannelEntries())
			{
				CastAttributeSource->AllowedAttributes.Add(Entry.ChannelName);
			}
			InitializeAttributes();	
			PendingNewSelectedIndex = AttribProps->Attributes.Num() > 0 ? 0 : -1;
		}
	}


	AddChannelProperties = NewObject<MeshPartition::UAttributePaintToolAddChannelProperties>();
	AddChannelProperties->Initialize(this);
	AddChannelProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	AddToolPropertySource(AddChannelProperties);

	// Initialize the weight channel if restored from the property cache.
	// Ensure that this occurs after the above AttributeSource filtering to guarantee
	// that the Channel and SelectedAttribute are in sync.
	if (!AddChannelProperties->WeightChannelName.IsNone())
	{
		AddChannel(AddChannelProperties->WeightChannelName.GetName());
	}

	WeightChannelNameWatcher.Initialize(
		[this]() { return AddChannelProperties->WeightChannelName.GetName(); },
		[this](FName NewName)
		{
			AddChannel(NewName);
		},
		AddChannelProperties->WeightChannelName.GetName());

	if (UEditableModifierToolTarget* ModifierTarget = Cast<UEditableModifierToolTarget>(Target))
	{
		ModifierTarget->UpdateRenderTextureForPreview(*EditedDynamicMesh);
		ModifierTarget->ConfigurePreviewForRendering(PreviewMesh->GetRootComponent());
	}
}

void UAttributePaintTool::OnTick(float DeltaTime)
{
	WeightChannelNameWatcher.CheckAndUpdate();
	Super::OnTick(DeltaTime);
}

void UAttributePaintTool::OnShutdown(EToolShutdownType ShutdownType)
{
	using namespace MegaMeshAttributePaintToolLocals;

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshAttributePaintTool", "Paint Channels"));
		
		// For layer modifier, actually apply the changes to its active channels
		MeshPartition::UProjectMeshLayersModifier* LayersModifier = 
			Cast<MeshPartition::UProjectMeshLayersModifier>(ToolTarget::GetTargetSceneComponent(Target));
		if (LayersModifier && ensure(bFilteredAttributeSourceInitialized))
		{
			TArray<FSculptLayerModifierWeightAttributeEntry> ChannelEntries = LayersModifier->GetChannelEntries();

			FFilteredDynamicMeshVertexAttributeSource* CastAttributeSource = static_cast<FFilteredDynamicMeshVertexAttributeSource*>(AttributeSource.Get());
			TSet<FName> ChannelsToAdd = CastAttributeSource->AllowedAttributes;
			for (const FSculptLayerModifierWeightAttributeEntry& Entry : ChannelEntries)
			{
				ChannelsToAdd.Remove(Entry.ChannelName);
			}
			if (!ChannelsToAdd.IsEmpty())
			{
				for (FName Channel : ChannelsToAdd)
				{
					FSculptLayerModifierWeightAttributeEntry Entry;
					Entry.ChannelName = Channel;
					ChannelEntries.Add(Entry);
				}
			}
			LayersModifier->Modify();
			LayersModifier->SetChannelEntries(ChannelEntries);
		}
	}

	Super::OnShutdown(ShutdownType);

	if (ShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->EndUndoTransaction();
	}

	AddChannelProperties->SaveProperties(this, GetPropertyCacheIdentifier());
	AddChannelProperties = nullptr;
	bFilteredAttributeSourceInitialized = false;
}

void UAttributePaintTool::ApplyStamp(const FBrushStampData& Stamp)
{
	Super::ApplyStamp(Stamp);

	if (UEditableModifierToolTarget* ModifierTarget = Cast<UEditableModifierToolTarget>(Target))
	{
		ModifierTarget->UpdateRenderTextureForPreview(*EditedDynamicMesh);
		ModifierTarget->ConfigurePreviewForRendering(PreviewMesh->GetRootComponent());
	}
}

void UAttributePaintTool::ExternalUpdateValues(int32 AttribIndex, const TArray<int32>& VertexIndices, const TArray<float>& NewValues)
{
	Super::ExternalUpdateValues(AttribIndex, VertexIndices, NewValues);

	if (UEditableModifierToolTarget* ModifierTarget = Cast<UEditableModifierToolTarget>(Target))
	{
		ModifierTarget->UpdateRenderTextureForPreview(*EditedDynamicMesh);
		ModifierTarget->ConfigurePreviewForRendering(PreviewMesh->GetRootComponent());
	}
}

FString UAttributePaintTool::GetPropertyCacheIdentifier() const
{
	return TEXT("MeshPartition::UAttributePaintTool");
}

void UAttributePaintTool::AddChannel(FName NewChannel)
{
	using namespace MegaMeshAttributePaintToolLocals;
	using namespace Geometry;

	if (NewChannel.IsNone())
	{
		return;
	}

	if (!ensure(EditedDynamicMesh))
	{
		// We could support mesh descriptions, but this tool currently just works on dynamic meshes
		return;
	}

	// See if given channel is already paintable. Note that layer modifiers may hold channels that
	//  are not paintable, so the index here is specifically into the paintable attributes.
	int32 SelectionIndex = AttributeSource->GetAttributeList().IndexOfByKey(NewChannel);
	if (SelectionIndex >= 0)
	{
		// Already present and paintable, so just set the selection
		PendingNewSelectedIndex = SelectionIndex;
		return;
	}

	// If we got here, this channel is not paintable/present.
	EditedDynamicMesh->EnableAttributes();
	FDynamicMeshAttributeSet* AttributeSet = EditedDynamicMesh->Attributes();

	// For layer modifiers, see if it already exists on the mesh.
	bool bNeedToAddChannel = true;
	if (MeshPartition::UProjectMeshLayersModifier* LayersModifier =
		Cast<MeshPartition::UProjectMeshLayersModifier>(ToolTarget::GetTargetSceneComponent(Target)))
	{
		for (int32 LayerIndex = 0; LayerIndex < AttributeSet->NumWeightLayers(); ++LayerIndex)
		{
			FDynamicMeshWeightAttribute* WeightLayer = AttributeSet->GetWeightLayer(LayerIndex);
			if (!ensure(WeightLayer))
			{
				continue;
			}
			if (WeightLayer->GetName() == NewChannel)
			{
				bNeedToAddChannel = false;
				break;
			}
		}

		// Regardless of whether it was already there or is about to be added, mark the channel paintable
		if (ensure(bFilteredAttributeSourceInitialized))
		{
			FFilteredDynamicMeshVertexAttributeSource* CastAttributeSource = static_cast<FFilteredDynamicMeshVertexAttributeSource*>(AttributeSource.Get());
			CastAttributeSource->AllowedAttributes.Add(NewChannel);
		}
	}

	// Add the channel if needed
	if (bNeedToAddChannel)
	{
		int32 WeightLayerIndex = AttributeSet->NumWeightLayers();
		AttributeSet->SetNumWeightLayers(WeightLayerIndex + 1);
		FDynamicMeshWeightAttribute* WeightLayer = AttributeSet->GetWeightLayer(WeightLayerIndex);
		if (ensure(WeightLayer))
		{
			WeightLayer->SetName(NewChannel);
		}
	}
	
	InitializeAttributes();
	PendingNewSelectedIndex = AttributeSource->GetAttributeList().IndexOfByKey(NewChannel);
	SelectedAttributeWatcher.SilentUpdate();
}

TArray<FName> UAttributePaintTool::GetChannelOptions()
{
	// Note: currently this relies on the fact that multi section tool targets still give one of their
	//  sections back from GetTargetSceneComponent. We could instead have an interface for this that all
	//  the megamesh tool targets implement, and perhaps we someday will.
	USceneComponent* Component = ToolTarget::GetTargetSceneComponent(Target);
	MeshPartition::UModifierComponent* Modifier = Cast<MeshPartition::UModifierComponent>(Component);
	if (!Modifier && Component)
	{
		// Single section targets have the base section as the owner of the component
		Modifier = Cast<MeshPartition::UModifierComponent>(Component->GetAttachParent());
	}
	if (Modifier)
	{
		return Modifier->GetMegaMeshDefinitionChannels();
	}
	return {};
}

void UAttributePaintTool::InitializeAttributes()
{
	Super::InitializeAttributes();
	
	// The base class issues a user warning if no attributes are present. For MeshPartition channels
	// this warning is not applicable, so clear those warnings explicitly after each call to InitializeAttributes().
	GetToolManager()->DisplayMessage(FText(), EToolMessageLevel::UserWarning);
}

UMeshSurfacePointTool* UAttributePaintToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	MeshPartition::UAttributePaintTool* Tool = NewObject<MeshPartition::UAttributePaintTool>(SceneState.ToolManager);
	Tool->SetWorld(SceneState.World);
	return Tool;
}

void UAttributePaintToolAddChannelProperties::Initialize(MeshPartition::UAttributePaintTool* ParentToolIn)
{
	ParentTool = ParentToolIn;
}

TArray<FName> UAttributePaintToolAddChannelProperties::GetChannelOptions() const
{
	if (!ParentTool.IsValid())
	{
		return TArray<FName>();
	}
	return ParentTool->GetChannelOptions();
}
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE
