// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionConvertTool.h"

#include "Engine/World.h"
#include "DynamicMeshEditor.h"
#include "InteractiveToolManager.h"
#include "MeshPartition.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionCommonProperties.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModelingToolsetModule.h"
#include "MeshPartitionPreviewUtils.h"
#include "ModelingToolTargetUtil.h"
#include "Drawing/PreviewGeometryActor.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "Operations/MeshPlaneCut.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/MeshTransforms.h"
#include "TargetInterfaces/DynamicMeshProvider.h"

// -------------------------------------------------------------------------------------------------------------------------

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "MeshPartition::UConvertTool"

namespace UE::MeshPartition
{
USingleSelectionMeshEditingTool* MeshPartition::UConvertToolBuilder::CreateNewTool(const FToolBuilderState& InSceneState) const
{
	return NewObject<MeshPartition::UConvertTool>(InSceneState.ToolManager);
}

const FToolTargetTypeRequirements& MeshPartition::UConvertToolBuilder::GetTargetRequirements() const
{
	// #todo_megamesh [roey]: how can we restrict this from being usable on mega mesh actors? Do we even want to?
	// maybe it's valid to convert a piece of a mega mesh into it's own new megamesh. If so, some questions:
	// Should a user be able to do this only on the base?
	// Would we want to allow this only on PreviewSection?
	// Should we clone also the modifiers affecting this part?
	static FToolTargetTypeRequirements TypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshProvider::StaticClass(),
			UPrimitiveComponentBackedTarget::StaticClass()});

	return TypeRequirements;
}

// -------------------------------------------------------------------------------------------------------------------------

void UConvertTool::Setup()
{
	Super::Setup();

	SplitProperties = NewObject<MeshPartition::USplitProperties>(this);
	AddToolPropertySource(SplitProperties);
	SplitProperties->RestoreProperties(this);

	SplitProperties->WatchProperty(SplitProperties->SectionLayout, [this](FIntVector SectionLayout)
	{
		bPreviewGeometryNeedsUpdate = true;
	});

	CreateProperties = NewObject<MeshPartition::UCreateProperties>(this);
	AddToolPropertySource(CreateProperties);
	CreateProperties->RestoreProperties(this);

	if (!TargetWorld.IsValid() || !TargetWorld->GetWorldPartition())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("MeshPartition_WorldPartitionRequired", "Mesh Partition requires World Partition to be enabled."),
			EToolMessageLevel::UserError);
	}

	if (USceneComponent* TargetComponent = UE::ToolTarget::GetTargetSceneComponent(Target))
	{
		PreviewGeometry = NewObject<UPreviewGeometry>(this);
		PreviewGeometry->CreateInWorld(TargetComponent->GetWorld(), TargetComponent->GetComponentTransform());
		SplitBounds = FBox(UE::ToolTarget::GetDynamicMeshCopy(Target).GetBounds());
		CreateOrUpdatePreviewGeometry();
	}
}

void UConvertTool::OnTick(float DeltaTime)
{
	if (PreviewGeometry)
	{
		if (bPreviewGeometryNeedsUpdate)
		{
			CreateOrUpdatePreviewGeometry();
			bPreviewGeometryNeedsUpdate = false;
		}
		
		// make sure the preview transform continues to match the target
		if (USceneComponent* TargetComponent = UE::ToolTarget::GetTargetSceneComponent(Target))
		{
			PreviewGeometry->SetTransform(TargetComponent->GetComponentTransform());
		}
	}
}

FIntVector UConvertTool::GetClampedSectionLayout() const
{
	return SplitProperties->SectionLayout.ComponentMax(FIntVector(1));
}

void UConvertTool::CreateOrUpdatePreviewGeometry()
{
	CreatePreviewGridLines(SplitBounds, GetClampedSectionLayout(), TEXT("ConvertSectionLines"), PreviewGeometry);
}

bool UConvertTool::CanAccept() const
{
	if (!TargetWorld.IsValid() || !TargetWorld->GetWorldPartition())
	{
		return false;
	}

	return true;
}

void UConvertTool::Shutdown(EToolShutdownType InShutdownType)
{
	if (InShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshPartitionConvertTransaction", "Convert to Mesh Partition"));

		AActor* TargetActor = UE::ToolTarget::GetTargetActor(Target);
		
		FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
		TArray<UMaterialInterface*> Materials = MaterialSet.Materials;

		const FTransform Transform = TargetActor->GetActorTransform();
		AMeshPartition* MegaMesh = nullptr;
		UMeshPartitionEditorComponent* MegaMeshEditorComponent = nullptr;

		if (CreateProperties->ExistingMegaMesh != nullptr)
		{
			MegaMesh = CreateProperties->ExistingMegaMesh;
			MegaMeshEditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent());
			
			UMaterialInterface* MegaMeshMaterial = MegaMeshEditorComponent->GetDefinitionMaterial();
			if (MegaMeshMaterial != nullptr)
			{
				Materials.Empty();
				Materials.Emplace(MegaMeshMaterial);
			}
		}
		else
		{
			MegaMesh = TargetWorld->SpawnActor<AMeshPartition>(AMeshPartition::StaticClass(), FTransform::Identity);
			MegaMeshEditorComponent = NewObject<UMeshPartitionEditorComponent>(MegaMesh, UMeshPartitionEditorComponent::StaticClass(), TEXT("MegaMeshEditorComponent"), RF_Transactional);
			MegaMesh->SetMeshPartitionComponent(MegaMeshEditorComponent);

			if (CreateProperties->NewMegaMeshDefinition != nullptr)
			{
				MegaMesh->SetMeshPartitionDefinition(CreateProperties->NewMegaMeshDefinition);
			}
		}

		TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
		if (GetClampedSectionLayout().GetMax() > 1)
		{
			SplitMeshes = SplitMesh(UE::ToolTarget::GetDynamicMeshCopy(Target));
		}
		else
		{
			SplitMeshes.Add(MakeUnique<FDynamicMesh3>(UE::ToolTarget::GetDynamicMeshCopy(Target)));
		}

		TArray<FBox> TotalBounds;
		for (TUniquePtr<FDynamicMesh3>& SectionMesh : SplitMeshes)
		{
			// Fixup vertices to center them
			FAxisAlignedBox3d Bounds = SectionMesh->GetBounds();
			FVector Center = Bounds.Center();

			TotalBounds.Emplace(static_cast<FBox>(Bounds).TransformBy(Transform));

			FTransform SectionToWorld = FTransform(Center);
			MeshTransforms::ApplyTransformInverse(*SectionMesh, SectionToWorld);

			SectionToWorld = (CreateProperties->ExistingMegaMesh) ? SectionToWorld * Transform : SectionToWorld;
			MegaMeshEditorComponent->SpawnBaseModifier(MoveTemp(*SectionMesh), Materials, SectionToWorld);
		}

		if (!CreateProperties->ExistingMegaMesh)
		{
			MegaMesh->SetActorTransform(Transform);
		}

		MegaMeshEditorComponent->OnBoundsChanged(TotalBounds, EChangeType::StateChange);

		TargetWorld->DestroyActor(TargetActor);

		GetToolManager()->EndUndoTransaction();
	}
	
	SplitProperties->SaveProperties(this);
	CreateProperties->SaveProperties(this);

	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
	}
}

TArray<TUniquePtr<FDynamicMesh3>> MeshPartition::UConvertTool::SplitMesh(const FDynamicMesh3& InMesh) const
{
	FDynamicMesh3 EditMesh(InMesh);
	
	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
	int32 MaxSubObjectID = -1;
	const FBox MeshBounds3d = FBox(EditMesh.GetBounds());
	const FVector MeshBoundsSize = MeshBounds3d.GetSize();
	const FIntVector Layout = GetClampedSectionLayout();

	// Setup the dynamic mesh with attributes needed by FMeshPlaneCut and Geometry::FDynamicMeshEditor::SplitMesh
	EditMesh.EnableAttributes();
	TDynamicMeshScalarTriangleAttribute<int32>* SubObjectIDs = new TDynamicMeshScalarTriangleAttribute<int32>(&EditMesh);
	SubObjectIDs->Initialize(0);
	EditMesh.Attributes()->AttachAttribute(TEXT("ObjectIndexAttribute"), SubObjectIDs);

	// Per-axis plane cuts. Skip any axis with only one section, or with near-zero extent
	const FVector3d AxisNormals[3] = { FVector3d(1.0, 0.0, 0.0), FVector3d(0.0, 1.0, 0.0), FVector3d(0.0, 0.0, 1.0) };
	for (int32 Axis = 0; Axis < 3; ++Axis)
	{
		if (Layout[Axis] <= 1 || MeshBoundsSize[Axis] <= UE_DOUBLE_SMALL_NUMBER)
		{
			continue;
		}

		FVector3d SliceOffset = FVector3d::ZeroVector;
		SliceOffset[Axis] = MeshBoundsSize[Axis] / Layout[Axis];

		for (FVector3d SlicePos = MeshBounds3d.Min + SliceOffset; SlicePos[Axis] < MeshBounds3d.Max[Axis]; SlicePos += SliceOffset)
		{
			// We need to retrieve the new MaxSubObjectID after each cut.
			for (int32 TID : EditMesh.TriangleIndicesItr())
			{
				MaxSubObjectID = FMath::Max(MaxSubObjectID, SubObjectIDs->GetValue(TID));
			}

			FMeshPlaneCut Cut(&EditMesh, SlicePos, AxisNormals[Axis]);
			Cut.CutWithoutDelete(true, 0, SubObjectIDs, MaxSubObjectID + 1);
		}
	}

	constexpr bool bSplitIfSingle = true;
	const bool bWasSplit = FDynamicMeshEditor::SplitMesh(&EditMesh, SplitMeshes, bSplitIfSingle, [SubObjectIDs](int32 TID)
		{
			return SubObjectIDs->GetValue(TID);
		});

	return SplitMeshes;
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE