// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionCreateMeshTool.h"

#include "Engine/World.h"
#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"
#include "ToolBuilderUtil.h"
#include "InteractiveToolManager.h"
#include "SceneQueries/SceneSnappingManager.h"
#include "BaseBehaviors/MouseHoverBehavior.h"
#include "Selection/ToolSelectionUtil.h"
#include "Mechanics/DragAlignmentMechanic.h"
#include "ModelingObjectsCreationAPI.h"
#include "ToolSceneQueriesUtil.h"
#include "ToolSetupUtil.h"

#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionCommonProperties.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionRectangleGenerator.h" 
#include "Modifiers/MeshPartitionMeshProvider.h"

#include "Components/StaticMeshComponent.h"
#include "Engine/StaticMeshActor.h"

#include "Generators/RectangleMeshGenerator.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "FaceGroupUtil.h"
#include "DynamicMesh/MeshTransforms.h"
#include "Operations/MeshPlaneCut.h"
#include "Spatial/SegmentTree3.h"
#include "Spatial/PointHashGrid3.h"

#include "DynamicMeshEditor.h"
#include "Editor.h"
#include "UObject/PropertyIterator.h"
#include "UObject/UnrealType.h"


#include "PackageSourceControlHelper.h"



#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionCreateMeshTool)

using namespace UE::Geometry;

#define LOCTEXT_NAMESPACE "UCreateMeshTool"


namespace UE::MeshPartition
{
namespace MegaMeshCreateMeshToolHelpers
{
	// Keep only segments that have no duplicates in the array. 
	// NOTE: this is not the same as make unique -- if there is a segment that has a double, then BOTH of those segments are removed
	static void RemoveDuplicateSegments(TArray<FSegment3d>& InOutSegments)
	{
		if (InOutSegments.Num() == 0)
		{
			return;
		}

		Geometry::TPointHashGrid3d<TPair<int32, int32>> VertexHashGrid(FMath::Max(0.1 * InOutSegments[0].Length(), 1.0), {-1, -1});
		VertexHashGrid.Reserve(InOutSegments.Num() * 2);

		for (int32 SegmentIndex = 0; SegmentIndex < InOutSegments.Num(); ++SegmentIndex)
		{
			VertexHashGrid.InsertPoint({ SegmentIndex, 0 }, InOutSegments[SegmentIndex].StartPoint());
			VertexHashGrid.InsertPoint({ SegmentIndex, 1 }, InOutSegments[SegmentIndex].EndPoint());
		}

		TArray<FSegment3d> UniqueSegments;

		for (int32 SegmentIndex = 0; SegmentIndex < InOutSegments.Num(); ++SegmentIndex)
		{
			bool bFoundDuplicate = false;

			for (const FVector3d& QueryPoint : { InOutSegments[SegmentIndex].StartPoint(), InOutSegments[SegmentIndex].EndPoint() })
			{
				TArray<TPair<int32, int32>> NearbyVertices;

				VertexHashGrid.FindPointsInBall(QueryPoint, UE_KINDA_SMALL_NUMBER,
					[&InOutSegments, &QueryPoint](const TPair<int32, int32>& Other) -> double
					{
						if (Other.Value == 0)
						{
							return FVector3d::DistSquared(InOutSegments[Other.Key].StartPoint(), QueryPoint);
						}
						else
						{
							return FVector3d::DistSquared(InOutSegments[Other.Key].EndPoint(), QueryPoint);
						}
					},
					NearbyVertices);

				for (const TPair<int32, int32>& NearbyVertex : NearbyVertices)
				{
					if (NearbyVertex.Key == SegmentIndex)
					{
						continue;
					}

					const FSegment3d& SegmentA = InOutSegments[SegmentIndex];

					const int32 OtherSegmentIndex = NearbyVertex.Key;
					const FSegment3d& SegmentB = InOutSegments[OtherSegmentIndex];

					if (SegmentA.Extent == SegmentB.Extent &&
						(SegmentA.Direction.Equals(SegmentB.Direction, UE_SMALL_NUMBER) || SegmentA.Direction.Equals(-SegmentB.Direction, UE_SMALL_NUMBER)) &&
						(SegmentA.Center.Equals(SegmentB.Center, UE_SMALL_NUMBER)))
					{
						bFoundDuplicate = true;
						break;
					}
				}

				if (bFoundDuplicate)
				{
					break;
				}
			}

			if (!bFoundDuplicate)
			{
				UniqueSegments.Add(InOutSegments[SegmentIndex]);
			}
		}

		InOutSegments = MoveTemp(UniqueSegments);
	}

}

FInt32Point UCreateRectangleToolProperties::ComputeMeshResolutionInSections() const 
{
	FInt32Point Resolution(1,1);

	if (SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Automatic)
	{
		Resolution = MeshPartition::FRectangleGeneratorUtils::ComputeSectionResolution(MeshResolution, MaxTrianglesPerSection);
	}
	
	if (SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Explicit)
	{
		if (SectionLayout.X > 0 && SectionLayout.Y > 0)
		{
			Resolution = SectionLayout;
		}
	}

	return Resolution;
}

FInt32Point UCreateRectangleToolProperties::ComputeMeshResolutionInQuads() const
{
	FInt32Point QuadResolution(0);
	
	if (SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Automatic)
	{
		QuadResolution = MeshResolution;
	}
	else if (SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Explicit)
	{
		if (SectionsResolution.X > 0 && SectionsResolution.Y > 0 && SectionLayout.X > 0 && SectionLayout.Y > 0)
		{
			QuadResolution = FInt32Point(SectionsResolution.X * SectionLayout.X, SectionsResolution.Y * SectionLayout.Y);
		
		}
	}
	return QuadResolution;
}

FInt32Point  ULocationVolumesProperties::GetLocationVolumesResolution() const
{
	return bCreateLocationVolumes ? LocationVolumesResolution : FInt32Point{ 0 };
}

/*
* ToolBuilder
*/
bool UCreateMeshToolBuilder::CanBuildTool(const FToolBuilderState& SceneState) const
{
	return true;
}

UInteractiveTool* UCreateMeshToolBuilder::BuildTool(const FToolBuilderState& SceneState) const
{
	MeshPartition::UCreateMeshTool* NewTool = nullptr;
	switch (ShapeType)
	{
	case EMakeMeshShapeType::Rectangle:
		NewTool = NewObject<MeshPartition::UCreateRectangleMeshTool>(SceneState.ToolManager);
		break;
	default:
		break;
	}
	NewTool->SetWorld(SceneState.World);
	return NewTool;
}

void UCreateMeshTool::SetWorld(UWorld* InWorld)
{
	World = InWorld;
}

UCreateMeshTool::UCreateMeshTool(const FObjectInitializer&)
{
	PlacementProperties = CreateDefaultSubobject<MeshPartition::UPlacementProperties>(TEXT("PlacementProperties"));
	// CreateDefaultSubobject automatically sets RF_Transactional flag, we need to clear it so that undo/redo doesn't affect tool properties
	PlacementProperties->ClearFlags(RF_Transactional);
}

bool UCreateMeshTool::CanAccept() const
{
	if (!World || !World->GetWorldPartition())
	{
		return false;
	}

	return CurrentState == EState::AdjustingSettings;
}

void UCreateMeshTool::Setup()
{
	USingleClickTool::Setup();

	UMouseHoverBehavior* HoverBehavior = NewObject<UMouseHoverBehavior>(this);
	HoverBehavior->Initialize(this);
	AddInputBehavior(HoverBehavior);

	CreateRectangleProperties = NewObject<MeshPartition::UCreateRectangleToolProperties>(this);
	AddToolPropertySource(CreateRectangleProperties);
	CreateRectangleProperties->RestoreProperties(this);
	CreateRectangleProperties->WatchProperty(CreateRectangleProperties->SectionsGeneration, [this](MeshPartition::ECreateRectangleSectionsGenerationMode Mode)
	{
		if (Mode == MeshPartition::ECreateRectangleSectionsGenerationMode::Explicit)
		{
			CreateRectangleProperties->CachedMeshResolution = CreateRectangleProperties->MeshResolution;
			CreateRectangleProperties->MeshResolution = CreateRectangleProperties->ComputeMeshResolutionInQuads();
		}
		else if (Mode == MeshPartition::ECreateRectangleSectionsGenerationMode::Automatic)
		{
			const FInt32Point& CachedRes = CreateRectangleProperties->CachedMeshResolution;
			if (CachedRes.X > 0 && CachedRes.Y > 0)
			{
				CreateRectangleProperties->MeshResolution = CachedRes;
			}
		}

		UpdatePreview();
	});

	CreateRectangleProperties->WatchProperty(CreateRectangleProperties->SectionLayout, [this](FInt32Point)
	{
		FInt32Point& Layout = CreateRectangleProperties->SectionLayout;
		
		if (Layout.X < 1) Layout.X = 1;
		if (Layout.Y < 1) Layout.Y = 1;
		
		if (CreateRectangleProperties->SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Explicit)
		{
			CreateRectangleProperties->MeshResolution = CreateRectangleProperties->ComputeMeshResolutionInQuads();
		}
		UpdatePreview();
	});
	
	CreateRectangleProperties->WatchProperty(CreateRectangleProperties->SectionsResolution, [this](FInt32Point)
	{
		FInt32Point& Resolution = CreateRectangleProperties->SectionsResolution;

		if (Resolution.X < 1) Resolution.X = 1;
		if (Resolution.Y < 1) Resolution.Y = 1;
		
		if (CreateRectangleProperties->SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Explicit)
		{
			CreateRectangleProperties->MeshResolution = CreateRectangleProperties->ComputeMeshResolutionInQuads();
		}
		UpdatePreview();
	});


	AddToolPropertySource(PlacementProperties);

	PlacementProperties->WatchProperty(PlacementProperties->TargetSurface, [this](MeshPartition::ECreateMeshPlacementType) 
	{
		UpdateTargetSurface(); 
	});
	PlacementProperties->WatchProperty(PlacementProperties->bSnapToExistingMegaMesh, [this](bool) 
	{
		UpdatePreview();
		InitSnapTarget();
	});
	PlacementProperties->WatchProperty(PlacementProperties->CornerThresholdDegrees, [this](float)
	{
		UpdatePreview();
		InitSnapTarget();
	});
	

	PlacementProperties->RestoreProperties(this);

	
	MegaMeshCreateProperties = NewObject<MeshPartition::UCreateProperties>(this);
	MegaMeshCreateProperties->WatchProperty(  MegaMeshCreateProperties->ExistingMegaMesh, [this](TObjectPtr<AMeshPartition> NewPtr)
	{
		UpdatePreview();
		InitSnapTarget();
	});

	AddToolPropertySource(  MegaMeshCreateProperties);
	MegaMeshCreateProperties->RestoreProperties(this);
	

	// create preview mesh object
	PreviewMesh = NewObject<UPreviewMesh>(this);
	PreviewMesh->CreateInWorld(World.Get(), FTransform::Identity);
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);
	PreviewMesh->SetVisible(false);
	

	UTransformProxy* TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->OnTransformChanged.AddWeakLambda(this, [this](UTransformProxy*, FTransform NewTransform)
	{
		FTransform SnappedTransform = NewTransform;
		if (  MegaMeshCreateProperties->ExistingMegaMesh && PlacementProperties->bSnapToExistingMegaMesh)
		{
			FVector3d SnapOffsetVector;
			if (SnapPreviewMeshToExisting(SnappedTransform, SnapOffsetVector))
			{
				SnappedTransform.AddToTranslation(SnapOffsetVector);
			}
		}

		PreviewMesh->SetTransform(SnappedTransform);
	});

	// TODO: It might be nice to use a repositionable gizmo, but the drag alignment mechanic can't currently 
	// hit the preview mesh, which makes middle click repositioning feel broken and not very useful.
	Gizmo = UE::TransformGizmoUtil::CreateCustomTransformGizmo(GetToolManager(),
		ETransformGizmoSubElements::StandardTranslateRotate, this);
	Gizmo->SetActiveTarget(TransformProxy, GetToolManager());

	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);
	DragAlignmentMechanic->AddToGizmo(Gizmo);

	UpdatePreview();
	SetState(EState::PlacingPrimitive);
	UpdateDisplayMessage();
}

void UCreateMeshTool::SetState(EState NewState)
{
	CurrentState = NewState;

	bool bGizmoActive = (CurrentState == EState::AdjustingSettings);
	Gizmo->SetVisibility(bGizmoActive && PlacementProperties->bShowGizmo);
	PlacementProperties->bShowGizmoOptions = bGizmoActive;
	NotifyOfPropertyChangeByTool(PlacementProperties);

	if (CurrentState == EState::PlacingPrimitive)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartAddPrimitiveTool", "This Tool creates new shapes. Click in the scene to choose initial placement of mesh."),
			EToolMessageLevel::UserNotification);
	}
	else
	{
		// Initialize gizmo to current preview location
		Gizmo->ReinitializeGizmoTransform(PreviewMesh->GetTransform());

		GetToolManager()->DisplayMessage(
			LOCTEXT("OnStartAddPrimitiveTool2", "Alter shape settings in the detail panel or modify placement with gizmo, then accept the tool."),
			EToolMessageLevel::UserNotification);
	}
}


void UCreateMeshTool::Shutdown(EToolShutdownType ShutdownType)
{
	if (ShutdownType == EToolShutdownType::Accept)
	{
		GenerateAsset();
	}

	DragAlignmentMechanic->Shutdown();
	DragAlignmentMechanic = nullptr;
	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	Gizmo = nullptr;

	PreviewMesh->SetVisible(false);
	PreviewMesh->Disconnect();
	PreviewMesh = nullptr;

	// restore the meshresolution if needed before saving
	{
		if (CreateRectangleProperties->SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Explicit)
		{
			CreateRectangleProperties->MeshResolution = CreateRectangleProperties->CachedMeshResolution;
		}
	}
	CreateRectangleProperties->SaveProperties(this);
	PlacementProperties->SaveProperties(this);
	MegaMeshCreateProperties->SaveProperties(this);
}

void UCreateMeshTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);

	if (PlacementProperties->bSnapToExistingMegaMesh && PlacementProperties->bDrawSnapTargets)
	{
		for (const FSegment3d& Seg : SnapTargetSegments)
		{
			RenderAPI->GetPrimitiveDrawInterface()->DrawLine(Seg.StartPoint(), Seg.EndPoint(), FLinearColor::Green, SDPG_World);
		}
		if (PreviewMesh)
		{
			const FTransform PreviewTransform = PreviewMesh->GetTransform();
			for (const FSegment3d& Segment : PreviewBoundarySegments)
			{
				RenderAPI->GetPrimitiveDrawInterface()->DrawLine(PreviewTransform.TransformPosition(Segment.StartPoint()),
					PreviewTransform.TransformPosition(Segment.EndPoint()),
					FLinearColor::Yellow, 
					SDPG_World);
			}
		}
	}
}

void UCreateMeshTool::InitSnapTarget()
{
	SnapTargetSegments.Reset();

	if (PlacementProperties->bSnapToExistingMegaMesh &&   MegaMeshCreateProperties->ExistingMegaMesh != nullptr)
	{
		if (const UMeshPartitionEditorComponent* const EditorComponent = Cast<UMeshPartitionEditorComponent>(  MegaMeshCreateProperties->ExistingMegaMesh->GetMeshPartitionComponent()))
		{
			// Find existing mesh provider components and compute their boundaries
			const TArray<MeshPartition::UModifierComponent*> MeshProviders = EditorComponent->GetModifiersFiltered([](const MeshPartition::UModifierComponent* Component) -> bool
			{
				return Component->IsA<MeshPartition::UMeshProviderModifier>();
			});

			for (const MeshPartition::UModifierComponent* const Component : MeshProviders)
			{
				if (const MeshPartition::UMeshProviderModifier* const MeshProvider = Cast<MeshPartition::UMeshProviderModifier>(Component))
				{
					const FDynamicMesh3* const SnapTargetMesh = MeshProvider->GetMesh();
					const FTransform SnapTargetMeshTransform = MeshProvider->GetComponentTransform();

					constexpr bool bAutoCompute = false;
					Geometry::FMeshBoundaryLoops BoundaryLoops(SnapTargetMesh, bAutoCompute);
					BoundaryLoops.bOnlyComputeSpans = true;
					BoundaryLoops.Compute();

					for (const FEdgeSpan& Span : BoundaryLoops.Spans)
					{
						TArray<FEdgeSpan> CurrentNewSpans;
						Span.GetSubspansByAngle(PlacementProperties->CornerThresholdDegrees, 1, CurrentNewSpans);

						for (const FEdgeSpan& NewSpan : CurrentNewSpans)
						{
							const FSegment3d NewSegment(SnapTargetMeshTransform.TransformPosition(SnapTargetMesh->GetVertex(NewSpan.Vertices[0])),
								SnapTargetMeshTransform.TransformPosition(SnapTargetMesh->GetVertex(NewSpan.Vertices.Last())));

							SnapTargetSegments.Add(NewSegment);
						}
					}
				}
			}

			// Remove interior boundaries of the MegaMesh (i.e. boundaries between sections). To do this, we keep the segments that appear only once in the list
			MegaMeshCreateMeshToolHelpers::RemoveDuplicateSegments(SnapTargetSegments);
	
			SnapTargetSegmentSpatial = MakeUnique<FSegmentTree3>();
			SnapTargetSegmentSpatial->Build(SnapTargetSegments);
		}
	}
}


void UCreateMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	// Because of how the PlacementProperties property set is implemented in this Tool, changes to it are transacted,
	// and if the user exits the Tool and then tries to undo/redo those transactions, this function will end up being called.
	// So we need to ensure that we handle this case.
	if (PreviewMesh)
	{
		UpdatePreview();
	}

	if (Gizmo)
	{
		Gizmo->SetVisibility(CurrentState == EState::AdjustingSettings && PlacementProperties->bShowGizmo);
	}

	if (Property && (Property->GetFName() == GET_MEMBER_NAME_CHECKED(UCreateProperties, ExistingMegaMesh)))
	{
		PlacementProperties->bEnableSnapOptions = (  MegaMeshCreateProperties->ExistingMegaMesh != nullptr);
		NotifyOfPropertyChangeByTool(PlacementProperties);

		InitSnapTarget();
	}

	Super::OnPropertyModified(PropertySet, Property);
}




FInputRayHit UCreateMeshTool::BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos)
{
	FInputRayHit Result(0);
	Result.bHit = (CurrentState == EState::PlacingPrimitive);
	return Result;
}

void UCreateMeshTool::OnBeginHover(const FInputDeviceRay& DevicePos)
{
	UpdatePreviewPosition(DevicePos);
}

bool UCreateMeshTool::OnUpdateHover(const FInputDeviceRay& DevicePos)
{
	UpdatePreviewPosition(DevicePos);
	return true;
}

void UCreateMeshTool::OnEndHover()
{
	// do nothing
}


bool UCreateMeshTool::SnapPreviewMeshToExisting(const FTransform& ShapeTransform, FVector3d& OutSnapOffset) const
{
	auto SegmentSegmentDistance = [](const FSegment3d& SegmentA, const FSegment3d& SegmentB, FVector3d& OutSeparationVector) -> double
	{
		FVector3d ClosestOnA;
		FVector3d ClosestOnB;
		FMath::SegmentDistToSegmentSafe(SegmentA.StartPoint(), SegmentA.EndPoint(), SegmentB.StartPoint(), SegmentB.EndPoint(), ClosestOnA, ClosestOnB);
		OutSeparationVector = ClosestOnA - ClosestOnB;
		return OutSeparationVector.Length();
	};

	double MinDist = UE_BIG_NUMBER;
	FVector3d MinSeparationVector = FVector3d::Zero();

	// TODO: Use two spatial acceleration structures to get the closest pair of segments. We would need a new function in FSegmentTree3 similar to TMeshAABBTree's FindNearestTriangles().
	// If this was performant enough we might even get away with using all boundary edges, rather then having to compute spans and convert them to segments.

	for (int32 PreviewSegmentID = 0; PreviewSegmentID < PreviewBoundarySegments.Num(); ++PreviewSegmentID)
	{
		const FSegment3d RawPreviewSegment = PreviewBoundarySegments[PreviewSegmentID];
		const FVector3d PSA = ShapeTransform.TransformPosition(RawPreviewSegment.StartPoint());
		const FVector3d PSB = ShapeTransform.TransformPosition(RawPreviewSegment.EndPoint());
		const FSegment3d PreviewSegment(PSA, PSB);

		if (SnapTargetSegmentSpatial)
		{
			const double AlignmentDotProductThreshold = FMath::Cos(FMath::DegreesToRadians(PlacementProperties->AlignmentThreshold));

			auto EdgeFilter = [&PreviewSegment, AlignmentDotProductThreshold, this](int SegmentIndex) -> bool
			{
				const double DotProduct = PreviewSegment.Direction.Dot(SnapTargetSegments[SegmentIndex].Direction);
				return (FMath::Abs(DotProduct) >= AlignmentDotProductThreshold);
			};
			IMeshSpatial::FQueryOptions Options;
			Options.TriangleFilterF = EdgeFilter;
			FSegmentTree3::FSegment QueryResult;
			if (SnapTargetSegmentSpatial->FindNearestSegment(PreviewSegment.Center, QueryResult, Options))
			{
				FVector3d Separation;
				const double CurrDist = SegmentSegmentDistance(QueryResult.Segment, PreviewSegment, Separation);

				if (CurrDist < MinDist)
				{
					MinDist = CurrDist;
					MinSeparationVector = Separation;

					// TODO: Also compute the rotation snap if requested
				}
			}
		}
	}

	if (MinDist < PlacementProperties->DistanceThreshold)
	{
		OutSnapOffset = MinSeparationVector;
		return true;
	}

	return false;
}


void UCreateMeshTool::UpdatePreviewPosition(const FInputDeviceRay& DeviceClickPos)
{
	FRay ClickPosWorldRay = DeviceClickPos.WorldRay;

	// hit position (temp)
	bool bHit = false;

	auto RaycastPlaneWithFallback = [](const FVector3d& Origin, const FVector3d& Direction, const FPlane& Plane, double FallbackDistance = 1000) -> FVector3d
		{
			double IntersectTime = FMath::RayPlaneIntersectionParam(Origin, Direction, Plane);
			if (IntersectTime < 0 || !FMath::IsFinite(IntersectTime))
			{
				IntersectTime = FallbackDistance;
			}
			return Origin + Direction * IntersectTime;
		};

	FPlane DrawPlane(FVector::ZeroVector, FVector(0, 0, 1));
	if (PlacementProperties->TargetSurface == MeshPartition::ECreateMeshPlacementType::GroundPlane)
	{
		FVector3d DrawPlanePos = RaycastPlaneWithFallback(ClickPosWorldRay.Origin, ClickPosWorldRay.Direction, DrawPlane);
		bHit = true;
		ShapeFrame = FFrame3d(DrawPlanePos);
	}
	else if (PlacementProperties->TargetSurface == MeshPartition::ECreateMeshPlacementType::OnScene)
	{
		// cast ray into scene
		FHitResult Result;
		bHit = ToolSceneQueriesUtil::FindNearestVisibleObjectHit(this, Result, ClickPosWorldRay);
		if (bHit)
		{
			FVector3d Normal = (FVector3d)Result.ImpactNormal;
			if (!PlacementProperties->bAlignToNormal)
			{
				Normal = FVector3d::UnitZ();
			}
			ShapeFrame = FFrame3d((FVector3d)Result.ImpactPoint, Normal);
			ShapeFrame.ConstrainedAlignPerpAxes();
		}
		else
		{
			// fall back to ground plane if we don't have a scene hit
			FVector3d DrawPlanePos = RaycastPlaneWithFallback(ClickPosWorldRay.Origin, ClickPosWorldRay.Direction, DrawPlane);
			bHit = true;
			ShapeFrame = FFrame3d(DrawPlanePos);
		}
	}
	else
	{
		bHit = true;
		ShapeFrame = FFrame3d();
	}


	bool bSnapped = false;
	if (PlacementProperties->bSnapToExistingMegaMesh &&   MegaMeshCreateProperties->ExistingMegaMesh)
	{
		FVector3d SnapOffsetVector;
		if (SnapPreviewMeshToExisting(ShapeFrame.ToFTransform(), SnapOffsetVector))
		{
			bSnapped = true;
			ShapeFrame.Origin += SnapOffsetVector;
		}
	}

	if (!bSnapped)
	{
		// Snap to grid
		USceneSnappingManager* SnapManager = USceneSnappingManager::Find(GetToolManager());
		if (SnapManager)
		{
			FSceneSnapQueryRequest Request;
			Request.RequestType = ESceneSnapQueryType::Position;
			Request.TargetTypes = ESceneSnapQueryTargetType::Grid;
			Request.Position = (FVector)ShapeFrame.Origin;
			TArray<FSceneSnapQueryResult> Results;
			if (SnapManager->ExecuteSceneSnapQuery(Request, Results))
			{
				ShapeFrame.Origin = (FVector3d)Results[0].Position;
			}
		}

		if (PlacementProperties->Rotation != 0)
		{
			ShapeFrame.Rotate(FQuaterniond(ShapeFrame.Z(), PlacementProperties->Rotation, true));
		}
	}

	if (bHit)
	{
		PreviewMesh->SetVisible(true);
		PreviewMesh->SetTransform(ShapeFrame.ToFTransform());
	}
	else
	{
		PreviewMesh->SetVisible(false);
	}
}

void UCreateMeshTool::UpdatePreview()
{
	FDynamicMesh3 NewMesh;
	GeneratePreviewMesh(&NewMesh);

	FaceGroupUtil::SetGroupID(NewMesh, 0);

	// set mesh position
	const FAxisAlignedBox3d Bounds = NewMesh.GetBounds(true);
	FVector3d TargetOrigin = ShouldCenterXY() ? Bounds.Center() : FVector3d(0.);

	for (const int Vid : NewMesh.VertexIndicesItr())
	{
		FVector3d Pos = NewMesh.GetVertex(Vid);
		Pos -= TargetOrigin;
		NewMesh.SetVertex(Vid, Pos);
	}

	PreviewMesh->UpdatePreview(&NewMesh);

	PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	const bool CalculateTangentsSuccessful = PreviewMesh->CalculateTangents();
	checkSlow(CalculateTangentsSuccessful);
	PreviewMesh->SetVisible(true);

	if (PlacementProperties->bSnapToExistingMegaMesh &&   MegaMeshCreateProperties->ExistingMegaMesh)
	{
		// Compute boundary edge spans for snapping
		PreviewBoundarySegments.Reset();

		constexpr bool bAutoCompute = false;
		Geometry::FMeshBoundaryLoops BoundaryLoops(PreviewMesh->GetMesh(), bAutoCompute);
		BoundaryLoops.bOnlyComputeSpans = true;
		BoundaryLoops.Compute();

		for (const FEdgeSpan& Span : BoundaryLoops.Spans)
		{
			TArray<FEdgeSpan> CurrentNewSpans;
			Span.GetSubspansByAngle(PlacementProperties->CornerThresholdDegrees, 4, CurrentNewSpans);

			for (const FEdgeSpan& NewSpan : CurrentNewSpans)
			{
				PreviewBoundarySegments.Add({ PreviewMesh->GetMesh()->GetVertex(NewSpan.Vertices[0]), PreviewMesh->GetMesh()->GetVertex(NewSpan.Vertices.Last()) });
			}
		}
	}
	UpdateDisplayMessage();

}

void UCreateMeshTool::UpdateDisplayMessage() const
{
	UInteractiveToolManager* Manager = GetToolManager();

	// If the tool can't run in this level, that message takes priority
	if (!World || !World->GetWorldPartition())
	{
		Manager->DisplayMessage(
			LOCTEXT("MeshPartition_WorldPartitionRequired", "Mesh Partition requires World Partition to be enabled."),
			EToolMessageLevel::UserError);
		return;
	}

	FText Message;
	BuildDisplayMessage(Message);
	// Always call DisplayMessage, even with empty text, so stale messages are cleared.
	Manager->DisplayMessage(Message, EToolMessageLevel::UserWarning);
}

void UCreateMeshTool::UpdateTargetSurface()
{
	if (PlacementProperties->TargetSurface == MeshPartition::ECreateMeshPlacementType::AtOrigin)
	{
		// default ray is used as coordinates will not be needed to set position at origin
		const FInputDeviceRay DefaultRay = FInputDeviceRay();
		UpdatePreviewPosition(DefaultRay);

		SetState(EState::AdjustingSettings);
		GetToolManager()->EmitObjectChange(this, MakeUnique<FStateChange>(PreviewMesh->GetTransform()),
			LOCTEXT("PlaceMeshTransaction", "Place Mesh"));
	}
	else
	{
		SetState(EState::PlacingPrimitive);
	}
}



bool UCreateMeshTool::SupportsWorldSpaceFocusBox()
{
	return PreviewMesh != nullptr;
}

FBox UCreateMeshTool::GetWorldSpaceFocusBox()
{
	if (PreviewMesh)
	{
		if (UPrimitiveComponent* Component = PreviewMesh->GetRootComponent())
		{
			return Component->Bounds.GetBox();
		}
	}
	return FBox();
}

bool UCreateMeshTool::SupportsWorldSpaceFocusPoint()
{
	return PreviewMesh != nullptr;

}

bool UCreateMeshTool::GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut)
{
	if (PreviewMesh)
	{
		FHitResult HitResult;
		if (PreviewMesh->FindRayIntersection(WorldRay, HitResult))
		{
			PointOut = HitResult.ImpactPoint;
			return true;
		}
	}
	return false;
}


FInputRayHit UCreateMeshTool::IsHitByClick(const FInputDeviceRay& ClickPos)
{
	FInputRayHit Result(0);
	Result.bHit = (CurrentState == EState::PlacingPrimitive);
	return Result;
}

void UCreateMeshTool::OnClicked(const FInputDeviceRay& ClickPos)
{
	if (!ensure(CurrentState == EState::PlacingPrimitive))
	{
		return;
	}

	UpdatePreviewPosition(ClickPos);
	SetState(EState::AdjustingSettings);
	GetToolManager()->EmitObjectChange(this, MakeUnique<FStateChange>(PreviewMesh->GetTransform()),
		LOCTEXT("PlaceMeshTransaction", "Place Mesh"));
}



TArray<TUniquePtr<FDynamicMesh3>> UCreateMeshTool::SplitMesh(const FDynamicMesh3& InMesh, const FVector3d SectionsPerSide)
{
	FDynamicMesh3 EditMesh(InMesh);

	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
	int32 MaxSubObjectID = -1;
	const FBox MeshBounds3d = FBox(EditMesh.GetBounds());
	const FVector MeshBoundsSize = MeshBounds3d.GetSize();
	FVector3d SliceNormal(1.0, 0.0, 0.0);
	FVector3d SliceOffset(MeshBoundsSize.X / SectionsPerSide.X, 0.f, 0.f);


	// Setup the dynamic mesh with attributes needed by FMeshPlaneCut and Geometry::FDynamicMeshEditor::SplitMesh
	EditMesh.EnableAttributes();
	TDynamicMeshScalarTriangleAttribute<int32>* SubObjectIDs = new TDynamicMeshScalarTriangleAttribute<int32>(&EditMesh);
	SubObjectIDs->Initialize(0);
	EditMesh.Attributes()->AttachAttribute(TEXT("ObjectIndexAttribute"), SubObjectIDs);

	if (SectionsPerSide.X > 0)
	{
		for (FVector3d SlicePos = MeshBounds3d.Min + SliceOffset; SlicePos.X < MeshBounds3d.Max.X; SlicePos += SliceOffset)
		{
			// We need to retrieve the new MaxSubObjectID after each cut.
			for (int32 TID : EditMesh.TriangleIndicesItr())
			{
				MaxSubObjectID = FMath::Max(MaxSubObjectID, SubObjectIDs->GetValue(TID));
			}

			FMeshPlaneCut Cut(&EditMesh, SlicePos, SliceNormal);
			Cut.CutWithoutDelete(true, 0, SubObjectIDs, MaxSubObjectID + 1);
		}
	}

	if (SectionsPerSide.Y > 0)
	{
		SliceOffset = FVector3d(0.f, MeshBoundsSize.Y / SectionsPerSide.Y, 0.f);
		SliceNormal = FVector3d(0.0, 1.0, 0.0);

		for (FVector3d SlicePos = MeshBounds3d.Min + SliceOffset; SlicePos.Y < MeshBounds3d.Max.Y; SlicePos += SliceOffset)
		{
			// We need to retrieve the new MaxSubObjectID after each cut.
			for (int32 TID : EditMesh.TriangleIndicesItr())
			{
				MaxSubObjectID = FMath::Max(MaxSubObjectID, SubObjectIDs->GetValue(TID));
			}

			FMeshPlaneCut Cut(&EditMesh, SlicePos, SliceNormal);
			Cut.CutWithoutDelete(true, 0, SubObjectIDs, MaxSubObjectID + 1);
		}
	}


	if (SectionsPerSide.Z > 0)
	{
		SliceOffset = FVector3d(0.f, 0.f, MeshBoundsSize.Z / SectionsPerSide.Z);
		SliceNormal = FVector3d(0.0, 0.0, 1.0);

		for (FVector3d SlicePos = MeshBounds3d.Min + SliceOffset; SlicePos.Z < MeshBounds3d.Max.Z; SlicePos += SliceOffset)
		{
			// We need to retrieve the new MaxSubObjectID after each cut.
			for (int32 TID : EditMesh.TriangleIndicesItr())
			{
				MaxSubObjectID = FMath::Max(MaxSubObjectID, SubObjectIDs->GetValue(TID));
			}

			FMeshPlaneCut Cut(&EditMesh, SlicePos, SliceNormal);
			Cut.CutWithoutDelete(true, 0, SubObjectIDs, MaxSubObjectID + 1);
		}
	}

	constexpr bool bSplitIfSingle = true;
	const bool bWasSplit = FDynamicMeshEditor::SplitMesh(&EditMesh, SplitMeshes, bSplitIfSingle, [SubObjectIDs](int32 TID)
		{
			return SubObjectIDs->GetValue(TID);
		});

	return SplitMeshes;
}

void UCreateMeshTool::FStateChange::Apply(UObject* Object)
{
	UCreateMeshTool* Tool = Cast<MeshPartition::UCreateMeshTool>(Object);

	// Set preview transform before changing state so that the adjustment gizmo is initialized properly
	Tool->PreviewMesh->SetTransform(MeshTransform);

	Tool->SetState(EState::AdjustingSettings);
}

void UCreateMeshTool::FStateChange::Revert(UObject* Object)
{
	MeshPartition::UCreateMeshTool* Tool = Cast<UCreateMeshTool>(Object);
	Tool->SetState(EState::PlacingPrimitive);
}

UCreateRectangleMeshTool::UCreateRectangleMeshTool(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	AssetName = TEXT("Rectangle");
	UInteractiveTool::SetToolDisplayName(LOCTEXT("MegaMeshRectToolName", "Create Mesh Partition Rectangle"));
}

void UCreateRectangleMeshTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	Super::OnPropertyModified(PropertySet, Property);
}


void UCreateRectangleMeshTool::GeneratePreviewMesh(FDynamicMesh3* OutMesh) const
{

	FRectangleMeshGenerator RectGen;
	{
		RectGen.Width = CreateRectangleProperties->MeshSize.X;
		RectGen.Height = CreateRectangleProperties->MeshSize.Y;
	}
 
	RectGen.WidthVertexCount = 2;
	RectGen.HeightVertexCount = 2;
	RectGen.bSinglePolyGroup = true;

	RectGen.Generate();
	OutMesh->Copy(&RectGen);
}

void UCreateRectangleMeshTool::UpdatePreview()
{
	Super::UpdatePreview();

	// update the section info used for rendering the proxy
	const FInt32Point SectionResolution = CreateRectangleProperties->ComputeMeshResolutionInSections();
	SectionInfos = MeshPartition::FRectangleGeneratorUtils::ComputeSectionInfos(CreateRectangleProperties->MeshResolution, SectionResolution);

}
void UCreateRectangleMeshTool::RenderSections(IToolsContextRenderAPI* RenderAPI) const
{
	if (!RenderAPI) return;

	const FTransform PreviewTransform = PreviewMesh->GetTransform(); 

	if (SectionInfos.Num() > 0)
	{
		FPrimitiveDrawInterface* const PDI = RenderAPI->GetPrimitiveDrawInterface();

		const FVector2d MeshOffset = -CreateRectangleProperties->MeshSize *0.5;

		auto DrawLine = [PDI, &PreviewTransform, &MeshOffset, &MeshSize = CreateRectangleProperties->MeshSize](const FVector2d& UVStart, const FVector2d& UVEnd)
			{
				const FVector3d Start = { MeshOffset.X + UVStart.X * MeshSize.X, MeshOffset.Y + UVStart.Y * MeshSize.Y, 0.0 };
				const FVector3d End = { MeshOffset.X + UVEnd.X * MeshSize.X, MeshOffset.Y + UVEnd.Y * MeshSize.Y, 0.0 };

				PDI->DrawLine(PreviewTransform.TransformPosition(Start), PreviewTransform.TransformPosition(End), FLinearColor::Yellow, SDPG_World);
			};

		// The assumption is that the sections form a grid, and therefore we only need to render the vertical/horizontal lines of the grid.

		DrawLine({ 0.0, 0.0 }, { 1.0, 0.0 });
		DrawLine({ 0.0, 0.0 }, { 0.0, 1.0 });

		for (int32 SectionIndex = 0; SectionIndex < SectionInfos.Num(); ++SectionIndex)
		{
			const FSectionInfo& SectionInfo = SectionInfos[SectionIndex];

			if (SectionInfo.IndexXY.X == 0)
			{
				DrawLine({ 0.0, SectionInfo.MaxUV.Y }, { 1.0, SectionInfo.MaxUV.Y });
			}

			if (SectionInfo.IndexXY.Y == 0)
			{
				DrawLine({ SectionInfo.MaxUV.X, 0.0 }, { SectionInfo.MaxUV.X, 1.0 });
			}
		}
	}
}

void UCreateRectangleMeshTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	MeshPartition::UCreateMeshTool::Render(RenderAPI);

	RenderSections(RenderAPI);
}

void UCreateRectangleMeshTool::GenerateAsset()
{
	// Save-and-unload writes packages and unloads actors, which isn't reversible by undo.
	// Skip the transaction wrap and reset the undo buffer afterwards instead, to not leave stranded assets on disk.
	const bool bSaveAndUnload = CreateRectangleProperties->bSaveAndUnload;

	if (!bSaveAndUnload)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshPartitionCreateRectangleTransaction", "Create Mesh Partition Rectangle"));
	}

	const FTransform PreviewTransform = PreviewMesh->GetTransform();

	AMeshPartition* ExistingMegaMesh = MegaMeshCreateProperties->ExistingMegaMesh;

	FMegaMeshRectangleGenerator RectangleGenerator(GetGeneratorParams(), *GetWorld(), PreviewTransform, ExistingMegaMesh);

	FPackageSourceControlHelper SourceControlHelper;
	if (!RectangleGenerator.Generate(&SourceControlHelper))
	{
		GetToolManager()->DisplayMessage(RectangleGenerator.GetErrorText(), EToolMessageLevel::UserError);
	}

	if (!bSaveAndUnload)
	{
		GetToolManager()->EndUndoTransaction();
	}
	else if (GEditor)
	{
		GEditor->ResetTransaction(LOCTEXT("MeshPartitionCreateRectangleResetTransaction", "Create Mesh Partition Rectangle (Save and Unload)"));
	}
}

bool UCreateRectangleMeshTool::IsConfigurationValid(FText* OutErrorMessage) const
{
	// For Explicit section generation, validate settings are within MaxTrianglesPerSection
	if (CreateRectangleProperties->SectionsGeneration == MeshPartition::ECreateRectangleSectionsGenerationMode::Explicit)
	{
		const FInt32Point& SectionsResolution = CreateRectangleProperties->SectionsResolution;
		const int64 PerSectionTris = 2LL * static_cast<int64>(SectionsResolution.X) * static_cast<int64>(SectionsResolution.Y);
		const int64 MaxAllowed = static_cast<int64>(CreateRectangleProperties->MaxTrianglesPerSection);

		if (PerSectionTris > MaxAllowed)
		{
			if (OutErrorMessage)
			{
				*OutErrorMessage = FText::Format(LOCTEXT("PerSectionTooLarge",
					"Per-section triangle count ({0}) exceeds Max Triangles ({1}). Reduce Resolution or raise Max Triangles."),
					FText::AsNumber(PerSectionTris), FText::AsNumber(MaxAllowed));
			}
			return false;
		}
	}

	if (CreateRectangleProperties->bSaveAndUnload && World && World->GetPackage()->GetLoadedPath().IsEmpty())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("SaveLevelFirst",
				"The current level must be saved before using \"Save and Unload\". Please save the level and try again.");
		}
		return false;
	}

	return true;
}

bool UCreateRectangleMeshTool::CanAccept() const
{
	return Super::CanAccept() && IsConfigurationValid();
}

void UCreateRectangleMeshTool::Shutdown(EToolShutdownType InShutdownType)
{
	// clean up the gizmos
	MeshPartition::UCreateMeshTool::Shutdown(InShutdownType);  	
}



void UCreateRectangleMeshTool::BuildDisplayMessage(FText& OutMessage) const
{
	// Configuration error blocks accept, so takes priority over other messages
	if (!IsConfigurationValid(&OutMessage))
	{
		return;
	}

	const int64 Threshold = 2048LL * 2048 * 2;
	const int64 ThresholdSaveAndUnload = Threshold * 4 * 4;

	const bool bSaveAndUnloadEnabled = CreateRectangleProperties->bSaveAndUnload;
	const FInt32Point QuadResolution = CreateRectangleProperties->ComputeMeshResolutionInQuads();
	const int64 NumTriangles = 2LL * static_cast<int64>(QuadResolution.X) * static_cast<int64>(QuadResolution.Y);

	if (!bSaveAndUnloadEnabled && NumTriangles >= Threshold)
	{
		OutMessage = FText::Format(
			LOCTEXT("LargeResolution", "Imported mesh will consist of {0} triangles.{1}"),
			FText::AsNumber(NumTriangles),
			LOCTEXT("EnableSaveAndUnload", " Consider enabling \"Save and Unload\" to limit memory consumption."));
	}

	if (bSaveAndUnloadEnabled && NumTriangles >= ThresholdSaveAndUnload)
	{
		OutMessage = FText::Format(
			LOCTEXT("LargeResolutionSaveAndUnload", "Imported mesh will consist of {0} triangles."),
			FText::AsNumber(NumTriangles));
	}
}

FMegaMeshRectangleGeneratorParams UCreateRectangleMeshTool::GetGeneratorParams() const
{
	FMegaMeshRectangleGeneratorParams Params;
	Params.MeshResolution = CreateRectangleProperties->MeshResolution;
	Params.MeshSize = CreateRectangleProperties->MeshSize;
	Params.SectionsResolution = CreateRectangleProperties->ComputeMeshResolutionInSections();
	Params.bSaveAndUnload = CreateRectangleProperties->bSaveAndUnload;
	Params.LocationVolumesResolution = FIntPoint(0,0); // no location volumes
	
	return Params;
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE

