// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionExpandTool.h"

#include "InteractiveToolManager.h"
#include "MeshPartitionComponentBackedTarget.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"
#include "MeshBoundaryLoops.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/MeshNormals.h"
#include "DynamicSubmesh3.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "Selection/BoundarySelectionMechanic.h"
#include "PreviewMesh.h"
#include "ToolSetupUtil.h"
#include "Operations/ExtrudeBoundaryEdges.h"
#include "CompositionOps/MirrorOp.h"
#include "Util/ColorConstants.h"
#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionModelingToolsetModule.h" // LogMegaMeshModelingToolset
#include "Parameterization/DynamicMeshUVEditor.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionExpandTool)

#define LOCTEXT_NAMESPACE "MegaMeshExpandTool"

using namespace UE::Geometry;

namespace UE::MeshPartition
{
// -------------------------------------------------------------------------------------------------------------------------

namespace MegaMeshExpandToolHelpers
{
	// Find edges that are at either the beginning or end of a selected span, and adjacent to another edge at the beginning or end of another selected span
	// If such a pair is found, create a new GroupID for the edge pair, and associate each edge with that GroupID. This will allow a new polygon to be extruded from the common span corner.
	// GroupID is modified and returned by this function
	static void FindAdjacentSpanEndEdges(const TArray<FEdgeSpan>& Spans, const TArray<int32>& SelectedSpans, int32& InOutGroupID, TMap<int32, int32>& OutEdgeToGroupMap)
	{
		TMap<int32, TArray<int32>> VertexToIncidentEdges;
		for (const int32 SelectedSpanIndex : SelectedSpans)
		{
			const FEdgeSpan& Span = Spans[SelectedSpanIndex];
			VertexToIncidentEdges.FindOrAdd(Span.Vertices[0]).Add(Span.Edges[0]);
			VertexToIncidentEdges.FindOrAdd(Span.Vertices.Last()).Add(Span.Edges.Last());
		}

		for (const TPair<int32, TArray<int32>>& VertexEdge : VertexToIncidentEdges)
		{
			if (VertexEdge.Value.Num() > 1)
			{
				for (const int32 EdgeID : VertexEdge.Value)
				{
					OutEdgeToGroupMap.Add(EdgeID, InOutGroupID);
				}
				++InOutGroupID;
			}
		}
	}


}	// namespace 


// -------------------------------------------------------------------------------------------------------------------------

const FToolTargetTypeRequirements& UExpandToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshProvider::StaticClass(),
			UPrimitiveComponentBackedTarget::StaticClass(),
			UMeshPartitionComponentBackedTarget::StaticClass(),
		});

	return TypeRequirements;
}

bool UExpandToolBuilder::CanBuildTool(const FToolBuilderState& InSceneState) const
{
	const bool bTargetRequirementsMet = (InSceneState.TargetManager->CountSelectedAndTargetable(InSceneState, GetTargetRequirements()) > 0);

	if (!bTargetRequirementsMet)
	{
		return false;
	}

	// Valid if we are selecting one AMeshPartition with a UMeshPartitionEditorComponent
	TSet<const AMeshPartition*> SelectedMegaMeshActors;

	InSceneState.TargetManager->EnumerateSelectedAndTargetableComponents(InSceneState, GetTargetRequirements(), [&SelectedMegaMeshActors](UActorComponent* Component)
	{
		const AActor* ComponentOwner = Component->GetOwner();
		while (ComponentOwner)
		{
			if (const AMeshPartition* const MegaMeshActor = Cast<AMeshPartition>(ComponentOwner))
			{
				if (const UMeshPartitionEditorComponent* const MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMeshActor->GetMeshPartitionComponent()))
				{
					SelectedMegaMeshActors.Add(MegaMeshActor);
					return;
				}
			}
			ComponentOwner = ComponentOwner->GetAttachParentActor();
		}
		SelectedMegaMeshActors.Add(nullptr);		// this component not owned by an AMeshPartition
	});

	return SelectedMegaMeshActors.Num() == 1;
}

UMultiSelectionMeshEditingTool* UExpandToolBuilder::CreateNewTool(const FToolBuilderState& InSceneState) const
{
	return NewObject<MeshPartition::UExpandTool>(InSceneState.ToolManager);
}


// -------------------------------------------------------------------------------------------------------------------------

void UExpandTool::Setup()
{
	Super::Setup();

	ensure(Targets.Num() > 0);

	ExpandProperties = NewObject<MeshPartition::UExpandToolProperties>(this);
	ExpandProperties->RestoreProperties(this);

	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		if (const IPrimitiveComponentBackedTarget* const TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Targets[TargetIndex]))
		{
			const int32 NewMeshIndex = OriginalMeshes.Add(MakeUnique<FDynamicMesh3>());
			ensure(NewMeshIndex == TargetIndex);
			*OriginalMeshes[TargetIndex] = UE::ToolTarget::GetDynamicMeshCopy(Targets[TargetIndex]);

			constexpr bool bAutoCompute = false;
			const int32 BoundaryLoopsIndex = MeshBoundaryLoops.Add(MakeUnique<Geometry::FMeshBoundaryLoops>(OriginalMeshes[TargetIndex].Get(), bAutoCompute));
			ensure(BoundaryLoopsIndex == TargetIndex);
			MeshBoundaryLoops[TargetIndex]->bOnlyComputeSpans = true;
			MeshBoundaryLoops[TargetIndex]->Compute();

			const float CornerThresholdDegrees = ExpandProperties->CornerThresholdDegrees;
			TArray<FEdgeSpan> NewSpans;
			for (const FEdgeSpan& Span : MeshBoundaryLoops[TargetIndex]->Spans)
			{
				TArray<FEdgeSpan> CurrentNewSpans;
				Span.GetSubspansByAngle(CornerThresholdDegrees, ExpandProperties->MinSpanSize, CurrentNewSpans);

				NewSpans.Append(CurrentNewSpans);
			}
			MeshBoundaryLoops[TargetIndex]->Spans = MoveTemp(NewSpans);

			const int32 MeshSpatialIndex = MeshSpatials.Add(MakeUnique<Geometry::FDynamicMeshAABBTree3>());
			ensure(MeshSpatialIndex == TargetIndex);
			MeshSpatials[TargetIndex]->SetMesh(OriginalMeshes[TargetIndex].Get());

			const int32 SeclectionMechanicIndex = SelectionMechanics.Add(NewObject<UBoundarySelectionMechanic>(this));
			ensure(SeclectionMechanicIndex == TargetIndex);
			SelectionMechanics[TargetIndex]->bAddSelectionFilterPropertiesToParentTool = false;
			SelectionMechanics[TargetIndex]->Setup(this);
			SelectionMechanics[TargetIndex]->Properties->bSelectEdges = true;
			SelectionMechanics[TargetIndex]->Properties->bSelectFaces = false;
			SelectionMechanics[TargetIndex]->Properties->bSelectVertices = false;

			SelectionMechanics[TargetIndex]->Initialize(OriginalMeshes[TargetIndex].Get(),
				(FTransform3d)TargetComponent->GetWorldTransform(),
				GetTargetWorld(),
				MeshBoundaryLoops[TargetIndex].Get(),
				[this, TargetIndex]()
				{
					return MeshSpatials[TargetIndex].Get();
				},
				UBoundarySelectionMechanic::EBoundarySelectionType::Spans
			);

			// allow toggling selection without modifier key (similar to HoleFillTool functionality)
			SelectionMechanics[TargetIndex]->SetShouldAddToSelectionFunc([]() { return true; });
			SelectionMechanics[TargetIndex]->SetShouldRemoveFromSelectionFunc([]() { return true; });
			SelectionMechanics[TargetIndex]->OnSelectionChanged.AddUObject(this, &UExpandTool::OnSelectionModified);

			SelectionMechanics[TargetIndex]->SetIsEnabled(true, true);


			if (AActor* OwnerActor = TargetComponent->GetOwnerActor())
			{
				while (OwnerActor)
				{
					if (AMeshPartition* OwnerMegaMeshActor = Cast<AMeshPartition>(OwnerActor))
					{
						ensure(!MegaMeshActor.IsValid() || MegaMeshActor == OwnerMegaMeshActor);		// There should be only one AMeshPartition in the target owners
						MegaMeshActor = OwnerMegaMeshActor;
						break;
					}
					OwnerActor = OwnerActor->GetAttachParentActor();
				}
			}
			checkf(MegaMeshActor.Get(), TEXT("No AMeshPartition selected, should have been checked in CanBuildTool"));

			const UMeshPartitionEditorComponent* const MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMeshActor.Get()->GetMeshPartitionComponent());
			checkf(MeshPartitionEditorComponent, TEXT("No UMeshPartitionEditorComponent selected, should be checked in CanBuildTool"));
		}
	}

	// Estimate the size and resolution of the extruded new meshes based on input meshes
	const double AverageMeshWidth = InputMeshWidth();
	const double AverageEdgeLength = AverageInputMeshBoundaryEdgeLength();
	ExpandProperties->NumSteps = FMath::Clamp(FMath::TruncToInt(AverageMeshWidth / AverageEdgeLength), 1, 1000);
	ExpandProperties->StepSize = AverageEdgeLength;

	ExpandProperties->WatchProperty(ExpandProperties->ExpandType,
		[this](MeshPartition::EExpandType)
		{
			GeneratePreviewMeshes();
		});
	ExpandProperties->WatchProperty(ExpandProperties->CornerThresholdDegrees,
		[this](float)
		{
			RecomputeBoundaryTopology();
		});
	ExpandProperties->WatchProperty(ExpandProperties->MinSpanSize,
		[this](int32)
		{
			RecomputeBoundaryTopology();
		});
	ExpandProperties->WatchProperty(ExpandProperties->StepSize,
		[this](float)
		{
			GeneratePreviewMeshes();
		});
	ExpandProperties->WatchProperty(ExpandProperties->NumSteps,
		[this](int32)
		{
			GeneratePreviewMeshes();
		});
	ExpandProperties->WatchProperty(ExpandProperties->ExtrudeDirection,
		[this](MeshPartition::EExpandExtrudeEdgeDirectionMode)
		{
			GeneratePreviewMeshes();
		});
	ExpandProperties->WatchProperty(ExpandProperties->bAdjustToExtrudeEvenly,
		[this](bool)
		{
			GeneratePreviewMeshes();
		});
	ExpandProperties->WatchProperty(ExpandProperties->bUseUnselectedForFrames,
		[this](bool)
		{
			GeneratePreviewMeshes();
		});
	ExpandProperties->WatchProperty(ExpandProperties->bRenderNewSections,
		[this](bool)
		{
			GeneratePreviewMeshes();
		});
	ExpandProperties->WatchProperty(ExpandProperties->bShouldMirrorCorners,
		[this](bool)
		{
			GeneratePreviewMeshes();
		});

	AddToolPropertySource(ExpandProperties);

	GeneratePreviewMeshes();
}

void UExpandTool::OnShutdown(EToolShutdownType InShutdownType)
{
	if (InShutdownType == EToolShutdownType::Accept)
	{
		CreateNewBaseModifiers();
	}

	ExpandProperties->SaveProperties(this);
	
	for (TObjectPtr<UBoundarySelectionMechanic> SelectionMechanic : SelectionMechanics)
	{
		SelectionMechanic->Shutdown();
	}

	for (UPreviewMesh* const PreviewMesh : PreviewMeshes)
	{
		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
	}
	PreviewMeshes.Reset();

}

void UExpandTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	for (TObjectPtr<UBoundarySelectionMechanic> SelectionMechanic : SelectionMechanics)
	{
		SelectionMechanic->Render(RenderAPI);
	}
}

void UExpandTool::OnSelectionModified()
{
	GeneratePreviewMeshes();
}

void UExpandTool::RecomputeBoundaryTopology()
{
	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		if (const IPrimitiveComponentBackedTarget* const TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Targets[TargetIndex]))
		{
			SelectionMechanics[TargetIndex]->ClearSelection();

			// Recompute boundary spans, and break them up according to the corner threshold

			MeshBoundaryLoops[TargetIndex]->bOnlyComputeSpans = true;
			MeshBoundaryLoops[TargetIndex]->Compute();

			ensure(ExpandProperties);
			const float CornerThresholdDegrees = ExpandProperties->CornerThresholdDegrees;
			TArray<FEdgeSpan> NewSpans;
			for (FEdgeSpan& Span : MeshBoundaryLoops[TargetIndex]->Spans)
			{
				TArray<FEdgeSpan> CurrentNewSpans;
				Span.GetSubspansByAngle(CornerThresholdDegrees, ExpandProperties->MinSpanSize, CurrentNewSpans);
				NewSpans.Append(CurrentNewSpans);
			}
			MeshBoundaryLoops[TargetIndex]->Spans = MoveTemp(NewSpans);

			// Reinitialize the selection mechanic based on new boundary spans
			ensure(TargetComponent);
			SelectionMechanics[TargetIndex]->Initialize(OriginalMeshes[TargetIndex].Get(),
				(FTransform3d)TargetComponent->GetWorldTransform(),
				GetTargetWorld(),
				MeshBoundaryLoops[TargetIndex].Get(),
				[this, TargetIndex]()
				{
					return MeshSpatials[TargetIndex].Get();
				},
				UBoundarySelectionMechanic::EBoundarySelectionType::Spans
			);
		}
	}
}

void UExpandTool::GeneratePreviewMeshes()
{
	for (UPreviewMesh* const PreviewMesh : PreviewMeshes)
	{
		PreviewMesh->SetVisible(false);
		PreviewMesh->Disconnect();
	}
	PreviewMeshes.Reset();

	switch (ExpandProperties->ExpandType)
	{
	case MeshPartition::EExpandType::Extrude:
		Extrude();
		break;
	case MeshPartition::EExpandType::Mirror:
		Mirror();
		break;
	}

	if (ExpandProperties->bRenderNewSections)
	{
		for (UPreviewMesh* const PreviewMesh : PreviewMeshes)
		{
			PreviewMesh->SetOverrideRenderMaterial(ToolSetupUtil::GetVertexColorMaterial(GetToolManager()));
			PreviewMesh->SetTriangleColorFunction([this](const FDynamicMesh3* Mesh, int TriangleID)
			{
				return LinearColors::SelectFColor(Mesh->GetTriangleGroup(TriangleID));
			}, UPreviewMesh::ERenderUpdateMode::FastUpdate);
		}
	}
	else
	{
		for (UPreviewMesh* const PreviewMesh : PreviewMeshes)
		{
			PreviewMesh->ClearOverrideRenderMaterial();
			PreviewMesh->ClearTriangleColorFunction(UPreviewMesh::ERenderUpdateMode::FastUpdate);
		}
	}
}


// TODO: See UPolyEditExtrudeEdgeActivity::RecalculateGizmoExtrudeFrame for how the single direction is computed in that case. 
// Determine if we want to add that direction as an alterative option
FVector3d UExpandTool::AverageOutDirection(int32 TargetIndex, int32 SpanIndex) const
{
	const FEdgeSpan& Span = MeshBoundaryLoops[TargetIndex]->Spans[SpanIndex];
	const FDynamicMesh3& OriginalMesh = *OriginalMeshes[TargetIndex];

	// Find the average "out" direction along the span
	FVector3d EdgePerpAvg(0.0, 0.0, 0.0);
	for (const int32 EdgeIndex : Span.Edges)
	{
		const FDynamicMesh3::FEdge& Edge = OriginalMesh.GetEdgeRef(EdgeIndex);
		ensure(Edge.Tri[1] == FDynamicMesh3::InvalidID);

		const FVector3d Normal = OriginalMesh.GetTriNormal(Edge.Tri[0]);
		FVector3d EdgeVector = OriginalMesh.GetVertex(Edge.Vert[1]) - OriginalMesh.GetVertex(Edge.Vert[0]);
		if (EdgeVector.Length() > UE_SMALL_NUMBER)
		{
			EdgeVector.Normalize();
			FVector3d EdgePerp = Normal.Cross(EdgeVector);

			const FVector3d TriangleCentroid = OriginalMesh.GetTriCentroid(Edge.Tri[0]);
			const FVector3d EdgeMidpoint = 0.5 * (OriginalMesh.GetVertex(Edge.Vert[1]) + OriginalMesh.GetVertex(Edge.Vert[0]));
			if (EdgePerp.Dot(EdgeMidpoint - TriangleCentroid) < 0.0)
			{
				EdgePerp = -EdgePerp;
			}

			EdgePerpAvg += EdgePerp;
		}
	}
	EdgePerpAvg.Normalize();

	return EdgePerpAvg;
}

double UExpandTool::InputMeshWidth() const
{
	double AverageWidth = 0.0;
	for (int32 TargetIndex = 0; TargetIndex < OriginalMeshes.Num(); ++TargetIndex)
	{
		const FDynamicMesh3& OriginalMesh = *OriginalMeshes[TargetIndex];
		const FAxisAlignedBox3d AABB = OriginalMesh.GetBounds();
		const FVector3d Extents = AABB.Extents();
		AverageWidth += 2.0 * Extents.GetMax();
	}
	return AverageWidth / OriginalMeshes.Num();
}

double UExpandTool::AverageInputMeshBoundaryEdgeLength() const
{
	// We will only consider boundary edges to speed things up
	double AverageEdgeLength = 0.0;
	int32 NumEdgesConsidered = 0;
	for (int32 MeshBoundaryIndex = 0; MeshBoundaryIndex < MeshBoundaryLoops.Num(); ++MeshBoundaryIndex)
	{
		const Geometry::FMeshBoundaryLoops* const Loops = MeshBoundaryLoops[MeshBoundaryIndex].Get();
		for (const FEdgeSpan& Span : Loops->Spans)
		{
			for (const int32 EdgeIndex : Span.Edges)
			{
				FVector3d VA, VB;
				OriginalMeshes[MeshBoundaryIndex]->GetEdgeV(EdgeIndex, VA, VB);
				AverageEdgeLength += FVector3d::Dist(VA, VB);
				++NumEdgesConsidered;
			}
		}
	}
	return AverageEdgeLength / NumEdgesConsidered;
}

void UExpandTool::Extrude()
{
	constexpr double MAX_VERT_MOVEMENT_ADJUSTMENT_SCALE = 4.0;

	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		TObjectPtr<UBoundarySelectionMechanic> SelectionMechanic = SelectionMechanics[TargetIndex];

		if (SelectionMechanic->HasSelection())
		{
			if (UWorld* const PreviewWorld = GetTargetWorld())
			{
				if (const IPrimitiveComponentBackedTarget* const TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Targets[TargetIndex]))
				{
					const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
					const TArray<FEdgeSpan>& Spans = MeshBoundaryLoops[TargetIndex]->Spans;

					int32 GroupID = OriginalMeshes[TargetIndex]->MaxGroupID();
					const int32 OriginalMaxGroupID = GroupID;

					TMap<int32, int32> EdgeToGroupMap;
					MegaMeshExpandToolHelpers::FindAdjacentSpanEndEdges(Spans, ActiveSelection.SelectedEdgeIDs.Array(), GroupID, EdgeToGroupMap);

					FVector3d SingleDirection(0.0, 0.0, 0.0);
					TArray<int32> SelectedMeshEdges;
					TArray<int32> EdgeGroupIDs;
					for (const int32 TopologyEdge : ActiveSelection.SelectedEdgeIDs)
					{
						// TODO: See UPolyEditExtrudeEdgeActivity::RecalculateGizmoExtrudeFrame for how the single direction is computed in that case
						// Determine if we want to add that direction as an alterative option
						SingleDirection += AverageOutDirection(TargetIndex, TopologyEdge);

						SelectedMeshEdges.Append(Spans[TopologyEdge].Edges);

						for (int32 EdgeArrayIndex = 0; EdgeArrayIndex < Spans[TopologyEdge].Edges.Num(); ++EdgeArrayIndex)
						{
							const int32 EID = Spans[TopologyEdge].Edges[EdgeArrayIndex];
							if (EdgeToGroupMap.Contains(EID))
							{
								EdgeGroupIDs.Add(EdgeToGroupMap[EID]);
							}
							else
							{
								EdgeGroupIDs.Add(GroupID);
							}
						}

						++GroupID;
					}

					ensure(EdgeGroupIDs.Num() == SelectedMeshEdges.Num());
					SingleDirection.Normalize();

					Geometry::FExtrudeBoundaryEdges::FExtrudeFrame ExtrudeFrame;

					FDynamicMesh3 EditMesh;
					EditMesh.Copy(*OriginalMeshes[TargetIndex]);

					FExtrudeBoundaryEdges Extruder(&EditMesh);
					
					switch (ExpandProperties->ExtrudeDirection)
					{
					case MeshPartition::EExpandExtrudeEdgeDirectionMode::LocalExtrudeFrames:
						Extruder.OffsetPositionFunc = [this](const FVector3d& Position, const FExtrudeBoundaryEdges::FExtrudeFrame& ExtrudeFrame, int32 SourceVid)
							{
								return ExtrudeFrame.FromFramePoint(FVector3d(ExpandProperties->StepSize, 0.0, 0.0));
							};
						break;
					case MeshPartition::EExpandExtrudeEdgeDirectionMode::SingleDirection:
						Extruder.OffsetPositionFunc = [this, &SingleDirection](const FVector3d& Position, const FExtrudeBoundaryEdges::FExtrudeFrame& ExtrudeFrame, int32 SourceVid)
							{
								return Position + ExpandProperties->StepSize * SingleDirection;
							};
						break;
					}

					Extruder.InputEids = SelectedMeshEdges;
					Extruder.GroupsToSetPerEid = EdgeGroupIDs;
					Extruder.ScalingAdjustmentLimit = ExpandProperties->bAdjustToExtrudeEvenly ? MAX_VERT_MOVEMENT_ADJUSTMENT_SCALE : 1.0;
					Extruder.bUsePerVertexExtrudeFrames = ExpandProperties->ExtrudeDirection == MeshPartition::EExpandExtrudeEdgeDirectionMode::LocalExtrudeFrames;
					Extruder.bAssignAnyBoundaryNeighborToUnmatched = ExpandProperties->bUseUnselectedForFrames;

					// Iteratively run extrude to generate new triangles

					TArray<int32> AllNewTriangles;
					for (int32 Step = 0; Step < ExpandProperties->NumSteps; ++Step)
					{
						Extruder.Apply(nullptr);

						ensure(Extruder.NewExtrudedEids.Num() == Extruder.InputEids.Num());
						Extruder.InputEids = Extruder.NewExtrudedEids;

						AllNewTriangles.Append(Extruder.NewTids);

						Extruder.NewExtrudedEids.Reset();
						Extruder.NewTids.Reset();
					}

					if (EditMesh.Attributes() && EditMesh.Attributes()->NumUVLayers() > 0 && EditMesh.Attributes()->NumNormalLayers() > 0)
					{
						// Set Normals and UVs on output tris
						FMeshNormals::InitializeOverlayRegionToPerVertexNormals(EditMesh.Attributes()->PrimaryNormals(), AllNewTriangles);

						FAxisAlignedBox3d Box;
						for (int32 Tid : AllNewTriangles)
						{
							FVector3d Vert1, Vert2, Vert3;
							EditMesh.GetTriVertices(Tid, Vert1, Vert2, Vert3);
							Box.Contain(Vert1);
							Box.Contain(Vert2);
							Box.Contain(Vert3);
						}
						FDynamicMeshUVEditor UVEd(&EditMesh, EditMesh.Attributes()->PrimaryUV());
						const FFrame3d BoxFrame(Box.Center() - Box.Extents());
						const FVector3d BoxDimensions = Box.Diagonal();
						UVEd.SetTriangleUVsFromBoxProjection(AllNewTriangles, [](FVector3d Position) { return Position; }, BoxFrame, BoxDimensions);
					}

					Geometry::FDynamicSubmesh3 Submesh(&EditMesh, AllNewTriangles);

					// Create new mesh
					UPreviewMesh* const PreviewMesh = NewObject<UPreviewMesh>(this);
					PreviewMesh->CreateInWorld(PreviewWorld, TargetComponent->GetWorldTransform());
					ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);

					PreviewMesh->EditMesh([&Submesh](FDynamicMesh3& Mesh)
					{
						Mesh = MoveTemp(Submesh.GetSubmesh());
					});
					PreviewMeshes.Add(PreviewMesh);

					PreviewMesh->SetVisible(true);
				}
			}
		}
	}
}

void UExpandTool::Mirror()
{
	for (int32 TargetIndex = 0; TargetIndex < Targets.Num(); ++TargetIndex)
	{
		TObjectPtr<UBoundarySelectionMechanic> SelectionMechanic = SelectionMechanics[TargetIndex];

		if (SelectionMechanic->HasSelection())
		{
			if (UWorld* const PreviewWorld = GetTargetWorld())
			{
				if (const IPrimitiveComponentBackedTarget* const TargetComponent = Cast<IPrimitiveComponentBackedTarget>(Targets[TargetIndex]))
				{
					int32 GroupID = OriginalMeshes[TargetIndex]->MaxGroupID();

					const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
					const TArray<FEdgeSpan>& Spans = MeshBoundaryLoops[TargetIndex]->Spans;

					auto DoMirror = [&](const FVector3d& MirrorPlaneOrigin, const FVector3d& MirrorPlaneNormal)
					{
						FDynamicMesh3 EditMesh;
						EditMesh.Copy(*OriginalMeshes[TargetIndex]);

						FMeshMirror Mirrorer(&EditMesh, MirrorPlaneOrigin, MirrorPlaneNormal);
						Mirrorer.bWeldAlongPlane = false;
						Mirrorer.WeldNormalMode = EMeshMirrorNormalMode::MirrorNormals;
						Mirrorer.bAllowBowtieVertexCreation = false;
						Mirrorer.PlaneTolerance = FMathf::ZeroTolerance * 10.0;

						Mirrorer.Mirror();

						for (const int32 TriID : EditMesh.TriangleIndicesItr())
						{
							EditMesh.SetTriangleGroup(TriID, GroupID);
						}

						// Create new mesh
						UPreviewMesh* const PreviewMesh = NewObject<UPreviewMesh>(this);
						PreviewMesh->CreateInWorld(PreviewWorld, TargetComponent->GetWorldTransform());
						ToolSetupUtil::ApplyRenderingConfigurationToPreview(PreviewMesh, nullptr);

						PreviewMesh->EditMesh([&EditMesh](FDynamicMesh3& Mesh)
						{
							Mesh = MoveTemp(EditMesh);
						});
						PreviewMeshes.Add(PreviewMesh);
						PreviewMesh->SetVisible(true);

						++GroupID;
					};

					for (const int32 SelectedSpanIndex : ActiveSelection.SelectedEdgeIDs)
					{
						const FEdgeSpan& Span = Spans[SelectedSpanIndex];
						const FVector3d OutDirection = AverageOutDirection(TargetIndex, SelectedSpanIndex);		// TODO: Alternative choices for the normal (e.g. voting scheme, snap to best axis, specify up direction, etc.)
						const FVector3d Origin = 0.5 * (OriginalMeshes[TargetIndex]->GetVertex(Span.Vertices[0]) + OriginalMeshes[TargetIndex]->GetVertex(Span.Vertices.Last()));
						DoMirror(Origin, OutDirection);
					}

					// check corners
					if (ExpandProperties->bShouldMirrorCorners)
					{
						for (int32 SpanIndex = 0; SpanIndex < Spans.Num(); ++SpanIndex)
						{
							const int32 NextSpanIndex = (SpanIndex + 1) % Spans.Num();
							if (ActiveSelection.SelectedEdgeIDs.Contains(SpanIndex) && ActiveSelection.SelectedEdgeIDs.Contains(NextSpanIndex))
							{
								const FVector3d OutDirectionA = AverageOutDirection(TargetIndex, SpanIndex);
								const FVector3d OutDirectionB = AverageOutDirection(TargetIndex, NextSpanIndex);

								const FVector3d Origin = OriginalMeshes[TargetIndex]->GetVertex(Spans[SpanIndex].Vertices.Last());
								DoMirror(Origin, 0.5 * (OutDirectionA + OutDirectionB));
							}
						}
					}

				}
			}
		}
	}
}


void UExpandTool::CreateNewBaseModifiers()
{
	AMeshPartition* const ResolvedMegaMeshActor = MegaMeshActor.Get();
	if (!ResolvedMegaMeshActor)
	{
		// The target MegaMesh was destroyed while the tool was open; nothing to commit to.
		UE_LOG(LogMegaMeshModelingToolset, Warning,
			TEXT("UExpandTool: target AMeshPartition was destroyed while the tool was open; discarding tool changes."));
		return;
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("MegaMeshExpandToolTransactionName", "Mesh Partition Expand Tool"));

	for (const UPreviewMesh* const PreviewMesh : PreviewMeshes)
	{
		if (const FDynamicMesh3* const PreviewDynamicMesh = PreviewMesh->GetPreviewDynamicMesh())
		{
			TMap<int32, TArray<int32>> TrianglesByGroup;
			for (const int32 TriangleID : PreviewDynamicMesh->TriangleIndicesItr())
			{
				const int32 GroupID = PreviewDynamicMesh->GetTriangleGroup(TriangleID);
				if (!TrianglesByGroup.Contains(GroupID))
				{
					TrianglesByGroup.Add(GroupID);
				}
				TrianglesByGroup[GroupID].Add(TriangleID);
			}

			const FTransform PreviewTransform = PreviewMesh->GetTransform();

			for (const TPair<int32, TArray<int32>>& TriangleGroup : TrianglesByGroup)
			{
				Geometry::FDynamicSubmesh3 Submesh(PreviewDynamicMesh, TriangleGroup.Value);
				Geometry::FDynamicMesh3 NewComponentMesh = Submesh.GetSubmesh();

				TArray<UMaterialInterface*> Materials;
				if (const UMeshPartitionDefinition* const MeshPartitionDefinition = ResolvedMegaMeshActor->GetMeshPartitionDefinition())
				{
					if (UMaterialInterface* const MegaMeshMaterial = MeshPartitionDefinition->GetMaterial())
					{
						Materials.Emplace(MegaMeshMaterial);
					}
				}

				// center vertices and compute the transform
				const FAxisAlignedBox3d Bounds = NewComponentMesh.GetBounds();
				const FVector Center = Bounds.Center();
				const FTransform SectionToWorld = FTransform(Center);
				MeshTransforms::ApplyTransformInverse(NewComponentMesh, SectionToWorld);

				UMeshPartitionEditorComponent* MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(ResolvedMegaMeshActor->GetMeshPartitionComponent());
				if (ensure(MeshPartitionEditorComponent != nullptr))
				{
					const FTransform BaseTransform = SectionToWorld * PreviewTransform;
					const AActor* const NewActor = MeshPartitionEditorComponent->SpawnBaseModifier(MoveTemp(NewComponentMesh), Materials, BaseTransform);
					ensure(NewActor);
				}
			}
		}
	}

	GetToolManager()->EndUndoTransaction();
}
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE


