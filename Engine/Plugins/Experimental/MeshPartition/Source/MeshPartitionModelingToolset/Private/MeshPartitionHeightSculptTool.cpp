// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionHeightSculptTool.h"

#include "InteractiveToolManager.h"
#include "ModelingToolTargetUtil.h"
#include "Sculpting/HeightBrushes.h"
#include "ToolContextInterfaces.h" // FToolBuilderState

#include "MeshPartitionModifierToolTarget.h"

#define LOCTEXT_NAMESPACE "UMegaMeshVertexSculptTool"

namespace UE::MeshPartition
{
UMeshSurfacePointTool* UHeightSculptToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	MeshPartition::UHeightSculptTool* SculptTool = NewObject<MeshPartition::UHeightSculptTool>(SceneState.ToolManager);
	SculptTool->SetWorld(SceneState.World);
	SculptTool->SetDefaultPrimaryBrushID(DefaultPrimaryBrushID);
	return SculptTool;
}

bool UHeightSculptTool::GetAllowStandardEditorGizmos()
{
	return ToolSettingsProperties && ToolSettingsProperties->bAllowEditorGizmo;
}

void UHeightSculptTool::Setup()
{
	// Add our extra properties before the Super call, to have them up top.
	HeightSculptProperties = NewObject<MeshPartition::UHeightSculptToolProperties>(this);
	HeightSculptProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	AddToolPropertySource(HeightSculptProperties);

	ToolSettingsProperties = NewObject<MeshPartition::UHeightSculptToolSettingsProperties>(this);
	ToolSettingsProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	AddToolPropertySource(ToolSettingsProperties);

	Super::Setup();

	// Symmetry does not make sense on a landscape. It could potentially make sense
	// for spherical height sculpt, but for now we explicitly disable symmetry support.
	if (SymmetryProperties)
	{
		SymmetryProperties->bEnableSymmetry = false;
	}
	SetToolPropertySourceEnabled(UMeshVertexSculptTool::SymmetryProperties, false);
	// have to update this because it might have been set in Setup before watcher update
	bApplySymmetry = false;

	// Reinitialize from target if needed
	if (ensure(GizmoProperties) && HeightSculptProperties->bInitializeReferenceTransformFromTarget && Target)
	{
		FTransform TransformToUse = UE::ToolTarget::GetLocalToWorldTransform(Target);
		GizmoProperties->Position = TransformToUse.GetLocation();
		GizmoProperties->Rotation = TransformToUse.GetRotation();
	}

	if (const UEditableModifierToolTarget* ModifierTarget = Cast<UEditableModifierToolTarget>(Target))
	{
		ModifierTarget->ConfigurePreviewForRendering(DynamicMeshComponent);
	}
	
	SculptProperties->bLockBoundaries = true;

	// Default to a smaller stamp opacity, so default Existing Material is more visible through the brush
	SculptProperties->StampOpacity = .5f;

	// Update default material mode to ExistingMaterial
	ViewProperties->MaterialMode = EMeshEditingMaterialModes::ExistingMaterial;
	ViewProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	
	// Update default brush size type to World.
	BrushProperties->BrushSize.SizeType = EBrushToolSizeType::World;
	BrushProperties->BrushSize.WorldRadius = 500.f;
	BrushProperties->StrokeType = EMeshSculptStrokeType::Airbrush;
	BrushProperties->RestoreProperties(this, GetPropertyCacheIdentifier());
	// make sure brush picks up the current brush WorldRadius
	CalculateBrushRadius();
}

void UHeightSculptTool::Shutdown(EToolShutdownType ShutdownType)
{
	HeightSculptProperties->SaveProperties(this, GetPropertyCacheIdentifier());
	HeightSculptProperties = nullptr;

	ToolSettingsProperties->SaveProperties(this, GetPropertyCacheIdentifier());
	ToolSettingsProperties = nullptr;

	Super::Shutdown(ShutdownType);
}


void UHeightSculptTool::RegisterBrushes()
{
	using namespace UE::Geometry;

	Super::RegisterBrushes();

	ensure(HeightSculptProperties); // Should be initialized before creating the brushes

	// register height brushes
	RegisterBrushType((int32)EBrushType::HeightSculpt, LOCTEXT("HeightBrush", "HeightSculpt"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FHeightSculptBrushOp>(HeightSculptProperties); }),
		NewObject<UMeshSculptBrushOpProps>(this, UE::Geometry::FHeightSculptBrushOp::GetPropertiesClass()));
	RegisterBrushType((int32)EBrushType::HeightSmooth, LOCTEXT("SmoothHeightBrush", "HeightSmooth"), 
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FHeightSmoothBrushOp>(HeightSculptProperties); }),
		NewObject<UMeshSculptBrushOpProps>(this, UE::Geometry::FHeightSmoothBrushOp::GetPropertiesClass( true)));
	RegisterBrushType((int32)EBrushType::HeightFlatten, LOCTEXT("FlattenHeightBrush", "HeightFlatten"), 
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FHeightFlattenBrushOp>(HeightSculptProperties); }),
		NewObject<UMeshSculptBrushOpProps>(this, UE::Geometry::FHeightFlattenBrushOp::GetPropertiesClass()));
	RegisterBrushType((int32)EBrushType::SlopeErode, LOCTEXT("SlopeErodeBrush", "SlopeErode"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FSlopeErodeBrushOp>(HeightSculptProperties); }),
		NewObject<UMeshSculptBrushOpProps>(this, UE::Geometry::FSlopeErodeBrushOp::GetPropertiesClass()));
	RegisterSecondaryBrushType((int32)EBrushType::HeightSmooth, LOCTEXT("SecondarySmoothHeightBrush", "HeightSmooth"),
		MakeUnique<FLambdaMeshSculptBrushOpFactory>([this]() { return MakeUnique<UE::Geometry::FHeightSmoothBrushOp>(HeightSculptProperties); }),
		NewObject<UMeshSculptBrushOpProps>(this, UE::Geometry::FHeightSmoothBrushOp::GetPropertiesClass( false)));
}

FString UHeightSculptTool::GetPropertyCacheIdentifier() const
{
	return TEXT("MeshPartition::UHeightSculptTool");
}

bool UHeightSculptTool::RequireConnectivityToHitPointInStamp() const
{
	return bHaveHeightBrushActive && HeightSculptProperties && HeightSculptProperties->bRequireConnectivity;
}

void UHeightSculptTool::UpdateBrushType(int32 BrushID)
{
	bHaveHeightBrushActive = BrushID >= (int32)EMeshVertexSculptBrushType::LastValue;
	SetToolPropertySourceEnabled(HeightSculptProperties, bHaveHeightBrushActive);
	bDrawWorkPlaneGridLines = !bHaveHeightBrushActive || HeightSculptProperties->ReferenceSurface == EHeightSculptReferenceSurface::Plane;
	SetActiveSecondaryBrushType(bHaveHeightBrushActive ?
		// Use our own secondary brush if we're using a height brush
		(int32)EBrushType::HeightSmooth
		// Our base class (the vertex sculpt tool) uses 0 for the default secondary brush id
		: 0);

	Super::UpdateBrushType(BrushID);
}

bool UHeightSculptTool::ShowWorkPlane() const
{
	if (GizmoProperties && !GizmoProperties->bShowPlane)
	{
		return false;
	}
	if (PrimaryBrushOp
		&& (PrimaryBrushOp->GetReferencePlaneType() == FMeshSculptBrushOp::EReferencePlaneType::WorkPlane
			|| PrimaryBrushOp->GetStampAlignmentType() == FMeshSculptBrushOp::EStampAlignmentType::ReferencePlane
			|| PrimaryBrushOp->GetStampAlignmentType() == FMeshSculptBrushOp::EStampAlignmentType::ReferenceSphere))
	{
		return true;
	}
	return Super::ShowWorkPlane();
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE