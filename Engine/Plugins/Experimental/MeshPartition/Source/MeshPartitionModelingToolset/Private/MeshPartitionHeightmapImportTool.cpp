// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionHeightmapImportTool.h"

#include "Editor.h"
#include "InteractiveToolManager.h"
#include "MeshPartitionHeightmapImporter.h"
#include "PackageSourceControlHelper.h"
#include "Engine/World.h"
#include "PrimitiveDrawInterface.h"

#define LOCTEXT_NAMESPACE "UHeightmapImportTool"

namespace UE::MeshPartition
{
UInteractiveTool* UHeightmapImportToolBuilder::BuildTool(const FToolBuilderState& InSceneState) const
{
	MeshPartition::UHeightmapImportTool* NewTool = NewObject<MeshPartition::UHeightmapImportTool>(InSceneState.ToolManager);
	NewTool->SetWorld(InSceneState.World);
	return NewTool;
}

FInt32Point UHeightmapImportPropertySet::GetSectionsResolution() const
{
	FInt32Point Resolution(0);

	if (SectionsGeneration == MeshPartition::EHeightmapImportSectionsGenerationMode::Automatic)
	{
		if (MeshResolution.X > 0 && MeshResolution.Y > 0 && MaxTrianglesPerSection > 0)
		{
			const int32 MaxQuadsPerSection = FMath::DivideAndRoundDown(MaxTrianglesPerSection, 2);
			const int32 SquareQuadsPerSectionEdge = FMath::Sqrt(static_cast<float>(MaxQuadsPerSection));

			const int32 ShortEdgeIndex = MeshResolution[0] <= MeshResolution[1] ? 0 : 1;
			const int32 ShortEdgeNumSections = FMath::DivideAndRoundUp(MeshResolution[ShortEdgeIndex], SquareQuadsPerSectionEdge);
			const int32 ShortEdgeQuadsPerSection = FMath::DivideAndRoundUp(MeshResolution[ShortEdgeIndex], ShortEdgeNumSections);

			const int32 LongEdgeNumSections = FMath::DivideAndRoundUp(MeshResolution[1 - ShortEdgeIndex], MaxQuadsPerSection / ShortEdgeQuadsPerSection);

			Resolution = ShortEdgeIndex == 0 ? FInt32Point(ShortEdgeNumSections, LongEdgeNumSections) : FInt32Point(LongEdgeNumSections, ShortEdgeNumSections);
		}
	}

	if (SectionsGeneration == MeshPartition::EHeightmapImportSectionsGenerationMode::Explicit)
	{
		if (SectionsResolution.X > 0 && SectionsResolution.Y > 0)
		{
			Resolution = SectionsResolution;
		}
	}

	return Resolution;
}

FInt32Point UHeightmapImportPropertySet::GetLocationVolumesResolution() const
{
	return bCreateLocationVolumes ? LocationVolumesResolution : FInt32Point{0};
}

void UHeightmapImportTool::SetWorld(UWorld* InWorld)
{
	World = InWorld;

	if (Properties)
	{
		Properties->bWorldPartitionIsAvailable = World && World->GetWorldPartition() != nullptr;
	}
}

void UHeightmapImportTool::Setup()
{
	Super::Setup();

	SetToolDisplayName(LOCTEXT("MegaMeshImportHeightmap", "Import Heightmap"));

	Properties = NewObject<MeshPartition::UHeightmapImportPropertySet>(this);
	Properties->bWorldPartitionIsAvailable = World && World->GetWorldPartition() != nullptr;

	Properties->RestoreProperties(this);

	Properties->WatchProperty(Properties->HeightmapFile.FilePath, [this](const FString&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->MeshResolution, [this](const FInt32Point&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->MeshSize, [this](const FVector&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->SectionsGeneration, [this](const MeshPartition::EHeightmapImportSectionsGenerationMode&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->MaxTrianglesPerSection, [this](const int32&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->SectionsResolution, [this](const FInt32Point&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->bSaveAndUnload, [this](const bool&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->bCreateLocationVolumes, [this](const bool&) { UpdateDisplayMessage(); });
	Properties->WatchProperty(Properties->LocationVolumesResolution, [this](const FInt32Point&) { UpdateDisplayMessage(); });
	UpdateDisplayMessage();

	AddToolPropertySource(Properties);
}

void UHeightmapImportTool::OnPropertyModified(UObject* PropertySet, FProperty* Property)
{
	Super::OnPropertyModified(PropertySet, Property);
	UpdateDisplayMessage();
}

void UHeightmapImportTool::Shutdown(EToolShutdownType InShutdownType)
{
	if (InShutdownType != EToolShutdownType::Accept)
	{
		return;
	}

	Properties->SaveProperties(this);

	// Save-and-unload writes packages and unloads actors, which isn't reversible by undo.
	// Skip the transaction wrap and reset the undo buffer afterwards instead, to not leave stranded assets on disk.
	const bool bSaveAndUnload = Properties->bSaveAndUnload;

	if (!bSaveAndUnload)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MeshPartitionHeightmapImportTransaction", "Import Heightmap to Mesh Partition"));
	}

	MeshPartition::FHeightmapImporter Importer(GetParams());

	FPackageSourceControlHelper SourceControlHelper;
	if (!Importer.Import(&SourceControlHelper))
	{
		GetToolManager()->DisplayMessage(Importer.GetErrorText(), EToolMessageLevel::UserError);
	}

	if (!bSaveAndUnload)
	{
		GetToolManager()->EndUndoTransaction();
	}
	else if (GEditor)
	{
		GEditor->ResetTransaction(LOCTEXT("MeshPartitionHeightmapImportResetTransaction", "Import Heightmap to Mesh Partition (Save and Unload)"));
	}
}

void UHeightmapImportTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	if (CanAccept())
	{
		const FHeightmapImportParams Params = GetParams();

		const TArray<FSectionInfo> SectionInfos = MeshPartition::FHeightmapImporter::GetSectionInfos(Params);

		if (SectionInfos.Num() > 0)
		{
			FPrimitiveDrawInterface *const PDI = RenderAPI->GetPrimitiveDrawInterface();

			const FVector2d MeshOffset = -FVector2d{ Params.MeshSize} * 0.5;

			auto DrawLine = [PDI, &MeshOffset, &MeshSize = Params.MeshSize](const FVector2d& UVStart, const FVector2d& UVEnd)
			{
				const FVector3d Start = {MeshOffset.X + UVStart.X * MeshSize.X, MeshOffset.Y + UVStart.Y * MeshSize.Y, 0.0};
				const FVector3d End = {MeshOffset.X + UVEnd.X * MeshSize.X, MeshOffset.Y + UVEnd.Y * MeshSize.Y, 0.0};

				PDI->DrawLine(Start, End, FLinearColor::Yellow, SDPG_World);
			};

			// The assumption is that the sections form a grid, and therefore we only need to render the vertical/horizontal lines of the grid.

			DrawLine({0.0, 0.0}, {1.0, 0.0});
			DrawLine({0.0, 0.0}, {0.0, 1.0});

			for (int32 SectionIndex = 0; SectionIndex < SectionInfos.Num(); ++SectionIndex)
			{
				const FSectionInfo& SectionInfo = SectionInfos[SectionIndex];

				if (SectionInfo.IndexXY.X == 0)
				{
					DrawLine({0.0, SectionInfo.MaxUV.Y}, {1.0, SectionInfo.MaxUV.Y});
				}

				if (SectionInfo.IndexXY.Y == 0)
				{
					DrawLine({SectionInfo.MaxUV.X, 0.0}, {SectionInfo.MaxUV.X, 1.0});
				}
			}
		}
	}
}

bool UHeightmapImportTool::IsConfigurationValid(FText* OutErrorMessage) const
{
	if (Properties->bSaveAndUnload && World->GetPackage()->GetLoadedPath().IsEmpty())
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("SaveLevelFirst",
				"The current level must be saved before using \"Save and Unload\". Please save the level and try again.");
		}
		return false;
	}

	const FInt32Point SectionsResolution = Properties->GetSectionsResolution();
	const FInt32Point LocationVolumesResolution = Properties->GetLocationVolumesResolution();

	if (Properties->HeightmapFile.FilePath.IsEmpty())
	{
		// No file selected yet - silently invalid (no error message to avoid a warning on tool startup).
		return false;
	}

	if (!Properties->HeightmapFile.FilePath.EndsWith(".png"))
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("HeightmapMustBePng", "Heightmap file must be a .png file.");
		}
		return false;
	}

	if (Properties->MeshResolution.X <= 0 || Properties->MeshResolution.Y <= 0)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("InvalidMeshResolution", "Mesh resolution must be greater than zero.");
		}
		return false;
	}

	if (Properties->MeshSize.X <= 0 || Properties->MeshSize.Y <= 0)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("InvalidMeshSize", "Mesh size must be greater than zero.");
		}
		return false;
	}

	if (SectionsResolution.X <= 0 || SectionsResolution.Y <= 0)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("InvalidSectionsResolution", "Sections resolution must be greater than zero.");
		}
		return false;
	}

	if (LocationVolumesResolution.X < 0 || LocationVolumesResolution.Y < 0)
	{
		if (OutErrorMessage)
		{
			*OutErrorMessage = LOCTEXT("InvalidLocationVolumesResolution", "Location volumes resolution must not be negative.");
		}
		return false;
	}

	return true;
}

bool UHeightmapImportTool::CanAccept() const
{
	if (!World || !World->GetWorldPartition())
	{
		return false;
	}

	return IsConfigurationValid();
}
void UHeightmapImportTool::UpdateDisplayMessage() const
{
	if (!World || !World->GetWorldPartition())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("MeshPartition_WorldPartitionRequired", "Mesh Partition requires World Partition to be enabled."),
			EToolMessageLevel::UserError);
		return;
	}

	FText ConfigErrorMessage;
	if (!IsConfigurationValid(&ConfigErrorMessage))
	{
		GetToolManager()->DisplayMessage(ConfigErrorMessage, EToolMessageLevel::UserError);
		return;
	}

	const int64 Threshold = 2048LL * 2048 * 2;
	const int64 ThresholdSaveAndUnload = Threshold * 4 * 4;

	const bool bSaveAndUnloadEnabled = Properties->bSaveAndUnload;
	const int64 NumTriangles = 2LL * static_cast<int64>(Properties->MeshResolution.X) * static_cast<int64>(Properties->MeshResolution.Y);

	FText Message;

	if (!bSaveAndUnloadEnabled && NumTriangles >= Threshold)
	{
		Message = FText::Format(
			LOCTEXT("LargeResolution", "Imported mesh will consist of {0} triangles.{1}"),
			FText::AsNumber(NumTriangles),
			LOCTEXT("EnableSaveAndUnload", " Consider enabling \"Save and Unload\" to limit memory consumption."));
	}

	if (bSaveAndUnloadEnabled && NumTriangles >= ThresholdSaveAndUnload)
	{
		Message = FText::Format(
			LOCTEXT("LargeResolutionSaveAndUnload", "Imported mesh will consist of {0} triangles."),
			FText::AsNumber(NumTriangles));
	}

	GetToolManager()->DisplayMessage(Message, EToolMessageLevel::UserWarning);
}

FHeightmapImportParams UHeightmapImportTool::GetParams() const
{
	FHeightmapImportParams Params;
	Params.World = World;
	Params.HeightmapFilename = Properties->HeightmapFile.FilePath;
	Params.MeshResolution = Properties->MeshResolution;
	Params.MeshSize = Properties->MeshSize;
	Params.SectionsResolution = Properties->GetSectionsResolution();
	Params.bSaveAndUnload = Properties->bSaveAndUnload;
	Params.LocationVolumesResolution = Properties->GetLocationVolumesResolution();

	return Params;
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
