// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMergeTool.h"

#include "DynamicMeshEditor.h"
#include "InteractiveToolManager.h"
#include "MeshPartitionComponentBackedTarget.h"
#include "MeshPartitionSectionToolTarget.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "Misc/ScopedSlowTask.h"
#include "Selection/ToolSelectionUtil.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionMergeTool)

#define LOCTEXT_NAMESPACE "MegaMeshMergeTool"

using namespace UE::Geometry;

namespace UE::MeshPartition
{
// -------------------------------------------------------------------------------------------------------------------------

const FToolTargetTypeRequirements& UMergeToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshProvider::StaticClass(),
			UPrimitiveComponentBackedTarget::StaticClass(),
			UMeshPartitionComponentBackedTarget::StaticClass(),
		});

		return TypeRequirements;
}

bool UMergeToolBuilder::CanBuildTool(const FToolBuilderState& InSceneState) const
{
	return (InSceneState.TargetManager->CountSelectedAndTargetable(InSceneState, GetTargetRequirements()) > 1);
}

UMultiSelectionMeshEditingTool* UMergeToolBuilder::CreateNewTool(const FToolBuilderState& InSceneState) const
{
	return NewObject<MeshPartition::UMergeTool>(InSceneState.ToolManager);
}

// -------------------------------------------------------------------------------------------------------------------------

void UMergeTool::Setup()
{
	Super::Setup();

	MergeProperties = NewObject<MeshPartition::UMergeToolProperties>(this);
	AddToolPropertySource(MergeProperties);
	MergeProperties->RestoreProperties(this);
}

void UMergeTool::OnShutdown(EToolShutdownType InShutdownType)
{
	MergeProperties->SaveProperties(this);
	
	if (InShutdownType == EToolShutdownType::Accept)
	{
		Merge();
	}
}

void UMergeTool::Merge()
{
	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;
	
	TArray<FDynamicMesh3> InputMeshes;
	InputMeshes.Reserve(Targets.Num());
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		InputMeshes.Add(ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx], GetMeshParams));
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("CombineMeshesToolTransactionName", "Merge Meshes"));

	FDynamicMesh3 AccumulatedMesh;
	AccumulatedMesh.EnableTriangleGroups();
	AccumulatedMesh.EnableAttributes();
	AccumulatedMesh.Attributes()->EnableTangents();
	AccumulatedMesh.Attributes()->EnableMaterialID();
	AccumulatedMesh.Attributes()->EnablePrimaryColors();

	constexpr int SkipIndex = 0;
	AActor* SkipActor = ToolTarget::GetTargetActor(Targets[SkipIndex]);
	check(SkipActor);

	FTransform TargetToWorld = SkipActor->GetRootComponent()->GetComponentTransform();

	{
		FScopedSlowTask SlowTask(Targets.Num() + 2, 
			LOCTEXT("CombineMeshesBuild", "Building merged mesh ..."));
		SlowTask.MakeDialog();

		for (int32 SectionIdx = 0; SectionIdx < Targets.Num(); SectionIdx++)
		{
			SlowTask.EnterProgressFrame(1);

			FDynamicMesh3& SectionMesh = InputMeshes[SectionIdx];

			if (SectionIdx != SkipIndex)
			{
				UPrimitiveComponent* SectionComponent = ToolTarget::GetTargetComponent(Targets[SectionIdx]);
				FTransform ComponentToWorld = SectionComponent->GetComponentTransform();

				// Premultiply the transforms so we only call ApplyTransform a single time
				// Note: we can only combine the transforms like this if neither has nonuniform scale.
				// If we need non-uniform scales, we'll need to do this via a matrix.
				const FTransform3d ComponentToTarget = ComponentToWorld * TargetToWorld.Inverse();
				
				MeshTransforms::ApplyTransform(SectionMesh, ComponentToTarget, true);
			}

			AccumulatedMesh.AppendWithOffsets(SectionMesh);
		}

		SlowTask.EnterProgressFrame(1);
		
		// Weld edges if requested
		if (MergeProperties->bWeldEdges)
		{
			FMergeCoincidentMeshEdges Merger(&AccumulatedMesh);
			Merger.MergeVertexTolerance = 0.0001; // #todo_megamesh [roey]: Expose as param?
			Merger.MergeSearchTolerance = 2 * Merger.MergeVertexTolerance;
			Merger.OnlyUniquePairs = false;
			Merger.bWeldAttrsOnMergedEdges = true;
			// #todo_megamesh [roey]: Set anything on Merger.SplitAttributeWelder?; 

			ensure(Merger.Apply());
		}
		
		SlowTask.EnterProgressFrame(1);

		// Commit changes:
		{
			// The operation of merging multiple MegaMesh sections collapses many edges while also guaranteeing that all the non-collapsed edges keep the same topology.
			MeshPartition::USectionToolTarget* SectionToolTarget = CastChecked<MeshPartition::USectionToolTarget>(Targets[0]);
			
			FComponentMaterialSet NewMaterialSet;
			NewMaterialSet.Materials = ToolTarget::GetMaterialSet(Targets[0]).Materials;
			ToolTarget::CommitDynamicMeshUpdate(Targets[0], AccumulatedMesh, true, FConversionToMeshDescriptionOptions(), &NewMaterialSet);

			// CommitDynamicMeshUpdate updates the materials for the underlying asset. However,
			// it does not update the component itself, so address that now.
			ToolTarget::CommitMaterialSetUpdate(Targets[0], NewMaterialSet, false);
		}

		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), SkipActor);
	}

	// Delete the source sections
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("RemoveSources", "Remove Inputs"));
	
		for (int Idx = 1; Idx < Targets.Num(); Idx++)
		{
			AActor* Actor = ToolTarget::GetTargetActor(Targets[Idx]);
			Actor->Destroy();
		}
		GetToolManager()->EndUndoTransaction();
	}

	GetToolManager()->EndUndoTransaction();
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE