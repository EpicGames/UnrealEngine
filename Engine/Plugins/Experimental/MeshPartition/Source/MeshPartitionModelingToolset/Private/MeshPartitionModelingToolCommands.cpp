// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModelingToolCommands.h"

#include "MeshPartitionModelingToolsStyle.h"
#include "MeshPartitionHeightSculptTool.h"

#define LOCTEXT_NAMESPACE "MeshModelingToolsCommands"

FMegaMeshModelingToolCommands::FMegaMeshModelingToolCommands()
	: TCommands<FMegaMeshModelingToolCommands>(
		"MegaMeshModelingToolCommands", // Context name for fast lookup
		NSLOCTEXT("Contexts", "MegaMeshModelingToolCommands", "Mesh Partition Modeling Tools"), // Localized context name for displaying
		NAME_None, // Parent
		FMegaMeshModelingToolsStyle::Get()->GetStyleSetName() // Icon Style Set
		)
{
}

void FMegaMeshModelingToolCommands::RegisterCommands()
{
	UI_COMMAND(MegaMeshToolsTabButton, "MeshPartition", "Mesh Partition Modeling Toolset", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(BeginConvertMeshTool, "Convert", "Convert the target mesh into a new Mesh Partition actor", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginSplitMeshTool, "Split", "Split a mesh into multiple Mesh Partition section actors", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginMergeMeshTool, "Merge", "Merges multiple Mesh Partition sections into a single section actor", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginResectionMeshTool, "Resection", "Merges and then divides selected mesh sections", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginStitchMeshTool, "Stitch", "Stitch a regular mesh into a Mesh Partition section", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginHeightmapImport, "Import Heightmap", "Create a new Mesh Partition from a heightmap", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginHeightSculptTool, "Height Sculpt", "Sculpt height based on a reference surface", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginExpandMeshTool, "Expand", "Insert new Mesh Partition sections at section boundaries", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginCreateMegaMeshRectangleTool, "Create Rectangle", "Create a new rectangular Mesh Partition", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(BeginAddModifierTool, "Add Modifier", "Create a new Mesh Partition modifier", EUserInterfaceActionType::ToggleButton, FInputChord());
}

//~ Modeled on ModelingToolsActions.cpp
UE::Geometry::FMegaMeshToolActionCommands::FMegaMeshToolActionCommands() :
	TInteractiveToolCommands<FMegaMeshToolActionCommands>(
		"MegaMeshHotkeys", // Context name for fast lookup
		LOCTEXT("HotkeysCategory", "Mesh Partition Hotkeys"), // Localized context name for displaying
		NAME_None, // Parent
		FMegaMeshModelingToolsStyle::Get()->GetStyleSetName() // Icon Style Set
	)
{
}
void UE::Geometry::FMegaMeshToolActionCommands::RegisterAllToolActions()
{
	UE::Geometry::FMegaMeshHeightSculptToolCommands::Register();
}
void UE::Geometry::FMegaMeshToolActionCommands::UnregisterAllToolActions()
{
	UE::Geometry::FMegaMeshHeightSculptToolCommands::Unregister();
}

//~ Modeled on ModelingToolsActions.cpp
#define DEFINE_TOOL_ACTION_COMMANDS(CommandsClassName, ContextNameString, SettingsDialogString, ToolClassName ) \
UE::Geometry::CommandsClassName::CommandsClassName() : TInteractiveToolCommands<CommandsClassName>( \
ContextNameString, NSLOCTEXT("Contexts", ContextNameString, SettingsDialogString), NAME_None, FMegaMeshModelingToolsStyle::Get()->GetStyleSetName()) {} \
void UE::Geometry::CommandsClassName::GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) \
{\
	ToolCDOs.Add(GetMutableDefault<ToolClassName>()); \
}

DEFINE_TOOL_ACTION_COMMANDS(FMegaMeshHeightSculptToolCommands, "HeightSculpt", "MegaMesh - Height Sculpt", UE::MeshPartition::UHeightSculptTool);

#undef LOCTEXT_NAMESPACE
