// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"
#include "Tools/InteractiveToolsCommands.h"

class FMegaMeshModelingToolCommands : public TCommands<FMegaMeshModelingToolCommands>
{
public:
	FMegaMeshModelingToolCommands();

	TSharedPtr<FUICommandInfo> MegaMeshToolsTabButton;
	TSharedPtr<FUICommandInfo> BeginConvertMeshTool;
	TSharedPtr<FUICommandInfo> BeginSplitMeshTool;
	TSharedPtr<FUICommandInfo> BeginMergeMeshTool;
	TSharedPtr<FUICommandInfo> BeginResectionMeshTool;
	TSharedPtr<FUICommandInfo> BeginStitchMeshTool;
	TSharedPtr<FUICommandInfo> BeginHeightmapImport;
	TSharedPtr<FUICommandInfo> BeginHeightSculptTool;
	TSharedPtr<FUICommandInfo> BeginExpandMeshTool;
	TSharedPtr<FUICommandInfo> BeginCreateMegaMeshRectangleTool;
	TSharedPtr<FUICommandInfo> BeginAddModifierTool;

	/**
	 * Initialize commands
	 */
	virtual void RegisterCommands() override;
};

namespace UE::Geometry
{
	//~ Modeled on ModelingToolsActions.h
	class FMegaMeshToolActionCommands : public TInteractiveToolCommands<FMegaMeshToolActionCommands>
	{
	public:
		FMegaMeshToolActionCommands();

		static void RegisterAllToolActions();
		static void UnregisterAllToolActions();

		//~ We don't have an UpdateToolCommandBinding() method because binding/unbinding for
		//~  extension tools is done by just supplying a commands object getter with the tool
		//~  (see registration of tools in the module cpp).
	};
}

//~ Modeled on ModelingToolsActions.h
#define DECLARE_TOOL_ACTION_COMMANDS(CommandsClassName) \
class CommandsClassName : public TInteractiveToolCommands<CommandsClassName> \
{\
public:\
	CommandsClassName();\
	virtual void GetToolDefaultObjectList(TArray<UInteractiveTool*>& ToolCDOs) override;\
};\

namespace UE::Geometry
{
	DECLARE_TOOL_ACTION_COMMANDS(FMegaMeshHeightSculptToolCommands);
}