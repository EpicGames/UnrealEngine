// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModelingToolsStyle.h"

#include "Styling/SlateStyleRegistry.h"
#include "Styling/SlateTypes.h"
#include "Styling/CoreStyle.h"
#include "Styling/AppStyle.h"
#include "Interfaces/IPluginManager.h"
#include "SlateOptMacros.h"
#include "Styling/SlateStyleMacros.h"
#include "Util/ColorConstants.h"

using namespace UE::Geometry;

#define IMAGE_PLUGIN_BRUSH( RelativePath, ... ) FSlateImageBrush( FMegaMeshModelingToolsStyle::InContent( RelativePath, ".png" ), __VA_ARGS__ )

// This is to fix the issue that SlateStyleMacros like IMAGE_BRUSH look for RootToContentDir but StyleSet->RootToContentDir is how this style is set up
#define RootToContentDir StyleSet->RootToContentDir

FString FMegaMeshModelingToolsStyle::InContent(const FString& InRelativePath, const ANSICHAR* InExtension)
{
	static FString ContentDir = IPluginManager::Get().FindPlugin(TEXT("MeshPartition"))->GetContentDir();
	return (ContentDir / InRelativePath) + InExtension;
}

TSharedPtr< FSlateStyleSet > FMegaMeshModelingToolsStyle::StyleSet = nullptr;
TSharedPtr< class ISlateStyle > FMegaMeshModelingToolsStyle::Get() { return StyleSet; }

FName FMegaMeshModelingToolsStyle::GetStyleSetName()
{
	static FName ModelingToolsStyleName(TEXT("MegaMeshModelingToolsStyle"));
	return ModelingToolsStyleName;
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void FMegaMeshModelingToolsStyle::Initialize()
{
	// Const icon sizes
	const FVector2D Icon20x20(20.0f, 20.0f);
	const FVector2D Icon40x40(40.0f, 40.0f);

	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet(GetStyleSetName()));
	StyleSet->SetContentRoot(IPluginManager::Get().FindPlugin(TEXT("MeshPartition"))->GetContentDir());
	StyleSet->SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	{
		StyleSet->Set("MegaMeshModelingToolCommands.MegaMeshToolsTabButton",			new IMAGE_BRUSH_SVG("Icons/ModelingSphere", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.MegaMeshToolsTabButton.Small",		new IMAGE_BRUSH_SVG("Icons/ModelingSphere", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginCreateMegaMeshRectangleTool",	new IMAGE_PLUGIN_BRUSH( "Icons/ModelingRectangle_x40", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginCreateMegaMeshRectangleTool.Small",	new IMAGE_PLUGIN_BRUSH( "Icons/ModelingRectangle_x40", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginConvertMeshTool",				new IMAGE_BRUSH_SVG("Icons/ConvertMesh", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginConvertMeshTool.Small",		new IMAGE_BRUSH_SVG("Icons/ConvertMesh", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginSplitMeshTool",				new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginSplitMeshTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginMergeMeshTool",				new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginMergeMeshTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/VoxMerge_40x", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginResectionMeshTool", new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", Icon40x40)); // TODO: new icon?
		StyleSet->Set("MegaMeshModelingToolCommands.BeginResectionMeshTool.Small", new IMAGE_PLUGIN_BRUSH("Icons/PlaneCut_40x", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginExpandMeshTool",				new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginExpandMeshTool.Small",         new IMAGE_BRUSH_SVG("Icons/ExtrudeEdge", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginStitchMeshTool",				new IMAGE_PLUGIN_BRUSH("Icons/StitchMesh_40x", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginStitchMeshTool.Small",			new IMAGE_PLUGIN_BRUSH("Icons/StitchMesh_40x", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginHeightmapImport",				new IMAGE_BRUSH_SVG("Icons/HeightmapImport", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginHeightmapImport.Small",		new IMAGE_BRUSH_SVG("Icons/HeightmapImport", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginHeightSculptTool",		new IMAGE_PLUGIN_BRUSH("Icons/HeightSculpt_40x", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginHeightSculptTool.Small",	new IMAGE_PLUGIN_BRUSH("Icons/HeightSculpt_40x", Icon20x20));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginAddModifierTool",				new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", Icon40x40));
		StyleSet->Set("MegaMeshModelingToolCommands.BeginAddModifierTool.Small",		new IMAGE_PLUGIN_BRUSH("Icons/Displace_40x", Icon20x20));
	}

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void FMegaMeshModelingToolsStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

#undef IMAGE_PLUGIN_BRUSH
