// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateStyle.h"

class FMegaMeshModelingToolsStyle
{
public:
	static void Initialize();

	static void Shutdown();

	static TSharedPtr< class ISlateStyle > Get();

	static FName GetStyleSetName();
private:
	static FString InContent(const FString& RelativePath, const ANSICHAR* Extension);

private:
	static TSharedPtr< class FSlateStyleSet > StyleSet;
};