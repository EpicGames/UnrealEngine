// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionModelingToolsetModule.h"

#include "EditorModeManager.h"
#include "Features/IModularFeatures.h"
#include "MeshPartition.h"
#include "MeshPartitionConvertTool.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionHeightmapImportTool.h"
#include "MeshPartitionMergeTool.h"
#include "MeshPartitionResectionTool.h"
#include "MeshPartitionExpandTool.h"
#include "MeshPartitionModelingToolCommands.h"
#include "MeshPartitionModelingToolsStyle.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionModifierToolTarget.h"
#include "MeshPartitionMultiSectionToolTarget.h"
#include "MeshPartitionSectionToolTarget.h"
#include "MeshPartitionSplitTool.h"
#include "MeshPartitionStitchTool.h"
#include "MeshPartitionToolTarget.h"
#include "MeshPartitionHeightSculptTool.h"
#include "MeshPartitionCreateMeshTool.h"
#include "MeshPartitionPlaceModifierTool.h"

#include "ModelingToolsEditorMode.h"
#include "Modules/ModuleManager.h"
#include "LevelEditor.h"
#include "UObject/UObjectIterator.h"

#include "InteractiveToolManager.h"
#include "InteractiveToolsContext.h"

#define LOCTEXT_NAMESPACE "FMegaMeshModelingToolsetModule"

DEFINE_LOG_CATEGORY(LogMegaMeshModelingToolset);

namespace UE::MeshPartition
{
void FMegaMeshModelingToolsetModule::StartupModule()
{
	FMegaMeshModelingToolsStyle::Initialize();
	FMegaMeshModelingToolCommands::Register();
	Geometry::FMegaMeshToolActionCommands::RegisterAllToolActions();

	IModularFeatures::Get().RegisterModularFeature(IModelingModeToolExtension::GetModularFeatureName(), this);
}

void FMegaMeshModelingToolsetModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(IModelingModeToolExtension::GetModularFeatureName(), this);

	Geometry::FMegaMeshToolActionCommands::UnregisterAllToolActions();
	FMegaMeshModelingToolCommands::Unregister();
	FMegaMeshModelingToolsStyle::Shutdown();
}

FText FMegaMeshModelingToolsetModule::GetExtensionName()
{
	return LOCTEXT("MegaMeshTools_ExtensionName", "MeshPartitionTools");
}

FText FMegaMeshModelingToolsetModule::GetToolSectionName()
{
	return LOCTEXT("MegaMeshTools_SectionName", "Mesh Partition");
}

void FMegaMeshModelingToolsetModule::GetExtensionTools(const FExtensionToolQueryInfo& Info, TArray<FExtensionToolDescription>& OutTools)
{
	const FMegaMeshModelingToolCommands& Commands = FMegaMeshModelingToolCommands::Get();
	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshConvertTool", "Mesh Partition Convert");
		ToolDesc.ToolCommand = Commands.BeginConvertMeshTool;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::UConvertToolBuilder>();
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshSplitTool", "Mesh Partition Split");
		ToolDesc.ToolCommand = Commands.BeginSplitMeshTool;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::USplitToolBuilder>();
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshMergeTool", "Mesh Partition Merge");
		ToolDesc.ToolCommand = Commands.BeginMergeMeshTool;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::UMergeToolBuilder>();
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MeshPartitionResectionTool", "Mesh Partition Resection");
		ToolDesc.ToolCommand = Commands.BeginResectionMeshTool;
		ToolDesc.ToolBuilder = NewObject<UMeshPartitionResectionToolBuilder>();
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshStitchTool", "Mesh Partition Stitch");
		ToolDesc.ToolCommand = Commands.BeginStitchMeshTool;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::UStitchToolBuilder>();
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshHeightmapImport", "Mesh Partition Heightmap Import");
		ToolDesc.ToolCommand = Commands.BeginHeightmapImport;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::UHeightmapImportToolBuilder>();
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshHeightSculpt", "Mesh Partition Height Sculpt");
		ToolDesc.ToolCommand = Commands.BeginHeightSculptTool;
		TObjectPtr<MeshPartition::UHeightSculptToolBuilder> ToolBuilder = NewObject<MeshPartition::UHeightSculptToolBuilder>();
		ToolBuilder->StylusAPI = Info.StylusAPI;
		ToolDesc.ToolBuilder = ToolBuilder;
		ToolDesc.ToolCommandsGetter = []() -> const UE::IInteractiveToolCommandsInterface&
		{ 
			return Geometry::FMegaMeshHeightSculptToolCommands::Get(); 
		};
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshCreateRectangle", "Mesh Partition Create Rectangle");
		ToolDesc.ToolCommand = FMegaMeshModelingToolCommands::Get().BeginCreateMegaMeshRectangleTool;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::UCreateMeshToolBuilder>();
		MeshPartition::UCreateMeshToolBuilder* Builder = StaticCast< MeshPartition::UCreateMeshToolBuilder*>(ToolDesc.ToolBuilder);
		Builder->ShapeType = MeshPartition::UCreateMeshToolBuilder::EMakeMeshShapeType::Rectangle;
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshAddModifier", "Mesh Partition Add Modifier");
		ToolDesc.ToolCommand = FMegaMeshModelingToolCommands::Get().BeginAddModifierTool;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::UPlaceModifierToolBuilder>();
		OutTools.Add(ToolDesc);
	}

	{
		FExtensionToolDescription ToolDesc;
		ToolDesc.ToolName = LOCTEXT("MegaMeshExpandTool", "Mesh Partition Expand");
		ToolDesc.ToolCommand = Commands.BeginExpandMeshTool;
		ToolDesc.ToolBuilder = NewObject<MeshPartition::UExpandToolBuilder>();
		OutTools.Add(ToolDesc);
	}

}

bool FMegaMeshModelingToolsetModule::GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& OutInfo)
{
	OutInfo.ExtensionCommand = FMegaMeshModelingToolCommands::Get().MegaMeshToolsTabButton;
	return true;
}

bool FMegaMeshModelingToolsetModule::GetExtensionToolTargets(TArray<TSubclassOf<UToolTargetFactory>>& OutToolTargetFactories)
{
	OutToolTargetFactories.Add(UMultiSectionToolTargetFactory::StaticClass());
	OutToolTargetFactories.Add(MeshPartition::USectionToolTargetFactory::StaticClass());
	OutToolTargetFactories.Add(UMeshPartitionToolTargetFactory::StaticClass());
	OutToolTargetFactories.Add(MeshPartition::UModifierToolTargetFactory::StaticClass());
	OutToolTargetFactories.Add(MeshPartition::UEditableModifierToolTargetFactory::StaticClass());
	return true;
}
IMPLEMENT_MODULE(FMegaMeshModelingToolsetModule, MeshPartitionModelingToolset)
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
