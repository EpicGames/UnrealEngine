// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPartitionModifierToolTarget.h"

#include "DynamicMeshEditor.h"
#include "MeshPartitionChannelCollection.h"
#include "MeshPartitionComponent.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionMeshBuilder.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "Modifiers/MeshPartitionEditableModifierBase.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "MeshPartitionMeshData.h"
#include "Engine/World.h"
#include "MeshPartitionToolPreviewActor.h"

#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"

using namespace UE::Geometry;

namespace UE::MeshPartition
{
void UModifierToolTarget::Initialize(MeshPartition::UModifierComponent* ModifierComponent)
{
	TargetModifier = ModifierComponent;
	InitializeComponent(ModifierComponent);
	PreviewSections.Empty();
	if (UMeshPartitionEditorComponent* EditorComponent = ModifierComponent->GetMeshPartitionEditorComponent())
	{
		TArray<MeshPartition::UModifierComponent*> ModifiersToProcess = GetModifiersToProcess();
		for (MeshPartition::UModifierComponent* Modifier : ModifiersToProcess)
		{
			if (Modifier->IsBase())
			{
				if (MeshPartition::APreviewSection* PreviewSection = Modifier->GetPreviewSection())
				{
					PreviewSections.Add(PreviewSection);
				}
			}
		}
	}
}

FDynamicMesh3 MeshPartition::UModifierToolTarget::GetDynamicMesh()
{
	FDynamicMesh3 Mesh;
	if (TargetModifier)
	{
		Mesh.EnableAttributes();
		BuildModifiedMeshUpToTarget(Mesh, true);
	}
	return Mesh;
}

int32 UModifierToolTarget::GetNumMaterials() const
{
	return MID ? 1 : 0;
}

UMaterialInterface* UModifierToolTarget::GetMaterial(int32 InMaterialIndex) const
{
	return MID;
}

void UModifierToolTarget::GetMaterialSet(FComponentMaterialSet& OutMaterialSet, bool bInPreferAssetMaterials) const
{
	if (MID)
	{
		OutMaterialSet.Materials.Add(MID);
	}
}

bool UModifierToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& InMaterialSet, bool bInApplyToAsset)
{
	return true;
}

UBodySetup* UModifierToolTarget::GetBodySetup() const
{
	return nullptr;
}

IInterface_CollisionDataProvider* UModifierToolTarget::GetComplexCollisionProvider() const
{
	return nullptr;
}

void UModifierToolTarget::SetOwnerVisibility(bool bInVisible) const
{
	// TODO: This will need adjustment to handle "interactive section" visibility once we start using that again.

	auto OnPreviewSectionReassigned = [this, WeakThis = TWeakObjectPtr<const UModifierToolTarget>(this), bInVisible](MeshPartition::UMeshProviderModifier* Modifier, MeshPartition::APreviewSection* PreviewSection)
	{
		if (WeakThis.IsValid())
		{
			// When a preview section is reassigned, it by definition means that a prior preview section was destroyed.
			// Find that (and any other) destroyed preview section and remove it and its preview:
			for (auto It = PreviewSections.CreateIterator(); It; ++It)
			{
				TWeakObjectPtr<MeshPartition::APreviewSection> RegisteredPreview = *It;
				if (!RegisteredPreview.IsValid())
				{
					if (TWeakObjectPtr<AActor>* WeakPreviewActor = PreviewActors.Find(RegisteredPreview))
					{
						if (AActor* PreviewActor = WeakPreviewActor->Get())
						{
							if (UWorld* World = PreviewActor->GetWorld())
							{
								World->DestroyActor(PreviewActor, false);
							}
						}
					}
					It.RemoveCurrent();
				}
			}

			if (PreviewSection && PreviewSection->GetRootComponent())
			{
				PreviewSection->GetRootComponent()->SetVisibility(bInVisible, /*bPropagateToChildren*/ true);

				// If the new preview section was not already registered to the list of previews, add it now
				PreviewSections.AddUnique(PreviewSection);

				MeshPartition::UMeshPartitionEditorComponent* EditorComponent = PreviewSection->GetMegaMeshEditorComponent();

				if (!PreviewActors.Contains(PreviewSection))
				{
					if (const MeshPartition::FModifierGroup* Group = EditorComponent->FindGroupInRegistry(PreviewSection->GetGroupRegistryKey()))
					{
						TArray<MeshPartition::UModifierComponent*> Modifiers;
						Algo::Transform(Group->AllResolvedModifierPtrs(), Modifiers, [](const TWeakObjectPtr<UModifierComponent>& Modifier) { return Modifier.Get(); });

						MeshPartition::FBuilderSettings BuilderSettings;
						BuilderSettings.bCacheResult = true;
		
						const UMeshPartitionDefinition* Definition = EditorComponent->GetMegaMeshDefinition() ? EditorComponent->GetMegaMeshDefinition() : UMeshPartitionDefinition::GetDefaultMegaMeshDefinition();

						BuilderSettings.ModifiersToProcess = MoveTemp(Modifiers);
						BuilderSettings.Transform = EditorComponent->GetOwner()->GetTransform();
						BuilderSettings.ModifierFilter = UE::MeshPartition::FilterHelpers::FilterModifiersByLastModifierToBuild(*TargetModifier, true);
						BuilderSettings.FilterBounds = GetBounds();
						BuilderSettings.FilterBoundsMode = MeshPartition::EFilterBoundsMode::Exclusive;
						// Note: Large max section complexity to get a single contiguous/combined mesh
						BuilderSettings.MaxSectionComplexity = FMathd::MaxReal;
						BuilderSettings.TypePriorities = Definition->GetModifierTypePriorities();
						BuilderSettings.TexcoordGenerationOptions = FChannelCollectionUVLayoutOptions::GetFromDefinition(Definition);
						BuilderSettings.bRecomputeNormals = true;

						FBuilderSettings::FChannelRenderSettings ChannelRenderSettings;
						ChannelRenderSettings.ChannelMap = Definition->GetChannelMap();
						ChannelRenderSettings.TexelSize = Definition->GetChannelTexelSize();
						BuilderSettings.ChannelRenderSettings = ChannelRenderSettings;

						TArray<MeshPartition::FBuildTaskHandle> BuildTaskHandles = UE::MeshPartition::Build::LaunchBuilds(BuilderSettings);

						check(BuildTaskHandles.Num() == 1);

						MeshPartition::FBuildTaskHandle& TaskHandle = BuildTaskHandles[0];

						TSharedPtr<const FMeshData> BuiltMesh = TaskHandle.GetTask()->GetMesh();
			
						MeshPartition::AToolPreviewMesh* PreviewMesh = EditorComponent->SpawnTransientActor<MeshPartition::AToolPreviewMesh>(EditorComponent->GetOwner()->GetTransform());
						PreviewMesh->SetMesh(BuiltMesh.ToSharedRef());
						PreviewMesh->SetMaterial(Definition->GetMaterial());
						PreviewMesh->SetChannelData(*TaskHandle.GetTask()->GetSectionChannels());

						PreviewActors.Add(PreviewSection, PreviewMesh);
					}
				}
			}
		}
	};

	if (!bInVisible)
	{
		// Gather all the base sections associated with our preview sections, and hide their preview sections in
		//  a way that is robust to preview section reassignment.
		for (TWeakObjectPtr<MeshPartition::APreviewSection> PreviewSection : PreviewSections)
		{
			for (TWeakObjectPtr<MeshPartition::UModifierComponent> UncastBase : PreviewSection->GetBaseModifiers())
			{
				if (MeshPartition::UMeshProviderModifier* BaseSection = Cast<MeshPartition::UMeshProviderModifier>(UncastBase.Get()))
				{
					HiddenBaseSections.Add(BaseSection);
					OnPreviewSectionReassigned(BaseSection, BaseSection->GetPreviewSection());
					BaseSection->OnPreviewSectionReassignment().AddWeakLambda(this, OnPreviewSectionReassigned);
				}
			}
		}
	}
	else // if we're unhiding
	{
		for (TWeakObjectPtr<MeshPartition::UMeshProviderModifier> HiddenBaseSection : HiddenBaseSections)
		{
			MeshPartition::UMeshProviderModifier* BaseSection = HiddenBaseSection.Get();
			if (!BaseSection)
			{
				continue;
			}
			OnPreviewSectionReassigned(BaseSection, BaseSection->GetPreviewSection());
			BaseSection->OnPreviewSectionReassignment().RemoveAll(this);
		}
		HiddenBaseSections.Reset();

		for (const TPair<TWeakObjectPtr<MeshPartition::APreviewSection>, TWeakObjectPtr<AActor>>& PreviewPair : PreviewActors)
		{
			if (AActor* PreviewActor = PreviewPair.Value.Get())
			{
				if (UWorld* World = PreviewActor->GetWorld())
				{
					World->DestroyActor(PreviewActor, false);
				}
			}
		}
		PreviewActors.Empty();
	}
}

void UModifierToolTarget::ConfigurePreviewForRendering(UPrimitiveComponent* PrimitiveComponent) const
{
	FChannelPacking::SetCustomPrimitiveData(PrimitiveComponent, ChannelTable, ChannelTexcoordDesc);
}

void UModifierToolTarget::UpdateRenderTextureForPreview(const FDynamicMesh3& InPreviewMesh)
{
	if (UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(TargetModifier->GetMeshPartitionEditorComponent()))
	{
		const UMeshPartitionDefinition* Definition = EditorComponent->GetMegaMeshDefinition() ? EditorComponent->GetMegaMeshDefinition() : UMeshPartitionDefinition::GetDefaultMegaMeshDefinition();

		const MeshPartition::FChannelMap ChannelMap = Definition->GetChannelMap();
		const float ChannelTexelSize = Definition->GetChannelTexelSize();

		constexpr bool bDownloadToAsset = false;

		if (!CachedUVTopology.IsSet())
		{
			const FTransformSRT3d ModifierTransform = (FTransformSRT3d)GetWorldTransform();
			CachedUVTopology = FChannelTextureRenderer::GenerateUVMeshTopology(InPreviewMesh, ModifierTransform.GetScale3D(), ChannelTexelSize);
		}

		Tasks::TTask<MeshPartition::FSectionChannels> RenderSectionChannels = FChannelTextureRenderer::BuildTextureForSectionWithCachedTopology(
			InPreviewMesh,
			CachedUVTopology.GetValue(),
			this,
			bDownloadToAsset,
			ChannelMap);

		RenderSectionChannels.Wait();

		MeshPartition::FSectionChannels SectionChannels = MoveTemp(RenderSectionChannels.GetResult());
		ChannelTable = MoveTemp(SectionChannels.Table);
		ChannelTexture = SectionChannels.Texture.Get();
		ChannelTexcoordDesc = SectionChannels.TexcoordMetrics;

		MID = MeshPartition::EditorUtils::GetOrCreateMaterialInstance(MID, EditorComponent->GetEditorOverrideMaterial(), this, TEXT("ModifierToolTargetMID"), RF_Transient);
		MID->SetTextureParameterValue(UE::MeshPartition::ChannelTextureParameterName, ChannelTexture);
	}
}

bool UModifierToolTarget::IsValid() const
{
	return TargetModifier && TargetModifier->GetMeshPartitionEditorComponent();
}


void UModifierToolTarget::BuildModifiedMeshUpToTarget(FDynamicMesh3& OutResultMesh, bool bIncludeTargetModifier)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UModifierToolTarget::AppendMeshUnderModifier);

	check(TargetModifier);

	if (UMeshPartitionEditorComponent* EditorComponent = Cast<UMeshPartitionEditorComponent>(TargetModifier->GetMeshPartitionEditorComponent()))
	{
		FTransformSRT3d EditorMeshTransform = (FTransformSRT3d)EditorComponent->GetOwner()->GetTransform();
		FTransformSRT3d ModifierTransform = (FTransformSRT3d)GetWorldTransform();

		TArray<Geometry::FOrientedBox3d> ModifierBounds = GetBounds();
		TArray<MeshPartition::UModifierComponent*> ModifiersToProcess = GetModifiersToProcess();

		MeshPartition::FBuilderSettings BuilderSettings;
		BuilderSettings.bCacheResult = true;
		
		const UMeshPartitionDefinition* Definition = EditorComponent->GetMegaMeshDefinition() ? EditorComponent->GetMegaMeshDefinition() : UMeshPartitionDefinition::GetDefaultMegaMeshDefinition();

		BuilderSettings.ModifiersToProcess = MoveTemp(ModifiersToProcess);
		BuilderSettings.Transform = EditorComponent->GetOwner()->GetTransform();
		BuilderSettings.ModifierFilter = UE::MeshPartition::FilterHelpers::FilterModifiersByLastModifierToBuild(*TargetModifier, bIncludeTargetModifier);
		BuilderSettings.FilterBounds = ModifierBounds;
		// Note: Large max section complexity to get a single contiguous/combined mesh
		BuilderSettings.MaxSectionComplexity = FMathd::MaxReal;
		BuilderSettings.TypePriorities = Definition->GetModifierTypePriorities();
		BuilderSettings.TexcoordGenerationOptions = FChannelCollectionUVLayoutOptions::GetFromDefinition(Definition);

		TArray<MeshPartition::FBuildTaskHandle> BuildTaskHandles = UE::MeshPartition::Build::LaunchBuilds(BuilderSettings);

		// We set the max section complexity to MaxReal so we should never have more than 1 section. This is important to ensure
		// a single global set of textures.
		if (!ensure(BuildTaskHandles.Num() == 1))
		{
			return;
		}

		MeshPartition::FBuildTaskHandle& TaskHandle = BuildTaskHandles[0];

		TSharedPtr<const FMeshData> BuiltMesh = TaskHandle.GetTask()->GetMesh();

		BuiltMesh->ConvertToDynamicMesh(OutResultMesh);
		MeshTransforms::ApplyTransform(OutResultMesh,
			[&EditorMeshTransform, &ModifierTransform](const FVector3d& Pos) -> FVector3d
			{
				return ModifierTransform.InverseTransformPosition(EditorMeshTransform.TransformPosition(Pos));
			},
			[&EditorMeshTransform, &ModifierTransform](const FVector3f Normal) -> FVector3f
			{
				return (FVector3f)ModifierTransform.InverseTransformNormal(EditorMeshTransform.TransformNormal((FVector3d)Normal));
			}
		);

		// Merge coincident wedges but do NOT weld attributes. The resulting mesh from the builder will have split vertices to support our different
		// uv islands. We simply want to merge vertices to avoid seams but keep the split uvs.
		FMergeCoincidentMeshEdges Merger(&OutResultMesh);
		Merger.bWeldAttrsOnMergedEdges = false;
		ensure(Merger.Apply());
	}
}

TArray<Geometry::FOrientedBox3d> UModifierToolTarget::GetBounds() const
{
	TArray<FBox> ModifierBounds = TargetModifier->ComputeBounds();
	TArray<Geometry::FOrientedBox3d> Bounds;
	Algo::Transform(ModifierBounds, Bounds, [Transform = TargetModifier->GetComponentToWorld()](const FBox& LocalBox)
		{
			Geometry::FOrientedBox3d OrientedBox(LocalBox); // Boxes returned by ComputeBounds are in world space already and orienting them would cause them to be larger than they should be.
			return OrientedBox;
		});
	return Bounds;
}

TArray<MeshPartition::UModifierComponent*> UModifierToolTarget::GetModifiersToProcess() const
{
	TArray<MeshPartition::UModifierComponent*> ModifiersToProcess;
	check(TargetModifier);
	if (UMeshPartitionEditorComponent* EditorComponent = TargetModifier->GetMeshPartitionEditorComponent())
	{
		EditorComponent->GetModifiersAffectingModifiers(ModifiersToProcess, { TargetModifier });
	}
	return ModifiersToProcess;
}

// -------------------------------------------------------------------------------------------------------------------------

FDynamicMesh3 UEditableModifierToolTarget::GetDynamicMesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEditableModifierToolTarget::GetDynamicMesh);

	FDynamicMesh3 Mesh;
	if (TargetModifier)
	{
		Mesh.EnableAttributes();
		BuildModifiedMeshUpToTarget(Mesh, false);
		if (MeshPartition::UEditableModifierBase* EditableModifier = Cast<MeshPartition::UEditableModifierBase>(TargetModifier))
		{
			EditableModifier->PrepareForEdit(Mesh);
		}

		UpdateRenderTextureForPreview(Mesh);
	}
	return Mesh;
}

void UEditableModifierToolTarget::CommitDynamicMesh(const FDynamicMesh3& CommitMesh, const FDynamicMeshCommitInfo& InCommitInfo)
{
	if (MeshPartition::UEditableModifierBase* EditableModifier = Cast<MeshPartition::UEditableModifierBase>(TargetModifier))
	{
		// Note that it is also the modifier's responsibility to create a transaction (if needed)
		// (the tool target can't do so because it doesn't know how the modifier stores the mesh)
		EditableModifier->ApplyEditWithMesh(CommitMesh);
	}
}

TArray<Geometry::FOrientedBox3d> UEditableModifierToolTarget::GetBounds() const
{
	if (MeshPartition::UEditableModifierBase* EditableModifier = Cast<MeshPartition::UEditableModifierBase>(TargetModifier))
	{
		return EditableModifier->GetBoundsForEdit();
	}
	return Super::GetBounds();
}

TArray<MeshPartition::UModifierComponent*> UEditableModifierToolTarget::GetModifiersToProcess() const
{
	TArray<MeshPartition::UModifierComponent*> ModifiersToProcess;
	if (UMeshPartitionEditorComponent* EditorComponent = TargetModifier->GetMeshPartitionEditorComponent())
	{
		// Note we expect this cast to succeed because the UEditableModifierToolTargetFactory::CanBuildTarget requires it
		MeshPartition::UEditableModifierBase* EditableModifier = Cast<MeshPartition::UEditableModifierBase>(TargetModifier);
		if (ensure(EditableModifier))
		{
			TArray<FBox> AxisAlignedBoxes;
			Algo::Transform(EditableModifier->GetBoundsForEdit(), AxisAlignedBoxes, [](const Geometry::FOrientedBox3d& OrientedBox)
				{
					return FBox(OrientedBox);
				});
			EditorComponent->GetModifiersAffectingBounds(ModifiersToProcess, AxisAlignedBoxes);
		}
	}
	return ModifiersToProcess;
}

// -------------------------------------------------------------------------------------------------------------------------

bool UModifierToolTargetFactory::CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) const
{
	const MeshPartition::UModifierComponent* Component = Cast<MeshPartition::UModifierComponent>(InSourceObject);
	return Component
		&& IsValidChecked(Component)
		&& !Component->IsUnreachable()
		&& Component->IsValidLowLevel()
		&& InRequirements.AreSatisfiedBy(MeshPartition::UModifierToolTarget::StaticClass());
}

UToolTarget* UModifierToolTargetFactory::BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements)
{
	MeshPartition::UModifierToolTarget* Target = NewObject<MeshPartition::UModifierToolTarget>();
	Target->Initialize(Cast<MeshPartition::UModifierComponent>(InSourceObject));
	checkSlow(Target->IsValid() && InRequirements.AreSatisfiedBy(Target));

	return Target;
}

// -------------------------------------------------------------------------------------------------------------------------

bool UEditableModifierToolTargetFactory::CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) const
{
	const MeshPartition::UEditableModifierBase* Component = Cast<MeshPartition::UEditableModifierBase>(InSourceObject);
	return Component
		&& IsValidChecked(Component)
		&& !Component->IsUnreachable()
		&& Component->IsValidLowLevel()
		&& Component->SupportsToolEditing()
		&& InRequirements.AreSatisfiedBy(MeshPartition::UEditableModifierToolTarget::StaticClass());
}

UToolTarget* UEditableModifierToolTargetFactory::BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements)
{
	MeshPartition::UEditableModifierToolTarget* Target = NewObject<MeshPartition::UEditableModifierToolTarget>();
	Target->Initialize(Cast<MeshPartition::UModifierComponent>(InSourceObject));
	checkSlow(Target->IsValid() && InRequirements.AreSatisfiedBy(Target));

	return Target;
}
} // namespace UE::MeshPartition
