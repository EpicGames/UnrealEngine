// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionMultiSectionToolTarget.h"

#include "Algo/Transform.h"
#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshTriangleAttribute.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "DynamicMeshEditor.h"
#include "HAL/IConsoleManager.h" // FAutoConsoleVariableRef
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionModifierUtils.h" // IsNonzeroWeightLayer
#include "MeshPartitionToolTargetUtils.h"
#include "MeshPartitionPreviewSection.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "ModelingToolTargetUtil.h"

namespace UE::MeshPartition
{
namespace MegaMeshMultiSectionToolTargetLocals
{
	const FName SectionIDOverlayAttributeName = TEXT("__SectionID__");

	// The multi section tool target typically has to be queried first so that it has
	//  a chance to create a single target out of inputs instead of multiple USectionToolTarget
	//  instances being created. However in the case of a single selected section, we
	//  may prefer to use USectionToolTarget, so this cvar can disallow building the multi
	//  section tool target for a single input object.
	bool bAllowForSingleObject = false;
	static FAutoConsoleVariableRef CVarAllowForSingleObject(
		TEXT("MeshPartition.MultiSectionToolTarget.AllowSingleObject"),
		bAllowForSingleObject,
		TEXT("Determines whether the multi-section tool target can be used for single sections (if not, the section tool target will be preferred)."));
}

void UMultiSectionToolTarget::Initialize(const TArray<TObjectPtr<MeshPartition::UMeshProviderModifier>> BaseSectionsIn)
{
	using namespace UE::Geometry;

	BaseSections.Reset(BaseSectionsIn.Num());
	PerSectionChannels.Reset(BaseSectionsIn.Num());

	for (const TObjectPtr<MeshPartition::UMeshProviderModifier>& BaseSection : BaseSectionsIn)
	{
		if (!::IsValid(BaseSection))
		{
			continue;
		}
		
		// Use the transform of the first valid section.
		if (BaseSections.IsEmpty())
		{
			TargetTransform = BaseSection->GetComponentToWorld();
		}
		BaseSections.Add(BaseSection);
		PerSectionChannels.Emplace();

		// Gather the weight maps used by this section
		const FDynamicMesh3* SectionMesh = BaseSection->GetMesh();
		if (!ensure(SectionMesh && SectionMesh->HasAttributes()))
		{
			continue;
		}
		if (const FDynamicMeshAttributeSet* Attributes = SectionMesh->Attributes())
		{
			for (int32 LayerIndex = 0; LayerIndex < Attributes->NumWeightLayers(); ++LayerIndex)
			{
				PerSectionChannels.Last().Add(Attributes->GetWeightLayer(LayerIndex)->GetName());
			}
		}
	}
}

Geometry::FDynamicMesh3 UMultiSectionToolTarget::BuildMergedMesh()
{
	using namespace MegaMeshMultiSectionToolTargetLocals;
	using namespace Geometry;

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UMultiSectionToolTarget::BuildMergedMesh);

	FDynamicMesh3 AccumulatedMesh;
	
	if (!IsValid())
	{
		return AccumulatedMesh; // empty
	}

	// We special case the case where we have a single base section, because we don't need to use
	//  a section ID overlay or do other processing.
	if (BaseSections.Num() == 1)
	{
		MeshPartition::UMeshProviderModifier* BaseSection = BaseSections[0].Get();
		if (ensure(BaseSection))
		{
			ensure(TargetTransform.Equals(BaseSection->GetComponentToWorld()));
			return *BaseSection->GetMesh();
		}
		return AccumulatedMesh; // empty
	}

	TDynamicMeshScalarTriangleAttribute<int32>* SectionIDOverlay = new TDynamicMeshScalarTriangleAttribute<int32>(&AccumulatedMesh);
	AccumulatedMesh.EnableAttributes();
	FDynamicMeshAttributeSet* AccumulatedAttributes = AccumulatedMesh.Attributes();
	AccumulatedAttributes->AttachAttribute(SectionIDOverlayAttributeName, SectionIDOverlay);
	SectionIDOverlay->Initialize(INDEX_NONE);
	
	for (int32 SectionIndex = 0; SectionIndex < BaseSections.Num(); ++SectionIndex)
	{
		MeshPartition::UMeshProviderModifier* Section = BaseSections[SectionIndex].Get();
		if (!Section)
		{
			continue;
		}

		const FDynamicMesh3* ModifierMesh = Section->GetMesh();
		if (ModifierMesh == nullptr)
		{
			continue;
		}
		
		// Before appending, make sure that our appended mesh has all the channels that
		//  the section mesh does.
		if (const FDynamicMeshAttributeSet* Attributes = ModifierMesh->Attributes())
		{
			AccumulatedAttributes->EnableMatchingWeightLayersByNames(Attributes, /*bDiscardUnmatched*/ false);
		}

		FDynamicMesh3::FAppendInfo AppendInfo;
		AccumulatedMesh.AppendWithOffsets(*ModifierMesh, &AppendInfo);

		// Assign the section id for all the triangles of the newly appended section
		for (int32 Tid = AppendInfo.TriangleOffset; Tid < AccumulatedMesh.MaxTriangleID(); ++Tid)
		{
			if (!AccumulatedMesh.IsTriangle(Tid))
			{
				continue;
			}
			SectionIDOverlay->SetValue(Tid, SectionIndex);
		}

		// Transform the section into target transform space, unless the transform already matches
		const FTransform& SectionToWorld = Section->GetComponentToWorld();
		if (!SectionToWorld.Equals(TargetTransform))
		{
			for (int32 Vid = AppendInfo.VertexOffset; Vid < AccumulatedMesh.MaxVertexID(); ++Vid)
			{
				if (!AccumulatedMesh.IsVertex(Vid))
				{
					continue;
				}
				AccumulatedMesh.SetVertex(Vid, 
					TargetTransform.InverseTransformPosition(
						SectionToWorld.TransformPosition(AccumulatedMesh.GetVertex(Vid))));
			}
		}
	}

	// Run an edge merger
	FMergeCoincidentMeshEdges Merger(&AccumulatedMesh);
	Merger.MergeVertexTolerance = 0.0001;
	Merger.MergeSearchTolerance = 2 * Merger.MergeVertexTolerance;
	Merger.OnlyUniquePairs = false;
	Merger.bWeldAttrsOnMergedEdges = true;

	ensure(Merger.Apply());

	return AccumulatedMesh;
}

void UMultiSectionToolTarget::SplitMeshesToSections(const Geometry::FDynamicMesh3& AccumulatedMesh)
{
	using namespace MegaMeshMultiSectionToolTargetLocals;
	using namespace Geometry;

	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UMultiSectionToolTarget::SplitMeshesToSections);

	// We special case the case where we have a single base section, because we don't need to use
	//  a section ID overlay.
	if (BaseSections.Num() == 1)
	{
		MeshPartition::UMeshProviderModifier* BaseSection = BaseSections[0].Get();
		if (ensure(BaseSection))
		{
			ensure(TargetTransform.Equals(BaseSection->GetComponentToWorld()));
			BaseSection->SetMesh(FDynamicMesh3(AccumulatedMesh), /*bEmitChange*/ true);
		}
		return;
	}


	const TDynamicMeshScalarTriangleAttribute<int32>* SectionIDs = (AccumulatedMesh.HasAttributes())
		? static_cast<const TDynamicMeshScalarTriangleAttribute<int32>*>(AccumulatedMesh.Attributes()->GetAttachedAttribute(SectionIDOverlayAttributeName))
		: nullptr;

	TArray<int32> FallbackSectionIDs;
	if (!SectionIDs)
	{
		// tool lost/destroyed our section ID attribute -- fall back to assigning triangles by source modifier bounds
		TArray<FAxisAlignedBox3d> SectionBounds;
		SectionBounds.Init(FAxisAlignedBox3d::Empty(), BaseSections.Num());
		ParallelFor(BaseSections.Num(), [&](int32 SectionIndex)
		{
			const UMeshProviderModifier* Section = BaseSections[SectionIndex].Get();
			if (!Section || !Section->GetMesh())
			{
				return;
			}
			const FDynamicMesh3* ModifierMesh = Section->GetMesh();
			const FTransform& SectionToWorld = Section->GetComponentToWorld();
			for (int32 VID : ModifierMesh->VertexIndicesItr())
			{
				SectionBounds[SectionIndex].Contain(
					TargetTransform.InverseTransformPosition(
						SectionToWorld.TransformPosition(FVector(ModifierMesh->GetVertex(VID))
				)) );
			}
		});

		FallbackSectionIDs = AssignMeshTrisToClosestBounds(AccumulatedMesh, SectionBounds);
	}

	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
	TArray<int32> SplitIdxToSectionIdx, SectionIdxToSplitIdx;
	constexpr bool bSplitIfSingle = true;
	const bool bWasSplit = FDynamicMeshEditor::SplitMesh(&AccumulatedMesh, SplitMeshes, bSplitIfSingle,
		[SectionIDs, &FallbackSectionIDs](int TID) -> int32
		{
			return SectionIDs ? SectionIDs->GetValue(TID) : FallbackSectionIDs[TID];
		},
		// DeleteMeshID, not used
		-1,
		// Will be 1:1 with SplitMeshes
		&SplitIdxToSectionIdx);

	ensure(bWasSplit);

	// build reverse map
	SectionIdxToSplitIdx.Init(INDEX_NONE, BaseSections.Num());
	for (int32 SplitIdx = 0; SplitIdx < SplitIdxToSectionIdx.Num(); ++SplitIdx)
	{
		int32 SectionIdx = SplitIdxToSectionIdx[SplitIdx];
		if (ensure(SectionIdxToSplitIdx.IsValidIndex(SectionIdx)))
		{
			SectionIdxToSplitIdx[SectionIdx] = SplitIdx;
		}
	}

	for (int32 SectionID = 0; SectionID < BaseSections.Num(); ++SectionID)
	{
		MeshPartition::UMeshProviderModifier* BaseSection = BaseSections[SectionID].Get();
		if (!BaseSection)
		{
			continue;
		}

		int32 SplitIdx = SectionIdxToSplitIdx[SectionID];
		if (SplitIdx == INDEX_NONE)
		{
			// This section had no triangles in the result mesh; delete it
			UE::ToolTarget::SafeDeleteActor(BaseSection->GetOwner());
			BaseSections[SectionID] = nullptr;
			continue;
		}

		FDynamicMesh3& SectionMesh = *SplitMeshes[SplitIdx];

		// Transform back into section space if needed
		if (!BaseSection->GetComponentToWorld().Equals(TargetTransform))
		{
			const FTransform3d& SectionToWorld = BaseSection->GetComponentToWorld();
			for (int32 Vid : SectionMesh.VertexIndicesItr())
			{
				SectionMesh.SetVertex(Vid,
					SectionToWorld.InverseTransformPosition(
						TargetTransform.TransformPosition(SectionMesh.GetVertex(Vid))));
			}
		}

		// Remove any unmodified weight layers that weren't on this section before
		FDynamicMeshAttributeSet* Attributes = SectionMesh.Attributes();
		if (ensure(Attributes))
		{
			for (int32 LayerIndex = Attributes->NumWeightLayers() - 1; LayerIndex >= 0; --LayerIndex)
			{
				const FDynamicMeshWeightAttribute* Layer = Attributes->GetWeightLayer(LayerIndex);
				if (!PerSectionChannels[SectionID].Contains(Layer->GetName()))
				{
					if (UE::MeshPartition::Utils::IsNonzeroWeightLayer(SectionMesh, LayerIndex))
					{
						// The weight layer was modified, so we keep it.
						PerSectionChannels[SectionID].Add(Layer->GetName());
					}
					else
					{
						// Layer stayed empty (all zeros) so don't add it
						Attributes->RemoveWeightLayer(LayerIndex);
					}
				}
			}
		}//end pruning unused weight channels
		
		BaseSection->SetMesh(MoveTemp(SectionMesh), /*bEmitChange*/ true);
	}
}

bool UMultiSectionToolTarget::IsValid() const
{
	for (const TWeakObjectPtr<MeshPartition::UMeshProviderModifier>& SectionWeak : BaseSections)
	{
		if (SectionWeak.IsValid() && SectionWeak->GetMesh())
		{
			return true;
		}
	}

	return false;
}

UPrimitiveComponent* UMultiSectionToolTarget::GetOwnerComponent() const
{
	// Use the first valid section we find
	for (const TWeakObjectPtr<MeshPartition::UMeshProviderModifier>& SectionWeak : BaseSections)
	{
		if (SectionWeak.IsValid())
		{
			return SectionWeak.Get();
		}
	}
	return nullptr;
}

USceneComponent* UMultiSectionToolTarget::GetOwnerSceneComponent() const
{
	return GetOwnerComponent();
}

AActor* UMultiSectionToolTarget::GetOwnerActor() const
{
	// Use the first valid section we find
	for (const TWeakObjectPtr<MeshPartition::UMeshProviderModifier>& SectionWeak : BaseSections)
	{
		if (SectionWeak.IsValid())
		{
			return SectionWeak->GetOwner();
		}
	}
	return nullptr;
}

void UMultiSectionToolTarget::SetOwnerVisibility(bool bVisible) const
{
	// Show/hide preview sections, using the reassignment callback to keep them hidden if new ones are assigned

	auto SetPreviewSectionVisibility = [bVisible](MeshPartition::UMeshProviderModifier*, MeshPartition::APreviewSection* PreviewSection)
	{
		if (PreviewSection && PreviewSection->GetRootComponent())
		{
			PreviewSection->GetRootComponent()->SetVisibility(bVisible, /*bPropagateToChildren*/ true);
		}
	};

	for (const TWeakObjectPtr<MeshPartition::UMeshProviderModifier>& SectionWeak : BaseSections)
	{
		MeshPartition::UMeshProviderModifier* BaseSection = SectionWeak.Get();
		if (!BaseSection)
		{
			continue;
		}

		SetPreviewSectionVisibility(BaseSection, BaseSection->GetPreviewSection());

		if (!bVisible)
		{
			// Weak so that it goes away if our target goes away without resetting visibility.
			BaseSection->OnPreviewSectionReassignment().AddWeakLambda(this, SetPreviewSectionVisibility);
		}
		else
		{
			BaseSection->OnPreviewSectionReassignment().RemoveAll(this);
		}
	}
}

FTransform UMultiSectionToolTarget::GetWorldTransform() const
{
	return TargetTransform;
}

bool UMultiSectionToolTarget::HitTestComponent(const FRay& WorldRay, FHitResult& OutHit) const
{
	// Line trace all the base sections and take the closest one
	FVector End = WorldRay.PointAt(HALF_WORLD_MAX);
	bool bHit = false;
	for (const TWeakObjectPtr<MeshPartition::UMeshProviderModifier>& SectionWeak : BaseSections)
	{
		MeshPartition::UMeshProviderModifier* BaseSection = SectionWeak.Get();
		if (!BaseSection)
		{
			continue;
		}
		FHitResult HitResult;
		if (BaseSection->LineTraceComponent(HitResult, WorldRay.Origin, End, FCollisionQueryParams(SCENE_QUERY_STAT(HitTest), true)))
		{
			bHit = true;
			End = HitResult.Location;
		}
	}

	return bHit;
}


Geometry::FDynamicMesh3 UMultiSectionToolTarget::GetDynamicMesh()
{
	return BuildMergedMesh();
}

void UMultiSectionToolTarget::CommitDynamicMesh(const Geometry::FDynamicMesh3& InMesh, const FDynamicMeshCommitInfo& InCommitInfo)
{
	SplitMeshesToSections(InMesh);
}

// Could consider returning the megamesh material from getters, but we wouldn't want to set it,
//  so for now all these material functions don't do anything.
int32 UMultiSectionToolTarget::GetNumMaterials() const
{
	return 0;
}
UMaterialInterface* UMultiSectionToolTarget::GetMaterial(int32 InMaterialIndex) const
{
	return nullptr;
}
void UMultiSectionToolTarget::GetMaterialSet(FComponentMaterialSet& OutMaterialSet, bool bInPreferAssetMaterials) const
{
}
bool UMultiSectionToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& InMaterialSet, bool bInApplyToAsset)
{
	return false;
}



bool UMultiSectionToolTargetFactory::CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const
{
	const MeshPartition::UMeshProviderModifier* Section = Cast<const MeshPartition::UMeshProviderModifier>(SourceObject);
	return Section && Section->GetMesh() && MegaMeshMultiSectionToolTargetLocals::bAllowForSingleObject;
}

UToolTarget* UMultiSectionToolTargetFactory::BuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo)
{
	TArray<bool> WasUsed;
	return BuildFirstTarget({ SourceObject }, TargetTypeInfo, WasUsed);
}

int32 UMultiSectionToolTargetFactory::CanBuildTargets(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WouldBeUsedOut)
{
	if (!TargetTypeInfo.AreSatisfiedBy(UMultiSectionToolTarget::StaticClass()))
	{
		return 0;
	}

	int32 NumSections = 0;
	WouldBeUsedOut.SetNum(InputObjects.Num());
	for (int32 ObjectIndex = 0; ObjectIndex < InputObjects.Num(); ++ObjectIndex)
	{
		const MeshPartition::UMeshProviderModifier* Section = Cast<const MeshPartition::UMeshProviderModifier>(InputObjects[ObjectIndex]);
		WouldBeUsedOut[ObjectIndex] = IsValid(Section) && Section->GetMesh();
		NumSections += WouldBeUsedOut[ObjectIndex];
	}
	// We build a single combined target from all sections
	return NumSections <= 0 ? 0
		: NumSections == 1 ? MegaMeshMultiSectionToolTargetLocals::bAllowForSingleObject
		: 1; // when NumSections > 1
}

TArray<UToolTarget*> UMultiSectionToolTargetFactory::BuildTargets(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WasUsedOut)
{
	UToolTarget* Target = BuildFirstTarget(InputObjects, TargetTypeInfo, WasUsedOut);
	if (Target)
	{
		return { Target };
	}
	return {};
}

UToolTarget* UMultiSectionToolTargetFactory::BuildFirstTarget(const TArray<UObject*>& InputObjects, const FToolTargetTypeRequirements& TargetTypeInfo, TArray<bool>& WasUsedOut)
{
	int32 Count = CanBuildTargets(InputObjects, TargetTypeInfo, WasUsedOut);
	if (Count == 0)
	{
		return nullptr;
	}
	ensure(Count == 1);

	TArray<TObjectPtr<MeshPartition::UMeshProviderModifier>> Sections;
	UMultiSectionToolTarget* Target = NewObject<UMultiSectionToolTarget>();
	for (int32 ObjectIndex = 0; ObjectIndex < InputObjects.Num(); ++ObjectIndex)
	{
		MeshPartition::UMeshProviderModifier* Section = Cast<MeshPartition::UMeshProviderModifier>(InputObjects[ObjectIndex]);
		if (Section && Section->GetMesh())
		{
			ensure(WasUsedOut[ObjectIndex]);
			Sections.Add(Section);
		}
	}
	Target->Initialize(Sections);

	return Target;
}
} // namespace UE::MeshPartition