// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPlaceModifierTool.h"

#include "InteractiveToolManager.h"

#include "BaseGizmos/TransformGizmoUtil.h"
#include "BaseGizmos/TransformProxy.h"

#include "MeshPartitionComponentBackedTarget.h"
#include "MeshPartitionSectionToolTarget.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "Selection/ToolSelectionUtil.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "ModelingToolTargetUtil.h"

#include "Components/SplineComponent.h"

#include "Modifiers/MeshPartitionPatchModifier.h"
#include "Modifiers/MeshPartitionInstancedPatchModifier.h"
#include "Modifiers/MeshPartitionMeshProjectModifier.h"
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h"
#include "Modifiers/MeshPartitionRemeshModifier.h"
#include "Modifiers/MeshPartitionSplineModifier.h"
#include "Modifiers/MeshPartitionTexturePatchModifier.h"
#include "Modifiers/MeshPartitionNoiseModifier.h"
#include "Modifiers/MeshPartitionBooleanModifier.h"
#include "Modifiers/MeshPartitionLatticeModifier.h"
#include "Modifiers/MeshPartitionSplineRemeshModifier.h"
#include "Modifiers/MeshPartitionWeightUtilityModifier.h"

#include "MeshPartitionTedsFactory.h"
#include "Columns/LayerOutlinerColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include "MeshPartitionModifierActor.h"
#include "MeshPartitionEditorComponent.h"

#include "Mechanics/DragAlignmentMechanic.h"

#include "MeshPartition.h"
#include "ModelingObjectsCreationAPI.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionPlaceModifierTool)

#define LOCTEXT_NAMESPACE "UPlaceModifierTool"

using namespace UE::Geometry;


namespace UE::MeshPartition
{
USingleSelectionMeshEditingTool* UPlaceModifierToolBuilder::CreateNewTool(const FToolBuilderState& SceneState) const
{
	MeshPartition::UPlaceModifierTool* PlaceModifierTool = NewObject<MeshPartition::UPlaceModifierTool>(SceneState.ToolManager);
	PlaceModifierTool->SetDefaultModifierType(DefaultModifierTypeID);
	return PlaceModifierTool;
}


const FToolTargetTypeRequirements& UPlaceModifierToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
		UMaterialProvider::StaticClass(),
		UDynamicMeshProvider::StaticClass(),
		UPrimitiveComponentBackedTarget::StaticClass(),
		UMeshPartitionComponentBackedTarget::StaticClass(),
		});

	return TypeRequirements;
}



UPlaceModifierTool::UPlaceModifierTool()
{

}

void UPlaceModifierTool::Setup()
{
	Super::Setup();

	AActor* TargetActor = UE::ToolTarget::GetTargetActor(Target);
	AMeshPartition* MegaMesh = Cast<AMeshPartition>(TargetActor);
	if (!MegaMesh)
	{
		MegaMesh = Cast<AMeshPartition>(TargetActor->GetAttachParentActor());
	}
	TargetMegaMesh = MegaMesh;

	Settings = NewObject<MeshPartition::UPlaceModifierToolProperties>(this);
	AddToolPropertySource(Settings);
	Settings->RestoreProperties(this);

	UTransformProxy* TransformProxy = NewObject<UTransformProxy>(this);
	TransformProxy->OnTransformChanged.AddWeakLambda(this, [this](UTransformProxy*, FTransform NewTransform)
		{
		});


	Gizmo = UE::TransformGizmoUtil::CreateCustomRepositionableTransformGizmo(GetToolManager(),
		ETransformGizmoSubElements::TranslateAllAxes | ETransformGizmoSubElements::FreeTranslate, this);
	Gizmo->SetActiveTarget(TransformProxy, GetToolManager());
	Gizmo->ReinitializeGizmoTransform(TargetActor->GetActorTransform());

	DragAlignmentMechanic = NewObject<UDragAlignmentMechanic>(this);
	DragAlignmentMechanic->Setup(this);
	DragAlignmentMechanic->AddToGizmo(Gizmo);

	// register modifier types
	constexpr int NumModifiers = static_cast<int>(MeshPartition::EModifierClassType::LastModifier);
	for (int ModID = 0; ModID < NumModifiers; ModID++)
	{
		MeshPartition::EModifierClassType Value = static_cast<MeshPartition::EModifierClassType>(ModID);
		ModifierTypes.Add(Value);
	}

	// when applicable, set a given modifier type on initial setup
	if (DefaultModifierTypeID >= 0 && ensure(DefaultModifierTypeID < ModifierTypes.Num()))
	{
		Settings->ModifierType = ModifierTypes[DefaultModifierTypeID];
	}
}

void UPlaceModifierTool::Shutdown(EToolShutdownType InShutdownType)
{
	if (InShutdownType == EToolShutdownType::Accept)
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("PlaceMeshModifier", "Place Modifier"));

		MeshPartition::AModifierActor* TemplateActor = NewObject<MeshPartition::AModifierActor>(this);

		MeshPartition::AModifierActor* NewActor;
		{
			FCreateActorParams CreateActorParams;
			CreateActorParams.TargetWorld = TargetWorld.Get();
			CreateActorParams.BaseName = StaticEnum<MeshPartition::EModifierClassType>()->GetNameStringByValue(static_cast<int64>(Settings->ModifierType));
			CreateActorParams.Transform = Gizmo->GetGizmoTransform();
			CreateActorParams.TemplateAsset = TemplateActor; // With no template, we create a basic AActor instance
			FCreateActorResult Result = UE::Modeling::CreateNewActor(GetToolManager(), MoveTemp(CreateActorParams));
			NewActor= StaticCast<MeshPartition::AModifierActor*>(Result.NewActor);
		}
		if(NewActor != nullptr)
		{
			FCreateComponentParams Params;
			Params.HostActor = NewActor;
			Params.BaseName = ""; //Use the autogenerated name instead
			Params.bSetAsRoot = false;
			Params.bTransact = true;
			Params.ComponentClass = GetModifierToCreateClass();

			FCreateComponentResult Result = UE::Modeling::CreateNewComponentOnActor(GetToolManager(), MoveTemp(Params));

			MeshPartition::UModifierComponent* ModifierComponent = StaticCast< MeshPartition::UModifierComponent*>(Result.NewComponent);

			SetupNewModifier(NewActor, ModifierComponent);

			if (ModifierComponent && TargetMegaMesh.IsValid())
			{
				ModifierComponent->Modify(true);
				ModifierComponent->SetAffectedMeshPartition(TargetMegaMesh.Get());
				ModifierComponent->PostEditChange();
				AssignToActiveLayer(ModifierComponent);

				UMeshPartitionEditorComponent* EditorComponent = StaticCast<UMeshPartitionEditorComponent*>(TargetMegaMesh->GetMeshPartitionComponent());
				if (EditorComponent)
				{
					EditorComponent->OnModifierAssigned();
				}

			}		

			NewActor->Modifier = ModifierComponent;
		}

		if (NewActor != nullptr)
		{
			// select newly-created object
			ToolSelectionUtil::SetNewActorSelection(GetToolManager(), NewActor);
		}

		GetToolManager()->EndUndoTransaction();

	}

	Settings->SaveProperties(this);
	Settings = nullptr;

	DragAlignmentMechanic->Shutdown();
	DragAlignmentMechanic = nullptr;

	GetToolManager()->GetPairedGizmoManager()->DestroyAllGizmosByOwner(this);
	Gizmo = nullptr;

	Super::Shutdown(InShutdownType);

}

void UPlaceModifierTool::Render(IToolsContextRenderAPI* RenderAPI)
{
	DragAlignmentMechanic->Render(RenderAPI);
}

void UPlaceModifierTool::SetDefaultModifierType(int32 InModifierTypeID)
{
	DefaultModifierTypeID = InModifierTypeID;
}

TObjectPtr<UClass> UPlaceModifierTool::GetModifierToCreateClass()
{
	switch (Settings->ModifierType)
	{
	case MeshPartition::EModifierClassType::Remesh:
		return MeshPartition::URemeshModifier::StaticClass();
	case MeshPartition::EModifierClassType::Patch:
		return MeshPartition::UPatchModifier::StaticClass();
	case MeshPartition::EModifierClassType::Project:
		return MeshPartition::UMeshProjectModifier::StaticClass();
	case MeshPartition::EModifierClassType::InstancedPatch:
		return MeshPartition::UInstancedPatchModifier::StaticClass();
	case MeshPartition::EModifierClassType::TexturePatch:
		return MeshPartition::UTexturePatchModifier::StaticClass();
	case MeshPartition::EModifierClassType::Spline:
		return MeshPartition::USplineModifier::StaticClass();
	case MeshPartition::EModifierClassType::MeshLayer:
		return MeshPartition::UProjectMeshLayersModifier::StaticClass();
	case MeshPartition::EModifierClassType::Noise:
		return MeshPartition::UNoiseModifier::StaticClass();
	case MeshPartition::EModifierClassType::Boolean:
		return MeshPartition::UBooleanModifier::StaticClass();
	case MeshPartition::EModifierClassType::Lattice:
		return MeshPartition::ULatticeModifier::StaticClass();
	case MeshPartition::EModifierClassType::SplineRemesh:
		return MeshPartition::USplineRemeshModifier::StaticClass();
	case MeshPartition::EModifierClassType::WeightUtility:
		return MeshPartition::UWeightUtilityModifier::StaticClass();
	default:
		ensure(false);
		return nullptr;
	}
}


void UPlaceModifierTool::SetupNewModifier(AActor* ContainingActor, MeshPartition::UModifierComponent* Component)
{
	auto SetScaledSpline = [this](USplineComponent & Component)
	{
		if (const MeshPartition::USectionToolTarget* const SectionToolTarget = Cast<MeshPartition::USectionToolTarget>(Target))
		{
			if (const USceneComponent* const SC = SectionToolTarget->GetOwnerSceneComponent())
			{
				const double SplineLength = 0.5 * SC->Bounds.BoxExtent.GetMax();
		
				const FVector StartPoint(0.0, 0.0, 0.0);
				const FVector EndPoint(SplineLength, 0.0, 0.0);
				constexpr float StartParam = 0.f;
				constexpr float EndParam = 1.f;

				Component.ClearSplinePoints(false);
				Component.AddPoint(FSplinePoint(StartParam, StartPoint), false);
				Component.AddPoint(FSplinePoint(EndParam, EndPoint), false);
				Component.UpdateSpline();
			}
		}
	};


	switch (Settings->ModifierType)
	{
	case MeshPartition::EModifierClassType::Remesh:
		// Nothing to do.
		break;
	case MeshPartition::EModifierClassType::Patch:
		// Nothing to do.
		break;
	case MeshPartition::EModifierClassType::Project:
		break;
	case MeshPartition::EModifierClassType::InstancedPatch:
		// Nothing to do.
		break;
	case MeshPartition::EModifierClassType::TexturePatch:
		// Nothing to do.
		break;
	case MeshPartition::EModifierClassType::Spline:		
	{
		FCreateComponentParams Params;
		Params.HostActor = ContainingActor;
		Params.BaseName = ""; //Use the autogenerated name instead
		Params.bSetAsRoot = false;
		Params.bTransact = false;
		Params.ComponentClass = USplineComponent::StaticClass();

		FCreateComponentResult Result = UE::Modeling::CreateNewComponentOnActor(GetToolManager(), MoveTemp(Params));
		USplineComponent* NewSplineComponent = StaticCast<USplineComponent*>(Result.NewComponent);
		SetScaledSpline(*NewSplineComponent);

		MeshPartition::USplineModifier* SplineModifier = StaticCast<MeshPartition::USplineModifier*>(Component);
		SplineModifier->SetSplineComponent(NewSplineComponent, false /*don't update now*/);
	}
		break;
	case MeshPartition::EModifierClassType::MeshLayer:
		break;
	case MeshPartition::EModifierClassType::Noise:
		break;
	case MeshPartition::EModifierClassType::Boolean:
		break;
	case MeshPartition::EModifierClassType::Lattice:
		break;
	case MeshPartition::EModifierClassType::SplineRemesh:
	{
		FCreateComponentParams Params;
		Params.HostActor = ContainingActor;
		Params.BaseName = ""; //Use the autogenerated name instead
		Params.bSetAsRoot = false;
		Params.bTransact = false;
		Params.ComponentClass = USplineComponent::StaticClass();

		FCreateComponentResult Result = UE::Modeling::CreateNewComponentOnActor(GetToolManager(), MoveTemp(Params));
		USplineComponent* NewSplineComponent = StaticCast<USplineComponent*>(Result.NewComponent);
		SetScaledSpline(*NewSplineComponent);

		MeshPartition::USplineRemeshModifier* SplineRemeshModifier = StaticCast<MeshPartition::USplineRemeshModifier*>(Component);
		SplineRemeshModifier->SetSplineComponent(NewSplineComponent, false /*don't update now*/);
	}
	case MeshPartition::EModifierClassType::WeightUtility:
		break;
	default:
		ensure(false);
	}
}

void UPlaceModifierTool::AssignToActiveLayer(MeshPartition::UModifierComponent* ModifierComponent)
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage =  GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	const UMegaMeshTedsFactory* Factory = DataStorage->FindFactory<UMegaMeshTedsFactory>();

	if (!Factory)
	{
		return;
	}

	RowHandle ActiveLayer = InvalidRowHandle;
	DataStorage->RunQuery(Factory->ActiveLayerQuery, Queries::CreateDirectQueryCallbackBinding(
		[&ActiveLayer](IDirectQueryContext& Context, const RowHandle Row)
		{
			ensure(ActiveLayer == InvalidRowHandle);
			ActiveLayer = Row;
		}));

	if (ActiveLayer == InvalidRowHandle)
	{
		return;
	}

	if (FMegaMeshLayerNameColumn* NameCol = DataStorage->GetColumn<FMegaMeshLayerNameColumn>(ActiveLayer); NameCol != nullptr)
	{
		ModifierComponent->SetType(NameCol->Name);
	}
}
} // namespace UE::MeshPartition


#undef LOCTEXT_NAMESPACE