// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionPreviewUtils.h"

#include "Drawing/PreviewGeometryActor.h"

namespace UE::MeshPartition
{
	void CreatePreviewGridLines(const FBox& Box, FIntVector Dims, const FString& LineGroupName, UPreviewGeometry* PreviewGeometry)
	{
		Dims = Dims.ComponentMax(FIntVector(1));

		PreviewGeometry->CreateOrUpdateLineSet(LineGroupName, 1, [&Box, &Dims](int32 Index, TArray<FRenderableLine>& LinesOut)
			{
				const float Thickness = 3.f;
				const FColor LineColor = FColor::Orange;

				const FIntVector LinesPer = Dims + FIntVector(1);
				const int32 NumEdges = LinesPer.X * LinesPer.Y + LinesPer.X * LinesPer.Z + LinesPer.Y * LinesPer.Z;
				LinesOut.Reserve(NumEdges);
				const FVector BoxDiag = Box.Max - Box.Min;
				// cycle through dims, drawing a grid of lines across the other dimensions
				for (int32 A = 0, B = 1, C = 2; A < 3; B = C, C = A++)
				{
					FVector StartPt = Box.Min, EndPt = Box.Min;
					EndPt[A] = Box.Max[A];
					const double BStep = BoxDiag[B] / (double)(LinesPer[B] - 1);
					const double CStep = BoxDiag[C] / (double)(LinesPer[C] - 1);
					for (int32 BIdx = 0; BIdx < LinesPer[B]; ++BIdx)
					{
						EndPt[B] = StartPt[B] = Box.Min[B] + BStep * BIdx;
						for (int32 CIdx = 0; CIdx < LinesPer[C]; ++CIdx)
						{
							EndPt[C] = StartPt[C] = Box.Min[C] + CStep * CIdx;
							LinesOut.Add(FRenderableLine(StartPt, EndPt, LineColor, Thickness));
						}
					}
				}
			}
		);
	}
}
