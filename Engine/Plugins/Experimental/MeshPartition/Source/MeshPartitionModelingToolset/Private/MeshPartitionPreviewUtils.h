// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/Box.h"
#include "Math/IntVector.h"

class UPreviewGeometry;

namespace UE::MeshPartition
{
	void CreatePreviewGridLines(const FBox& Box, FIntVector Dims, const FString& LineGroupName, UPreviewGeometry* PreviewGeometry);
}
