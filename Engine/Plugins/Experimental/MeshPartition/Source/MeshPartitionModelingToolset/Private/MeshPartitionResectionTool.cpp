// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionResectionTool.h"

#include "DynamicMeshEditor.h"
#include "InteractiveToolManager.h"
#include "MeshPartition.h"
#include "MeshPartitionComponentBackedTarget.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionPreviewUtils.h"
#include "MeshPartitionSectionToolTarget.h"
#include "ModelingToolTargetUtil.h"
#include "ToolTargetManager.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"
#include "GameFramework/Actor.h"
#include "Components/PrimitiveComponent.h"
#include "Misc/ScopedSlowTask.h"
#include "Selection/ToolSelectionUtil.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionResectionTool)

#define LOCTEXT_NAMESPACE "MeshPartitionResectionTool"

using namespace UE::Geometry;

namespace UE::MeshPartition
{
const FToolTargetTypeRequirements& UMeshPartitionResectionToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshProvider::StaticClass(),
			UPrimitiveComponentBackedTarget::StaticClass(),
			UMeshPartitionComponentBackedTarget::StaticClass(),
		});

	return TypeRequirements;
}

bool UMeshPartitionResectionToolBuilder::CanBuildTool(const FToolBuilderState& InSceneState) const
{
	return (InSceneState.TargetManager->CountSelectedAndTargetable(InSceneState, GetTargetRequirements()) >= 1);
}

UMultiSelectionMeshEditingTool* UMeshPartitionResectionToolBuilder::CreateNewTool(const FToolBuilderState& InSceneState) const
{
	return NewObject<UMeshPartitionResectionTool>(InSceneState.ToolManager);
}

// -------------------------------------------------------------------------------------------------------------------------

void UMeshPartitionResectionTool::Setup()
{
	Super::Setup();

	Properties = NewObject<UMeshPartitionResectionToolProperties>(this);
	AddToolPropertySource(Properties);
	Properties->RestoreProperties(this);

	Properties->WatchProperty(Properties->SectionLayout, [this](FIntVector SectionLayout)
	{
		bPreviewGeometryNeedsUpdate = true;
	});

	PreviewGeometry = NewObject<UPreviewGeometry>(this);

	if (!ensure(!Targets.IsEmpty()))
	{
		return;
	}

	// Setup world-aligned grid
	// TODO: Optionally support a target-aligned grid, and/or custom alignment.
	USceneComponent* Target0Component = ToolTarget::GetTargetSceneComponent(Targets[0]);
	PreviewGeometry->CreateInWorld(Target0Component->GetWorld(), FTransform::Identity);
	ResectionBounds = Target0Component->Bounds.GetBox();
	// append bounds of any additional targets, in the local space of the first target
	for (int32 TargetIdx = 1; TargetIdx < Targets.Num(); ++TargetIdx)
	{
		USceneComponent* SceneComp = ToolTarget::GetTargetSceneComponent(Targets[TargetIdx]);
		FBox TargetBox = SceneComp->Bounds.GetBox();
		ResectionBounds += TargetBox;
	}
	CreateOrUpdatePreviewGeometry();
}

void UMeshPartitionResectionTool::OnTick(float DeltaTime)
{
	if (bPreviewGeometryNeedsUpdate)
	{
		CreateOrUpdatePreviewGeometry();
		bPreviewGeometryNeedsUpdate = false;
	}
}

void UMeshPartitionResectionTool::CreateOrUpdatePreviewGeometry()
{
	CreatePreviewGridLines(ResectionBounds, GetClampedSectionLayout(), TEXT("SectionDivisionLines"), PreviewGeometry);
}

void UMeshPartitionResectionTool::OnShutdown(EToolShutdownType InShutdownType)
{
	Properties->SaveProperties(this);

	if (InShutdownType == EToolShutdownType::Accept)
	{
		Resection();
	}

	if (PreviewGeometry)
	{
		PreviewGeometry->Disconnect();
	}
}

void UMeshPartitionResectionTool::Resection()
{
	bool bNeedsMerge = Targets.Num() > 1;
	FIntVector UseSectionLayout = Properties->SectionLayout.ComponentMax(FIntVector(1, 1, 1));
	bool bNeedsSplit = UseSectionLayout.GetMax() > 1;
	if (!bNeedsMerge && !bNeedsSplit)
	{
		// nothing to do
		return;
	}

	static FGetMeshParameters GetMeshParams;
	GetMeshParams.bWantMeshTangents = true;

	TArray<FDynamicMesh3> InputMeshes;
	InputMeshes.Reserve(Targets.Num());
	for (int32 ComponentIdx = 0; ComponentIdx < Targets.Num(); ComponentIdx++)
	{
		InputMeshes.Add(ToolTarget::GetDynamicMeshCopy(Targets[ComponentIdx], GetMeshParams));
	}

	GetToolManager()->BeginUndoTransaction(LOCTEXT("ResectionMeshesToolTransactionName", "Resection Meshes"));

	FDynamicMesh3 AccumulatedMesh;
	AccumulatedMesh.EnableTriangleGroups();
	AccumulatedMesh.EnableAttributes();
	AccumulatedMesh.Attributes()->EnableTangents();
	AccumulatedMesh.Attributes()->EnableMaterialID();
	AccumulatedMesh.Attributes()->EnablePrimaryColors();

	constexpr int SkipIndex = 0;
	AActor* SkipActor = ToolTarget::GetTargetActor(Targets[SkipIndex]);
	check(SkipActor);

	FTransform TargetToWorld = SkipActor->GetRootComponent()->GetComponentTransform();

	{
		TArray<TUniquePtr<FDynamicMesh3>> OutputMeshes;
		TArray<FIntVector> OutputIDtoCell;
		TArray<int32> OutputIndexToID;

		{
			FScopedSlowTask SlowTask(Targets.Num() + 2,
				LOCTEXT("CombineMeshesBuild", "Merging and processing input meshes ..."));
			SlowTask.MakeDialog();

			for (int32 SectionIdx = 0; SectionIdx < Targets.Num(); SectionIdx++)
			{
				SlowTask.EnterProgressFrame(1);

				FDynamicMesh3& SectionMesh = InputMeshes[SectionIdx];

				if (SectionIdx != SkipIndex)
				{
					UPrimitiveComponent* SectionComponent = ToolTarget::GetTargetComponent(Targets[SectionIdx]);
					FTransform ComponentToWorld = SectionComponent->GetComponentTransform();

					// Premultiply the transforms so we only call ApplyTransform a single time
					// Note: we can only combine the transforms like this if neither has nonuniform scale.
					// If we need non-uniform scales, we'll need to do this via a matrix.
					const FTransform3d ComponentToTarget = ComponentToWorld * TargetToWorld.Inverse();

					MeshTransforms::ApplyTransform(SectionMesh, ComponentToTarget, true);
				}

				AccumulatedMesh.AppendWithOffsets(SectionMesh);
			}

			SlowTask.EnterProgressFrame(1);

			// Weld edges if requested
			if (Properties->bWeldEdges)
			{
				FMergeCoincidentMeshEdges Merger(&AccumulatedMesh);
				Merger.MergeVertexTolerance = 0.0001; // #todo_megamesh [roey]: Expose as param?
				Merger.MergeSearchTolerance = 2 * Merger.MergeVertexTolerance;
				Merger.OnlyUniquePairs = false;
				Merger.bWeldAttrsOnMergedEdges = true;

				ensure(Merger.Apply());
			}

			SlowTask.EnterProgressFrame(1);

			// Split meshes to grid cells

			FVector CellSize = FVector::Max(FVector(UE_DOUBLE_SMALL_NUMBER), (ResectionBounds.Max - ResectionBounds.Min) / (FVector)GetClampedSectionLayout());
			auto PointToCell = [this, CellSize](FVector Point) -> FIntVector const
				{
					FVector RelativeLocation = Point - ResectionBounds.Min;

					return FIntVector(RelativeLocation / CellSize);
				};

			// Tag triangles with integer cell coordinates
			TArray<FIntVector> TIDtoCell;
			TIDtoCell.SetNumUninitialized(AccumulatedMesh.MaxTriangleID());
			ParallelFor(AccumulatedMesh.MaxTriangleID(), [&AccumulatedMesh, &TargetToWorld, &PointToCell, &TIDtoCell](int32 TID)
				{
					if (!AccumulatedMesh.IsTriangle(TID))
					{
						return;
					}
					FVector V0, V1, V2;
					AccumulatedMesh.GetTriVertices(TID, V0, V1, V2);
					FVector Center = (V0 + V1 + V2) / 3.0;
					FVector WorldCenter = TargetToWorld.TransformPosition(Center);
					TIDtoCell[TID] = PointToCell(WorldCenter);
				}
			);
			// Map cell coordinates to dense int32 IDs, to split on
			// Tracking this mapping also allows us to label output section actors with cell coordinates
			TMap<FIntVector, int32> CellToOutputID;
			for (int32 TID : AccumulatedMesh.TriangleIndicesItr())
			{
				FIntVector Cell = TIDtoCell[TID];
				int32* FoundID = CellToOutputID.Find(Cell);
				if (!FoundID)
				{
					CellToOutputID.Add(Cell, OutputIDtoCell.Add(Cell));
				}
			}

			if (!FDynamicMeshEditor::SplitMesh(&AccumulatedMesh, OutputMeshes, false,
				[this, &CellToOutputID, &TIDtoCell](int32 TID) -> int32
				{
					return CellToOutputID[TIDtoCell[TID]];
				}, INDEX_NONE, &OutputIndexToID
			))
			{
				// Note: Split Mesh returns false if no split was needed
				// So we copy the input directly to the outputs in this case
				OutputMeshes.SetNum(1);
				OutputMeshes[0] = MakeUnique<FDynamicMesh3>(MoveTemp(AccumulatedMesh));
				OutputIndexToID.SetNum(1);
				OutputIndexToID[0] = 0;
				// The only failure case for split should be that there is only one output ID
				ensure(OutputIDtoCell.Num() == 1);
			}
		}

		// Commit changes:
		if (!OutputMeshes.IsEmpty())
		{
			// TODO: If targets can have varying material sets, need to consolidate rather than just taking the first
			FComponentMaterialSet MaterialSet = ToolTarget::GetMaterialSet(Targets[0]);
			TArray<UMaterialInterface*> Materials = MaterialSet.Materials;

			// Spawn new sections for output meshes

			// Find the mesh partition component to add to
			const AMeshPartition* MeshPartition = Cast<AMeshPartition>(SkipActor);
			if (!MeshPartition)
			{
				MeshPartition = Cast<AMeshPartition>(SkipActor->GetAttachParentActor());
			}
			UMeshPartitionEditorComponent* MeshPartitionEditorComponent = Cast<UMeshPartitionEditorComponent>(MeshPartition->GetMeshPartitionComponent());
			if (MeshPartitionEditorComponent != nullptr)
			{
				FScopedSlowTask SlowTask(OutputMeshes.Num(),
					LOCTEXT("SplitMeshesToSections", "Splitting meshes to sections ..."));
				SlowTask.MakeDialog();

				for (int32 Idx = 0; Idx < OutputMeshes.Num(); ++Idx)
				{
					SlowTask.EnterProgressFrame(1);

					AActor* NewActor = nullptr;

					// Center pivot for output meshes
					FVector Center = OutputMeshes[Idx]->GetBounds().Center();
					FVector WorldCenter = TargetToWorld.TransformPosition(Center);
					MeshTransforms::Translate(*OutputMeshes[Idx], -Center);
					FTransform OutputTransform = FTransform(Center) * TargetToWorld;

					NewActor = MeshPartitionEditorComponent->SpawnBaseModifier(MoveTemp(*OutputMeshes[Idx]), Materials, OutputTransform);
					FIntVector Cell = OutputIDtoCell[OutputIndexToID[Idx]];

					// TODO: Add better naming logic (avoid duplicates, don't include unused dimensions)
					const FString ActorLabel = FString::Format(TEXT("Resection_X{0}-Y{1}-Z{2}"), { Cell.X, Cell.Y, Cell.Z });
					NewActor->SetActorLabel(ActorLabel);
				}
			}
		}

		ToolSelectionUtil::SetNewActorSelection(GetToolManager(), SkipActor);
	}

	// Delete the source sections
	{
		GetToolManager()->BeginUndoTransaction(LOCTEXT("RemoveSources", "Remove Inputs"));

		for (int Idx = 0; Idx < Targets.Num(); Idx++)
		{
			AActor* Actor = ToolTarget::GetTargetActor(Targets[Idx]);
			Actor->Destroy();
		}
		GetToolManager()->EndUndoTransaction();
	}

	GetToolManager()->EndUndoTransaction();
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE