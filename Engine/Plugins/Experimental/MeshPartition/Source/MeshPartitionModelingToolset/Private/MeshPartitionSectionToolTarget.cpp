// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionSectionToolTarget.h"

#include "Components/DynamicMeshComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "Modifiers/MeshPartitionMeshProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionSectionToolTarget)

#define LOCTEXT_NAMESPACE "USectionToolTarget"

namespace UE::MeshPartition
{
void USectionToolTarget::SetOwnerVisibility(bool bVisible) const
{
	MeshPartition::UMeshProviderModifier* BaseMeshObject = Cast<MeshPartition::UMeshProviderModifier>(Component->GetOuter());
	if (!ensure(BaseMeshObject))
	{
		return;
	}

	// We want to show/hide the preview section, but as preview secitons get rebuilt, new ones are assigned to base sections,
	//  with visibility set back to visible. So, if we want the preview section to stay hidden, we will listen to preview
	//  section reassignment and update the new preview section if necessary.

	auto SetPreviewSectionVisibility = [bVisible](MeshPartition::UMeshProviderModifier* BaseSection, MeshPartition::APreviewSection* PreviewSection)
	{
		if (PreviewSection && PreviewSection->GetRootComponent())
		{
			PreviewSection->GetRootComponent()->SetVisibility(bVisible, /*bPropagateToChildren*/ true);
		}
	};

	SetPreviewSectionVisibility(BaseMeshObject, BaseMeshObject->GetPreviewSection());

	if (!bVisible)
	{
		// Weak so that it goes away if our target goes away without resetting visibility.
		BaseMeshObject->OnPreviewSectionReassignment().AddWeakLambda(this, SetPreviewSectionVisibility);
	}
	else
	{
		BaseMeshObject->OnPreviewSectionReassignment().RemoveAll(this);
	}
}


// Factory:

bool USectionToolTargetFactory::CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) const
{
	MeshPartition::UMeshProviderModifier* ParentComponent = Cast<MeshPartition::UMeshProviderModifier>(InSourceObject);
	if (!IsValid(ParentComponent))
	{
		return false;
	}
	UDynamicMeshComponent* MeshComponent = ParentComponent->MeshComponent;
	return IsValid(MeshComponent)
		&& !MeshComponent->IsUnreachable()
		&& MeshComponent->IsValidLowLevel()
		&& MeshComponent->GetDynamicMesh()
		&& InRequirements.AreSatisfiedBy(MeshPartition::USectionToolTarget::StaticClass());
}

UToolTarget* USectionToolTargetFactory::BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements)
{
	// Unwrap the internal UDynamicMeshComponent and use that as the component that the target uses
	MeshPartition::UMeshProviderModifier* ParentComponent = Cast<MeshPartition::UMeshProviderModifier>(InSourceObject);
	if (!ensure(IsValid(ParentComponent)))
	{
		return nullptr;
	}

	MeshPartition::USectionToolTarget* Target = NewObject<MeshPartition::USectionToolTarget>();
	Target->InitializeComponent(ParentComponent->MeshComponent);
	checkSlow(Target->IsValid() && InRequirements.AreSatisfiedBy(Target));

	return Target;
}
} // namespace UE::MeshPartition


#undef LOCTEXT_NAMESPACE
