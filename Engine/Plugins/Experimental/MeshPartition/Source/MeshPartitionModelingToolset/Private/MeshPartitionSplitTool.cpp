// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionSplitTool.h"

#include "InteractiveToolManager.h"

#include "Engine/World.h"
#include "MeshPartitionComponentBackedTarget.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "DynamicMeshEditor.h"
#include "DynamicMesh/MeshTransforms.h"
#include "ModelingToolTargetUtil.h"
#include "Properties/MeshUVChannelProperties.h"
#include "PropertySets/PolygroupLayersProperties.h"
#include "Drawing/MeshElementsVisualizer.h"
#include "Properties/MeshStatisticsProperties.h"

#include "MeshPartition.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionSectionToolTarget.h"
#include "TargetInterfaces/DynamicMeshProvider.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshPartitionSplitTool)

#define LOCTEXT_NAMESPACE "USplitTool"

using namespace UE::Geometry;

namespace UE::MeshPartition
{
UMeshSurfacePointTool* USplitToolBuilder::CreateNewTool(const FToolBuilderState& InSceneState) const
{
	return NewObject<MeshPartition::USplitTool>(InSceneState.ToolManager);
}

const FToolTargetTypeRequirements& USplitToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshProvider::StaticClass(),
			UPrimitiveComponentBackedTarget::StaticClass(),
			UMeshPartitionComponentBackedTarget::StaticClass(),
		});

		return TypeRequirements;
}


USplitTool::USplitTool()
{
}

void USplitTool::Setup()
{
	Super::Setup();

	// Remove all the tool settings that are automatically added by being a child of the selection tool:
	RemoveToolPropertySource(SelectionActions);
	RemoveToolPropertySource(SelectionProps);
	RemoveToolPropertySource(EditActions);
	RemoveToolPropertySource(UVChannelProperties);
	RemoveToolPropertySource(PolygroupLayerProperties);
	RemoveToolPropertySource(MeshStatisticsProperties);
}

void USplitTool::OnShutdown(EToolShutdownType InShutdownType)
{
	if (InShutdownType == EToolShutdownType::Accept)
	{
		SeparateSelectedTriangles();
	}
	
	Super::OnShutdown(InShutdownType);
}

void USplitTool::SeparateSelectedTriangles()
{
	check(SelectionType == EMeshSelectionElementType::Face);
	TArray<int32> SelectedFaces = Selection->GetElements(EMeshSelectionElementType::Face);
	if (SelectedFaces.Num() == 0)
	{
		return;
	}

	const FDynamicMesh3* SourceMesh = PreviewMesh->GetPreviewDynamicMesh();
	if (SelectedFaces.Num() == SourceMesh->TriangleCount())
	{
		// if we're selecting all the faces, there's no point separating.
		return;
	}

	FTransform3d Transform(PreviewMesh->GetTransform());

	// extract copy of triangles
	FDynamicMesh3 SeparatedMesh;
	SeparatedMesh.EnableTriangleGroups();
	SeparatedMesh.EnableAttributes();
	SeparatedMesh.Attributes()->EnableMatchingAttributes(*SourceMesh->Attributes());
	FDynamicMeshEditor Editor(&SeparatedMesh);
	FMeshIndexMappings Mappings; FDynamicMeshEditResult EditResult;
	Editor.AppendTriangles(SourceMesh, SelectedFaces, Mappings, EditResult);

	GetToolManager()->BeginUndoTransaction(LOCTEXT("MegaMeshSplitTool_Split", "Split"));

	FComponentMaterialSet MaterialSet = ToolTarget::GetMaterialSet(Target);
	TArray<UMaterialInterface*> Materials = MaterialSet.Materials;

	AActor* TargetActor = ToolTarget::GetTargetActor(Target);
	const AMeshPartition* MegaMesh = Cast<AMeshPartition>(TargetActor);
	if (!MegaMesh)
	{
		MegaMesh = Cast<AMeshPartition>(TargetActor->GetAttachParentActor());
	}
	FString AssetName = TargetActor->GetActorNameOrLabel();

	UMeshPartitionEditorComponent* MegaMeshEditorComponent = Cast<UMeshPartitionEditorComponent>(MegaMesh->GetMeshPartitionComponent());
	AActor* NewActor = nullptr;
	
	if (MegaMeshEditorComponent != nullptr)
	{
		NewActor = MegaMeshEditorComponent->SpawnBaseModifier(MoveTemp(SeparatedMesh), Materials, Transform);
	}

	GetToolManager()->EndUndoTransaction();

	if (NewActor != nullptr)
	{
		SpawnedActors.Add(NewActor);
		DeleteSelectedTriangles();
		// Mark the mesh as 'modified' so we can Accept, because if we Cancel, Actor created by duplicate operation will be rolled back
		bHaveModifiedMesh = true;
	}
}
} // namespace UE::MeshPartition

#undef LOCTEXT_NAMESPACE
