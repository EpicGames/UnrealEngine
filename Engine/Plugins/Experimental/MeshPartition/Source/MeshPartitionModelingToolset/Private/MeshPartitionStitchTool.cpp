// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionStitchTool.h"

#include "DynamicMeshEditor.h"
#include "InteractiveToolManager.h"
#include "MeshPartition.h"
#include "MeshPartitionComponentBackedTarget.h"
#include "MeshPartitionPreviewSection.h"
#include "MeshPartitionSectionToolTarget.h"
#include "MeshBoundaryLoops.h"
#include "MeshOpPreviewHelpers.h"
#include "ModelingToolTargetUtil.h"
#include "ToolSetupUtil.h"
#include "ToolTargetManager.h"
#include "CompGeom/PolygonTriangulation.h"
#include "Operations/SimpleHoleFiller.h"
#include "Operations/WeldEdgeSequence.h"
#include "Selection/PolygonSelectionMechanic.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#define LOCTEXT_NAMESPACE "MegaMeshStitchTool"

using namespace UE::Geometry;

namespace UE::MeshPartition
{
UInteractiveTool* UStitchToolBuilder::BuildTool(const FToolBuilderState& InSceneState) const
{
	UToolTarget* Target = nullptr;
	UToolTarget* TargetToStitch = nullptr;

	// Search through all targetable components which match the base set of target requirements
	InSceneState.TargetManager->EnumerateSelectedAndTargetableComponents(InSceneState, GetStitchableTargetRequirements(), [&](UActorComponent* Component)
	{
		// Find the first MegaMeshComponentBackedTarget which also matches all the other requirements. This will be the base target for the tool.
		if (InSceneState.TargetManager->CanBuildTarget(Component, GetTargetRequirements()))
		{
			Target = InSceneState.TargetManager->BuildTarget(Component, GetTargetRequirements());
		}
		// Then find another non-megamesh target which meet the base requirements. This will be the target mesh to stitch into the megamesh.
		// Skip components that belong to a MeshPartition (preview sections or the MeshPartition actor itself).
		else if (TargetToStitch == nullptr
			&& !Component->GetOwner()->IsA<MeshPartition::APreviewSection>()
			&& !Component->GetOwner()->IsA<MeshPartition::AMeshPartition>())
		{
			TargetToStitch = InSceneState.TargetManager->BuildTarget(Component, GetStitchableTargetRequirements());
		}
	});

	check(Target && TargetToStitch);

	MeshPartition::UStitchTool* Result = NewObject<MeshPartition::UStitchTool>(InSceneState.ToolManager);
	Result->SetWorld(InSceneState.World);

	Result->SetTarget(Target);
	Result->SetTargetToStitch(TargetToStitch);

	return Result;
}

bool UStitchToolBuilder::CanBuildTool(const FToolBuilderState& InSceneState) const
{
	int32 NumMegaMeshTargets = 0;
	int32 NumStitchableMeshes = 0;

	InSceneState.TargetManager->EnumerateSelectedAndTargetableComponents(InSceneState, GetStitchableTargetRequirements(), [&](UActorComponent* Component)
	{
		// Find the first MegaMeshComponentBackedTarget which also matches all the other requirements. This will be the base target for the tool.
		if (InSceneState.TargetManager->CanBuildTarget(Component, GetTargetRequirements()))
		{
			++NumMegaMeshTargets;
		}
		// Then collect all other non-megamesh targets which meet the base requirements. These will be the target meshes to stitch into the megamesh.
		// Skip components that belong to a MeshPartition (preview sections or the MeshPartition actor itself).
		else if (!Component->GetOwner()->IsA<MeshPartition::APreviewSection>()
			&& !Component->GetOwner()->IsA<MeshPartition::AMeshPartition>())
		{
			++NumStitchableMeshes;
		}
	});

	// #todo_megamesh [roey]: We could also fairly easily support multiple meshes simultaneously if we wanted but I'm not certain  this would be desirable as it would complicate stitching process.
	return NumMegaMeshTargets == 1 && NumStitchableMeshes == 1;
}

const FToolTargetTypeRequirements& UStitchToolBuilder::GetTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshProvider::StaticClass(),
			UPrimitiveComponentBackedTarget::StaticClass(),
			UE::MeshPartition::UMeshPartitionComponentBackedTarget::StaticClass(),
		});

	return TypeRequirements;
}

const FToolTargetTypeRequirements& UStitchToolBuilder::GetStitchableTargetRequirements() const
{
	static FToolTargetTypeRequirements TypeRequirements({
			UMaterialProvider::StaticClass(),
			UDynamicMeshProvider::StaticClass(),
			UPrimitiveComponentBackedTarget::StaticClass(),
		});

	return TypeRequirements;
}

void UStitchToolActions::PostAction(MeshPartition::EStitchToolActions InAction)
{
	if (ParentTool.IsValid())
	{
		ParentTool->RequestAction(InAction);
	}
}

void UStitchToolActions::Weld()
{
	PostAction(MeshPartition::EStitchToolActions::WeldEdges);
}

void UStitchToolActions::FillHole()
{
	PostAction(MeshPartition::EStitchToolActions::FillHole);
}

void FStitchToolMeshChange::Apply(UObject* InObject)
{
	UStitchTool* Tool = Cast<MeshPartition::UStitchTool>(InObject);

	MeshChange->Apply(Tool->CurrentMesh.Get(), /* bRevert = */ false);
	Tool->UpdateFromCurrentMesh(false);
}

void FStitchToolMeshChange::Revert(UObject* InObject)
{
	UStitchTool* Tool = Cast<MeshPartition::UStitchTool>(InObject);
	
	MeshChange->Apply(Tool->CurrentMesh.Get(), true);
	Tool->UpdateFromCurrentMesh(false);
}

void UStitchTool::Setup()
{
	CurrentMesh = MakeShared<FDynamicMesh3>(UE::ToolTarget::GetDynamicMeshCopy(Target));

	const FDynamicMesh3 MeshToStitch = UE::ToolTarget::GetDynamicMeshCopy(TargetToStitch);

	FDynamicMeshEditor Editor(CurrentMesh.Get());
	FMeshIndexMappings IndexMapping;

	WorldTransform = UE::ToolTarget::GetLocalToWorldTransform(Target);
	const FTransform MeshToStitchTransform = UE::ToolTarget::GetLocalToWorldTransform(TargetToStitch);

	// Transform MeshToStitch into the local space of the Target Mesh
	const FTransform MeshToTargetTransform = MeshToStitchTransform * WorldTransform.Inverse();
	Editor.AppendMesh(&MeshToStitch, IndexMapping, [&MeshToTargetTransform](int VertexID, const FVector3d& Position)
	{
		return MeshToTargetTransform.TransformPosition(Position);
	});

	Topology = MakeShared<FTriangleGroupTopology, ESPMode::ThreadSafe>(CurrentMesh.Get(), false);
	Topology->RebuildTopology();
	
	MeshSpatial = MakeShared<FDynamicMeshAABBTree3>();
	MeshSpatial->SetMesh(CurrentMesh.Get());

	Preview = NewObject<UMeshOpPreviewWithBackgroundCompute>();
	Preview->Setup(GetTargetWorld());
	ToolSetupUtil::ApplyRenderingConfigurationToPreview(Preview->PreviewMesh, Target);
	Preview->PreviewMesh->bBuildSpatialDataStructure = true;
	
	const FComponentMaterialSet MaterialSet = UE::ToolTarget::GetMaterialSet(Target);
	Preview->ConfigureMaterials(MaterialSet.Materials,
		ToolSetupUtil::GetDefaultWorkingMaterial(GetToolManager())
	);

	Preview->PreviewMesh->SetTangentsMode(EDynamicMeshComponentTangentsMode::AutoCalculated);
	Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get());
	Preview->PreviewMesh->EnableWireframe(true);
	Preview->SetVisibility(true);

	Preview->PreviewMesh->SetTransform(WorldTransform);

	SelectionMechanic = NewObject<UPolygonSelectionMechanic>(this);

	SelectionMechanic->bAddSelectionFilterPropertiesToParentTool = false;
	SelectionMechanic->SetShowSelectableCorners(false);
	SelectionMechanic->Setup(this);
	
	UMeshTopologySelectionMechanicProperties* SelectionProps = SelectionMechanic->Properties;
	// We only need edge selection for Weld/hole filling. May need to enable more in the future.
	SelectionProps->bSelectEdges = true;
	SelectionProps->bSelectVertices = false;
	SelectionProps->bSelectFaces = false;
	SelectionProps->bDisplayPolygroupReliantControls = false;

	// Configuration for face selection. Requires a secondary material and secondary triangle buffers to store the selected triangle IDs.
	{
		// configure secondary render material for rendering face selection
		if (UMaterialInterface* SelectionMaterial = ToolSetupUtil::GetSelectionMaterial(FLinearColor::Yellow, GetToolManager()))
		{
			Preview->SecondaryMaterial = SelectionMaterial;
		}

		Preview->PreviewMesh->EnableSecondaryTriangleBuffers(
			[this](const FDynamicMesh3* Mesh, int32 TriangleID)
		{
			return SelectionMechanic->GetActiveSelection().IsSelectedTriangle(Mesh, Topology.Get(), TriangleID);
		});

		SelectionMechanic->OnFaceSelectionPreviewChanged.AddWeakLambda(this, [this]() {
			Preview->PreviewMesh->FastNotifySecondaryTrianglesChanged();
		});
	}

	SelectionMechanic->OnSelectionChanged.AddUObject(this, &UStitchTool::OnSelectionModifiedEvent);
	SelectionMechanic->PolyEdgesRenderer.LineThickness = 2.0;
	SelectionMechanic->Initialize(CurrentMesh.Get(),
		(FTransform3d)Preview->PreviewMesh->GetTransform(),
		GetTargetWorld(),
		Topology.Get(),
		[this]() { return &GetSpatial(); }
	);

	Preview->InvalidateResult();

	EditActions = NewObject<MeshPartition::UStitchToolActions>();
	EditActions->Initialize(this);
	AddToolPropertySource(EditActions);

	UE::ToolTarget::HideSourceObject(Target);
	UE::ToolTarget::HideSourceObject(TargetToStitch);
}

void UStitchTool::OnShutdown(EToolShutdownType InShutdownType)
{
	UE::ToolTarget::ShowSourceObject(Target);
	UE::ToolTarget::ShowSourceObject(TargetToStitch);

	if (InShutdownType == EToolShutdownType::Accept)
	{
		check(CurrentMesh.IsValid());
		GetToolManager()->BeginUndoTransaction(LOCTEXT("MegaMeshStitchTransactionName", "Stitch Mesh Partition"));
		UE::ToolTarget::CommitDynamicMeshUpdate(Target, *CurrentMesh.Get(), true);

		// We consume the actor that gets stitched into the megamesh, destroy it now.
		if (AActor* MeshToStitchActor = UE::ToolTarget::GetTargetActor(TargetToStitch))
		{
			MeshToStitchActor->Destroy();
		}
		GetToolManager()->EndUndoTransaction();
	}
	Preview->Shutdown();
	SelectionMechanic->Shutdown();

	CurrentMesh.Reset();
	Topology.Reset();
	MeshSpatial.Reset();
}

void UStitchTool::OnTick(float InDeltaTime)
{
	Super::OnTick(InDeltaTime);

	if (bSelectionStateDirty)
	{
		Preview->PreviewMesh->FastNotifySecondaryTrianglesChanged();

		bSelectionStateDirty = false;
	}

	switch (PendingAction)
	{
	case MeshPartition::EStitchToolActions::NoAction:
		break;
	case MeshPartition::EStitchToolActions::WeldEdges:
		ApplyWeld();
		break;
	case MeshPartition::EStitchToolActions::FillHole:
		ApplyFillHole();
		break;
	default: ;
	}

	PendingAction = MeshPartition::EStitchToolActions::NoAction;
}

void UStitchTool::Render(IToolsContextRenderAPI* InRenderAPI)
{
	SelectionMechanic->Render(InRenderAPI);
}

void UStitchTool::RequestAction(MeshPartition::EStitchToolActions InAction)
{
	if (PendingAction == MeshPartition::EStitchToolActions::NoAction)
	{
		PendingAction = InAction;
	}
}

FDynamicMeshAABBTree3& UStitchTool::GetSpatial()
{
	if (bSpatialDirty)
	{
		MeshSpatial->Build();
		bSpatialDirty = false;
	}
	return *MeshSpatial;
}

void UStitchTool::OnSelectionModifiedEvent()
{
	bSelectionStateDirty = true;
}

void UStitchTool::ApplyWeld()
{
	if (SelectionMechanic->GetActiveSelection().SelectedEdgeIDs.Num() != 2)
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnWeldEdgesFailedEdgeCount", "Cannot Weld current selection, selection must be exactly 2 edges."),
			EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMesh3 MeshCopy(*Mesh);
	
	FGroupTopologySelection CurrentSelection = SelectionMechanic->GetActiveSelection();
	TArray<int32> SelectedEdgeIDs = CurrentSelection.SelectedEdgeIDs.Array();
	FEdgeSpan& SpanA = Topology->Edges[SelectedEdgeIDs[0]].Span;
	FEdgeSpan& SpanB = Topology->Edges[SelectedEdgeIDs[1]].Span;
	
	if (SpanA.Vertices[0] == SpanA.Vertices.Last() || SpanB.Vertices[0] == SpanB.Vertices.Last())
	{
		GetToolManager()->DisplayMessage(
			LOCTEXT("OnWeldEdgesFailedEdgesAreLoops", "Cannot Weld current selection, selected edges must not be loops."),
			EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();

	// Save one ring tri's for vertices along first edge
	for (int Vert : SpanA.Vertices)
	{
		ChangeTracker.SaveVertexOneRingTriangles(Vert, true);
	}

	// Save one ring tri's for vertices along second edge
	for (int Vert : SpanB.Vertices)
	{
		ChangeTracker.SaveVertexOneRingTriangles(Vert, true);
	}

	FWeldEdgeSequence EdgeWelder(&MeshCopy, SpanA, SpanB);
	EdgeWelder.bAllowIntermediateTriangleDeletion = true;
	EdgeWelder.bAllowFailedMerge = true;
	
	FWeldEdgeSequence::EWeldResult Result = EdgeWelder.Weld();
	if (Result != FWeldEdgeSequence::EWeldResult::Ok)
	{
		switch (Result)
		{
		case FWeldEdgeSequence::EWeldResult::Failed_EdgesNotBoundaryEdges:
			GetToolManager()->DisplayMessage(
				LOCTEXT("OnWeldEdgesFailedBoundary", "Cannot Weld current selection, selected edges must be boundary edges."),
				EToolMessageLevel::UserWarning);
			break;

		case FWeldEdgeSequence::EWeldResult::Failed_CannotSplitEdge:
			GetToolManager()->DisplayMessage(
				LOCTEXT("OnWeldEdgesFailedSplitEdge", "Cannot Weld current selection, failed to insert vertex."), 
				EToolMessageLevel::UserWarning);
			break;

		case FWeldEdgeSequence::EWeldResult::Failed_TriangleDeletionDisabled:
			GetToolManager()->DisplayMessage(
				LOCTEXT("OnWeldEdgesFailedTriDeleteDisabled", "Cannot Weld current selection, deletion of edges connecting selected edges is disabled."), 
				EToolMessageLevel::UserWarning);
			break;

		case FWeldEdgeSequence::EWeldResult::Failed_CannotDeleteTriangle:
			GetToolManager()->DisplayMessage(
				LOCTEXT("OnWeldEdgesFailedTriDeleteFailed", "Cannot Weld current selection, failed to delete edge connecting selected edges."), 
				EToolMessageLevel::UserWarning);
			break;

		case FWeldEdgeSequence::EWeldResult::Failed_Other:
		default:
			GetToolManager()->DisplayMessage(
				LOCTEXT("OnWeldEdgesFailedOther", "Cannot Weld current selection, bad geometry."), 
				EToolMessageLevel::UserWarning);
			break;
		}

		return;
	}
	else
	{
		// On success, apply the result by copying over the existing mesh
		*Mesh = MeshCopy;

		if (EdgeWelder.UnmergedEdgePairsOut.Num() != 0)
		{
			GetToolManager()->DisplayMessage(
				LOCTEXT("OnWeldEdgesCompletedSeamsRemain", "Warning: welding incomplete because it would create "
					"invalid geometry (attached non manifold edge or duplicate triangle). Seam still exists at weld "
					"location. Modify attached triangles and retry, or undo."),
				EToolMessageLevel::UserWarning);
		}
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("MegaMeshStitchWeldEdgeChange", "Weld Edges"), ChangeTracker.EndChange());
}

void UStitchTool::ApplyFillHole()
{
	if (BeginMeshEdgeEditChange() == false)
	{
		GetToolManager()->DisplayMessage( LOCTEXT("OnEdgeFillFailed", "Cannot Fill current selection"), EToolMessageLevel::UserWarning);
		return;
	}

	FDynamicMesh3* Mesh = CurrentMesh.Get();
	FDynamicMeshChangeTracker ChangeTracker(Mesh);
	ChangeTracker.BeginChange();
	for (FSelectedEdge& FillEdge : ActiveEdgeSelection)
	{
		if (Mesh->IsBoundaryEdge(FillEdge.EdgeIDs[0]))		// may no longer be boundary due to previous fill
		{
			FMeshBoundaryLoops BoundaryLoops(Mesh);
			int32 LoopID = BoundaryLoops.FindLoopContainingEdge(FillEdge.EdgeIDs[0]);
			if (LoopID >= 0)
			{
				FEdgeLoop& Loop = BoundaryLoops.Loops[LoopID];
				FSimpleHoleFiller Filler(Mesh, Loop);
				Filler.FillType = FSimpleHoleFiller::EFillType::PolygonEarClipping;
				int32 NewGroupID = Mesh->AllocateTriangleGroup();
				Filler.Fill(NewGroupID);

				// Compute normals and UVs
				if (Mesh->HasAttributes())
				{
					TArray<FVector3d> VertexPositions;
					Loop.GetVertices(VertexPositions);
					FVector3d PlaneOrigin;
					FVector3d PlaneNormal;
					PolygonTriangulation::ComputePolygonPlane<double>(VertexPositions, PlaneNormal, PlaneOrigin);

					FDynamicMeshEditor Editor(Mesh);
					Editor.SetTriangleNormals(Filler.NewTriangles);
					// #todo_megamesh [roey]: UVs
				}
			}
		}
	}

	EmitCurrentMeshChangeAndUpdate(LOCTEXT("MegaMeshStitchFillHoleChange", "Fill Hole"), ChangeTracker.EndChange());
}

bool UStitchTool::BeginMeshFaceEditChange()
{
	ActiveTriangleSelection.Reset();

	// need some selected faces
	const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
	Topology->GetSelectedTriangles(ActiveSelection, ActiveTriangleSelection);
	if (ActiveSelection.SelectedGroupIDs.Num() == 0 || ActiveTriangleSelection.Num() == 0)
	{
		return false;
	}

	const FDynamicMesh3* Mesh = CurrentMesh.Get();
	ActiveSelectionBounds = FAxisAlignedBox3d::Empty();
	for (int TID : ActiveTriangleSelection)
	{
		ActiveSelectionBounds.Contain(Mesh->GetTriBounds(TID));
	}

	// world and local frames
	ActiveSelectionFrameLocal = Topology->GetSelectionFrame(ActiveSelection);
	ActiveSelectionFrameWorld = ActiveSelectionFrameLocal;
	ActiveSelectionFrameWorld.Transform(WorldTransform);

	return true;
}

bool UStitchTool::BeginMeshEdgeEditChange()
{
	ActiveEdgeSelection.Reset();

	const FGroupTopologySelection& ActiveSelection = SelectionMechanic->GetActiveSelection();
	const int NumEdges = ActiveSelection.SelectedEdgeIDs.Num();
	if (NumEdges == 0)
	{
		return false;
	}
	ActiveEdgeSelection.Reserve(NumEdges);
	for (const int32 EdgeID : ActiveSelection.SelectedEdgeIDs)
	{
		if (Topology->IsBoundaryEdge(EdgeID))
		{
			FSelectedEdge& Edge = ActiveEdgeSelection.Emplace_GetRef();
			Edge.EdgeTopoID = EdgeID;
			Edge.EdgeIDs = Topology->GetGroupEdgeEdges(EdgeID);
		}
	}

	return ActiveEdgeSelection.Num() > 0;
}

void UStitchTool::EmitCurrentMeshChangeAndUpdate(const FText& InTransactionLabel, TUniquePtr<FDynamicMeshChange> InMeshChange)
{
	GetToolManager()->BeginUndoTransaction(InTransactionLabel);

	if (!SelectionMechanic->GetActiveSelection().IsEmpty())
	{
		SelectionMechanic->BeginChange();
		SelectionMechanic->ClearSelection();
		GetToolManager()->EmitObjectChange(SelectionMechanic, SelectionMechanic->EndChange(), LOCTEXT("ClearSelection", "Clear Selection"));
	}

	TUniquePtr<MeshPartition::FStitchToolMeshChange> ChangeToEmit = MakeUnique<MeshPartition::FStitchToolMeshChange>(MoveTemp(InMeshChange));

	Topology->RebuildTopology();
	GetToolManager()->EmitObjectChange(this, 
		MoveTemp(ChangeToEmit),
		InTransactionLabel);

	// Update other related structures
	UpdateFromCurrentMesh(false);
	SelectionMechanic->NotifyMeshChanged(true);

	GetToolManager()->EndUndoTransaction();
}

void UStitchTool::UpdateFromCurrentMesh(bool bInUpdateTopology)
{
	Preview->PreviewMesh->UpdatePreview(CurrentMesh.Get(), UPreviewMesh::ERenderUpdateMode::FullUpdate);
	bSpatialDirty = true;
	SelectionMechanic->NotifyMeshChanged(bInUpdateTopology);

	if (bInUpdateTopology)
	{
		Topology->RebuildTopology();
	}
}

void UStitchTool::SetTargetToStitch(UToolTarget* InTargetToStitch)
{
	check(InTargetToStitch != nullptr);
	TargetToStitch = InTargetToStitch;
}
} // namespace UE::MeshPartition
#undef LOCTEXT_NAMESPACE
