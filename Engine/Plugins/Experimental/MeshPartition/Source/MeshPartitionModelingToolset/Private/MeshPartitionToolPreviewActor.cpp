// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionToolPreviewActor.h"
#include "Engine/CollisionProfile.h"
#include "MeshPartitionPreviewComponents.h"
#include "MeshPartitionEditorUtils.h"
#include "MeshPartitionChannelCollection.h"
#include "Engine/Texture.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "MeshPartitionDefinition.h"
#include "MeshPartitionChannel.h"

namespace UE::MeshPartition
{

AToolPreviewMesh::AToolPreviewMesh()
{
	bListedInSceneOutliner = false;

	PreviewMeshComponent = CreateDefaultSubobject<MeshPartition::UPreviewMeshComponent>(TEXT("PreviewMeshComponent"));
	RootComponent = PreviewMeshComponent;

	PreviewMeshComponent->SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);
	PreviewMeshComponent->SetMobility(EComponentMobility::Static);
}

void AToolPreviewMesh::SetMesh(TSharedRef<const MeshPartition::FMeshData> MeshData)
{
	PreviewMeshComponent->SetMeshData(MeshData);
}

void AToolPreviewMesh::SetMaterial(UMaterialInterface* InMaterial)
{
	MID = MeshPartition::EditorUtils::GetOrCreateMaterialInstance(MID, InMaterial, this, TEXT("ModifierToolTargetMID"), RF_Transient);
	PreviewMeshComponent->SetMaterial(0, MID);
}

void AToolPreviewMesh::SetChannelData(const MeshPartition::FSectionChannels& InChannelData)
{
	ChannelTable = InChannelData.Table;
	ChannelTexture = InChannelData.Texture.Get();
	ChannelTexcoordDesc = InChannelData.TexcoordMetrics;

	MID->SetTextureParameterValue(UE::MeshPartition::ChannelTextureParameterName, ChannelTexture);
	
	FChannelPacking::SetCustomPrimitiveData(PreviewMeshComponent, ChannelTable, ChannelTexcoordDesc);
}

}