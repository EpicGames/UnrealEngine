// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"

#include "MeshPartitionToolPreviewActor.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

class UMeshComponent;
class UMaterialInterface;
class UTexture;
class UMaterialInstanceDynamic;

namespace UE::MeshPartition
{
class FMeshData;
class UPreviewMeshComponent;
struct FSectionChannels;

UCLASS(MinimalAPI, Transient)
class AToolPreviewMesh : public AActor
{
	GENERATED_BODY()
public:
	AToolPreviewMesh();

	UE_API void SetMesh(TSharedRef<const MeshPartition::FMeshData> InMeshData);
	UE_API void SetChannelData(const MeshPartition::FSectionChannels& InChannelData);
	UE_API void SetMaterial(UMaterialInterface* InMaterial);

private:

	UPROPERTY(Transient)
	TObjectPtr<MeshPartition::UPreviewMeshComponent> PreviewMeshComponent;

	UPROPERTY(Transient)
	TObjectPtr<UTexture> ChannelTexture;

	UPROPERTY(Transient)
	TObjectPtr<UMaterialInstanceDynamic> MID;

	TArray<uint8> ChannelTable;
	FVector2f ChannelTexcoordDesc;
};
}

#undef UE_API