// Copyright Epic Games, Inc. All Rights Reserved.


#include "MeshPartitionToolTarget.h"

#include "Async/ParallelFor.h"
#include "DynamicMeshEditor.h"
#include "MeshPartitionToolTargetUtils.h"
#include "MeshPartitionComponent.h"
#include "ModelingToolTargetUtil.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "MeshPartitionEditorComponent.h"
#include "MeshPartitionEditorModule.h"
#include "MeshPartitionModifierComponent.h"
#include "MeshPartitionPreviewSection.h"
#include "Modifiers/MeshPartitionMeshProvider.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/MeshTransforms.h"
#include "DynamicMesh/Operations/MergeCoincidentMeshEdges.h"

using namespace UE::Geometry;

namespace
{
	const static FName SectionIDOverlayAttributeName = TEXT("SectionID");
}

namespace UE::MeshPartition
{
void UMeshPartitionToolTarget::Initialize(UMeshPartitionEditorComponent* InEditorComponent)
{
	InitializeComponent(InEditorComponent);

	InEditorComponent->ForAllCurrentModifiers([this](MeshPartition::UModifierComponent* InModifier)
	{
		if (InModifier && InModifier->IsBase())
		{
			if (UMeshProviderModifier* MeshProvider = Cast<UMeshProviderModifier>(InModifier))
			{
				MeshModifiers.Add(MeshProvider);
			}
		}
		return true; // continue iterating
	});

	bIsMeshModifiersInitialized = true;
}

FTransform UMeshPartitionToolTarget::GetWorldTransform() const
{
	if (!Component.IsValid())
	{
		return FTransform::Identity;
	}
	// Use the owning actor as the pivot, if available
	if (const AActor* Owner = Component->GetOwner())
	{
		return Owner->GetActorTransform();
	}
	else
	{
		return Component->GetComponentToWorld();
	}
}

FDynamicMesh3 UMeshPartitionToolTarget::GetDynamicMesh()
{
	return BuildMergedMesh();
}

void UMeshPartitionToolTarget::CommitDynamicMesh(const FDynamicMesh3& InCombinedMesh, const FDynamicMeshCommitInfo& InCommitInfo)
{
	SplitMeshesToSections(InCombinedMesh);
}

int32 UMeshPartitionToolTarget::GetNumMaterials() const
{
	// #todo_megamesh [roey]
	return 0;
}

UMaterialInterface* UMeshPartitionToolTarget::GetMaterial(int32 InMaterialIndex) const
{
	// #todo_megamesh [roey]
	return nullptr;
}

void UMeshPartitionToolTarget::GetMaterialSet(FComponentMaterialSet& OutMaterialSet, bool bInPreferAssetMaterials) const
{
	// #todo_megamesh [roey]
}

bool UMeshPartitionToolTarget::CommitMaterialSetUpdate(const FComponentMaterialSet& InMaterialSet, bool bInApplyToAsset)
{
	// #todo_megamesh [roey]
	return true;
}

UBodySetup* UMeshPartitionToolTarget::GetBodySetup() const
{
	// #todo_megamesh [roey]
	return nullptr;
}

IInterface_CollisionDataProvider* UMeshPartitionToolTarget::GetComplexCollisionProvider() const
{
	// #todo_megamesh [roey]
	return nullptr;
}

void UMeshPartitionToolTarget::SetOwnerVisibility(bool bInVisible) const
{
	check(bIsMeshModifiersInitialized);
	for (MeshPartition::UModifierComponent* Modifier : MeshModifiers)
	{
		if (MeshPartition::APreviewSection* PreviewSection = Modifier->GetPreviewSection())
		{
			PreviewSection->SetIsTemporarilyHiddenInEditor(!bInVisible);
		}
	}
}

FDynamicMesh3 UMeshPartitionToolTarget::BuildMergedMesh()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UMeshPartitionToolTarget::BuildMergedMesh);

	FDynamicMesh3 AccumulatedMesh;
	{
		TDynamicMeshScalarTriangleAttribute<int32>* SectionIDOverlay = new TDynamicMeshScalarTriangleAttribute<int32>(&AccumulatedMesh);
		AccumulatedMesh.EnableAttributes();
		AccumulatedMesh.Attributes()->AttachAttribute(SectionIDOverlayAttributeName, SectionIDOverlay);
		SectionIDOverlay->Initialize(INDEX_NONE);
	}
	
	const FTransform3d TargetToWorld = GetWorldTransform();

	check(bIsMeshModifiersInitialized);
	
	for (int32 SectionIndex = 0; SectionIndex < MeshModifiers.Num(); ++SectionIndex)
	{
		const FDynamicMesh3* ModifierMesh = MeshModifiers[SectionIndex]->GetMesh();

		if (ModifierMesh == nullptr)
		{
			continue;
		}
		
		FDynamicMesh3 SectionMesh(*ModifierMesh);
		TDynamicMeshScalarTriangleAttribute<int32>* SectionIDOverlay = new TDynamicMeshScalarTriangleAttribute<int32>(&SectionMesh);
		SectionMesh.Attributes()->AttachAttribute(SectionIDOverlayAttributeName, SectionIDOverlay);
		SectionIDOverlay->Initialize(SectionIndex);
		
		const FTransform3d SectionToWorld = MeshModifiers[SectionIndex]->GetComponentTransform();
		MeshTransforms::ApplyTransform(SectionMesh, SectionToWorld, true);
		// Note we separate out inverse transform, since FTransform::Inverse() can't handle non-uniform scales
		MeshTransforms::ApplyTransformInverse(SectionMesh, TargetToWorld, true);

		AccumulatedMesh.AppendWithOffsets(SectionMesh);
	}

	FMergeCoincidentMeshEdges Merger(&AccumulatedMesh);
	Merger.MergeVertexTolerance = 0.0001; // #todo_megamesh [roey]: Expose as param?
	Merger.MergeSearchTolerance = 2 * Merger.MergeVertexTolerance;
	Merger.OnlyUniquePairs = false;
	Merger.bWeldAttrsOnMergedEdges = true;
	// #todo_megamesh [roey]: Set anything on Merger.SplitAttributeWelder?; 

	ensure(Merger.Apply());

	return AccumulatedMesh;
}

void UMeshPartitionToolTarget::SplitMeshesToSections(const FDynamicMesh3& InCombinedMesh)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(MeshPartition::UMeshPartitionToolTarget::SplitMeshesToSections);

	const TDynamicMeshScalarTriangleAttribute<int32>* SectionIDs = (InCombinedMesh.Attributes())
		? static_cast<const TDynamicMeshScalarTriangleAttribute<int32>*>(InCombinedMesh.Attributes()->GetAttachedAttribute(SectionIDOverlayAttributeName))
		: nullptr;

	check(Component.IsValid());
	check(bIsMeshModifiersInitialized);

	TArray<TUniquePtr<FDynamicMesh3>> SplitMeshes;
	TArray<int32> SplitIdxToSectionIdx, SectionIdxToSplitIdx;
	constexpr int32 bDeleteID = INDEX_NONE;
	constexpr bool bSplitSingleMesh = true;

	TArray<int32> FallbackSectionIDs;
	if (!SectionIDs)
	{
		// tool lost/destroyed our section ID attribute -- fall back to assigning triangles by source modifier bounds
		const FTransform3d TargetToWorld = GetWorldTransform();
		TArray<FAxisAlignedBox3d> SectionBounds;
		SectionBounds.Init(FAxisAlignedBox3d::Empty(), MeshModifiers.Num());
		ParallelFor(MeshModifiers.Num(), [&](int32 SectionIndex)
		{
			if (!MeshModifiers[SectionIndex] || !MeshModifiers[SectionIndex]->GetMesh())
			{
				return;
			}
			const FDynamicMesh3* ModifierMesh = MeshModifiers[SectionIndex]->GetMesh();
			const FTransform3d SectionToWorld = MeshModifiers[SectionIndex]->GetComponentTransform();
			// Match BuildMergedMesh's vertex transform
			for (int32 VID : ModifierMesh->VertexIndicesItr())
			{
				SectionBounds[SectionIndex].Contain(TargetToWorld.InverseTransformPosition(SectionToWorld.TransformPosition(ModifierMesh->GetVertex(VID))));
			}
		});

		FallbackSectionIDs = AssignMeshTrisToClosestBounds(InCombinedMesh, SectionBounds);
	}

	const bool bWasSplit = FDynamicMeshEditor::SplitMesh(&InCombinedMesh, SplitMeshes, bSplitSingleMesh,
		[SectionIDs, &FallbackSectionIDs](int TID) -> int32
		{
			return SectionIDs ? SectionIDs->GetValue(TID) : FallbackSectionIDs[TID];
		}, bDeleteID, &SplitIdxToSectionIdx);

	check(bWasSplit);
	// build reverse map
	SectionIdxToSplitIdx.Init(INDEX_NONE, MeshModifiers.Num());
	for (int32 SplitIdx = 0; SplitIdx < SplitIdxToSectionIdx.Num(); ++SplitIdx)
	{
		int32 SectionIdx = SplitIdxToSectionIdx[SplitIdx];
		if (ensure(SectionIdxToSplitIdx.IsValidIndex(SectionIdx)))
		{
			SectionIdxToSplitIdx[SectionIdx] = SplitIdx;
		}
	}

	// Accumulate the new bounds of all the sections so we can trigger a single OnModifierChanged once all sections are updated
	TArray<FBox> AccumulatedBounds;
	for (int32 SectionIndex = 0; SectionIndex < MeshModifiers.Num(); ++SectionIndex)
	{
		int32 SplitIdx = SectionIdxToSplitIdx[SectionIndex];
		if (SplitIdx == INDEX_NONE)
		{
			// Section had no triangles in result; delete it
			UE::ToolTarget::SafeDeleteActor(MeshModifiers[SectionIndex]->GetOwner());
			MeshModifiers[SectionIndex] = nullptr;
			continue;
		}
		else
		{
			FDynamicMesh3& SectionMesh = *SplitMeshes[SplitIdx];

			// Vertices come back in TargetTransform-local space (the bake frame in BuildMergedMesh)
			const FTransform3d TargetToWorld = GetWorldTransform();
			const FTransform3d SectionToWorld = MeshModifiers[SectionIndex]->GetComponentTransform();
			MeshTransforms::ApplyTransform(SectionMesh, TargetToWorld, true);
			// Note we separate out inverse transform, since FTransform::Inverse() can't handle non-uniform scales
			MeshTransforms::ApplyTransformInverse(SectionMesh, SectionToWorld, true);
			MeshModifiers[SectionIndex]->SetIgnoreChanged(true);
			MeshModifiers[SectionIndex]->SetMesh(MoveTemp(SectionMesh), /*bEmitChange*/ true);
			MeshModifiers[SectionIndex]->SetIgnoreChanged(false);
		}

		AccumulatedBounds.Append(MeshModifiers[SectionIndex]->ComputeBounds());
	}

	// Trigger an update on the megamesh for all changed sections
	check(Component.IsValid());
	UMeshPartitionEditorComponent* MegaMeshEditorComponent = CastChecked<UMeshPartitionEditorComponent>(Component.Get());
	MegaMeshEditorComponent->OnBoundsChanged(AccumulatedBounds, UE::MeshPartition::EChangeType::StateChange);
}

// -------------------------------------------------------------------------------------------------------------------------

bool UMeshPartitionToolTargetFactory::CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) const
{
	const UMeshPartitionComponent* Component = Cast<UMeshPartitionComponent>(InSourceObject);
	return Component 
		&& IsValidChecked(Component)
		&& !Component->IsUnreachable() 
		&& Component->IsValidLowLevel() 
		&& InRequirements.AreSatisfiedBy(MeshPartition::UMeshPartitionToolTarget::StaticClass());
}

UToolTarget* UMeshPartitionToolTargetFactory::BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements)
{
	MeshPartition::UMeshPartitionToolTarget* Target = NewObject<MeshPartition::UMeshPartitionToolTarget>();
	Target->Initialize(Cast<UMeshPartitionEditorComponent>(InSourceObject));
	checkSlow(Target->IsValid() && InRequirements.AreSatisfiedBy(Target));

	return Target;
}
} // namespace UE::MeshPartition
