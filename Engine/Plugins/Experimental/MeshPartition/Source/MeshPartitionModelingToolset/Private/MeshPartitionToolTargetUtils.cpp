// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshPartitionToolTargetUtils.h"

#include "Async/ParallelFor.h"

namespace UE::MeshPartition
{

TArray<int32> AssignMeshTrisToClosestBounds(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const TArray<UE::Geometry::FAxisAlignedBox3d>& Bounds,
	int32 DefaultAssignment)
{
	using namespace UE::Geometry;

	TArray<int32> SectionIDs;
	SectionIDs.SetNumUninitialized(Mesh.MaxTriangleID());

	ParallelFor(Mesh.MaxTriangleID(), [&](int32 TID)
	{
		if (!Mesh.IsTriangle(TID))
		{
			SectionIDs[TID] = INDEX_NONE;
			return;
		}
		const FVector3d Centroid = Mesh.GetTriCentroid(TID);

		int32 BestSection = DefaultAssignment;
		double BestDistSq = TNumericLimits<double>::Max();
		for (int32 SectionIndex = 0; SectionIndex < Bounds.Num(); ++SectionIndex)
		{
			if (!Bounds[SectionIndex].IsEmpty())
			{
				const double DistSq = Bounds[SectionIndex].DistanceSquared(Centroid);
				if (DistSq < BestDistSq)
				{
					BestDistSq = DistSq;
					BestSection = SectionIndex;
				}
			}
		}
		SectionIDs[TID] = BestSection;
	});

	return SectionIDs;
}

} // namespace UE::MeshPartition
