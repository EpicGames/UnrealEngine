// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BoxTypes.h"
#include "DynamicMesh/DynamicMesh3.h"

namespace UE::MeshPartition
{

/**
 * Helper method used to split triangles out to sections, via bounding boxes
 * 
 * @param DefaultAssignment If no bounds are valid, all triangles will be assigned this value.
 * @return Array mapping triangle IDs to the index of the triangle centroid's closest Bounds
 */
TArray<int32> AssignMeshTrisToClosestBounds(
	const UE::Geometry::FDynamicMesh3& Mesh,
	const TArray<UE::Geometry::FAxisAlignedBox3d>& Bounds,
	int32 DefaultAssignment = 0);

} // namespace UE::MeshPartition
