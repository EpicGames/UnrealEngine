// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Sculpting/HeightBrushes.h"

#include "Async/ParallelFor.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "MeshPartitionHeightSculptTool.h" // MeshPartition::UHeightSculptToolProperties
#include "Sculpting/MeshSmoothingBrushOps.h"
#include "VectorUtil.h"

namespace HeightBrushesLocals
{
	double EvaluateFalloff(const FMeshSculptFallofFunc& FalloffWrapper, const FSculptBrushStamp& Stamp, const FVector3d& OrigPos)
	{
		// For the height brushes, we want our falloff to be insensitive to the Z value relative to the
		//  plane. To make it possible to use the z-sensitive falloff functions without modification,
		//  project our query point into a plane passing through the brush for evaluating falloff.
		FVector PositionForFalloff = Stamp.LocalFrame.ToPlane(OrigPos, 2);
		return FalloffWrapper.Evaluate(Stamp, PositionForFalloff);
	}

	void ApplySlopeErodeStamp(
		int Iters,
		double SlopeThresholdDeg,
		double ErodeStrength,
		double SmoothingBlend,
		TFunctionRef<FVector3d(const FVector3d& OrigPos)> GetMoveDirection,
		const FMeshSculptFallofFunc& FalloffWrapper,
		const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut
	)
	{
		TMap<int32, int32> VIDToIdx;
		if (Iters > 0)
		{
			// We need this map for iterations after the first, to go from neighbor vertex ID -> index in NewPositionsOut
			// (otherwise we'd use stale positions for the neighbor vertices)
			VIDToIdx.Reserve(Vertices.Num());
			for (int32 Idx = 0; Idx < Vertices.Num(); ++Idx)
			{
				VIDToIdx.Add(Vertices[Idx], Idx);
			}
		}

		double UsePower = Stamp.Power; // Power is a lerp parameter here, and already clamped [0,1], so we don't scale it by anything else
		double TargetSlopeRad = FMath::DegreesToRadians(SlopeThresholdDeg);
		double TargetSlopeTan = FMath::Tan(TargetSlopeRad);

		double DownBias = FMath::Clamp(ErodeStrength, -1., 1.) * .5 + .5;
		double UpBias = 1. - DownBias;

		for (int32 Iter = 0; Iter < Iters; ++Iter)
		{
			ParallelFor(Vertices.Num(), [&](int32 VerticesIdx)
			{
				int32 VID = Vertices[VerticesIdx];

				FVector3d OrigPos = Iter > 0 ? NewPositionsOut[VerticesIdx] : Mesh->GetVertex(VID);
				FVector3d UpDir = GetMoveDirection(OrigPos);
				double Falloff = EvaluateFalloff(FalloffWrapper, Stamp, OrigPos);
				if (Falloff < FMathd::ZeroTolerance)
				{
					NewPositionsOut[VerticesIdx] = OrigPos;
					return;
				}
				double SlopeThresh = TargetSlopeTan / Falloff;
				// TODO: Smoothing brush optionally uses cotan weights; should we have more triangulation-adaptive weight option here as well?
				double UniformEdgeWt = 1.0 / (double)Mesh->GetVtxEdgeCount(VID);
				double ErodeOffset = 0;
				double SmoothOffset = 0;
				Mesh->EnumerateVertexVertices(VID, [&](int32 NbrVID)
				{
					FVector3d NbrPos;
					if (Iter > 0)
					{
						if (int32* Idx = VIDToIdx.Find(NbrVID))
						{
							NbrPos = NewPositionsOut[*Idx];
						}
						else
						{
							NbrPos = Mesh->GetVertex(NbrVID);
						}
					}
					else
					{
						NbrPos = Mesh->GetVertex(NbrVID);
					}
					FVector3d EdgeVec = NbrPos - OrigPos;
					double Rise = EdgeVec.Dot(UpDir);
					SmoothOffset += Rise * UniformEdgeWt;
					FVector3d RunVec = EdgeVec - Rise * UpDir;
					double Run = FMath::Max(FMathd::ZeroTolerance, RunVec.Length());
					if (FMath::Abs(Rise) > Run * SlopeThresh) // "edge slope" is too high, w/ thresh modulated by falloff
					{
						double TargetRise = FMath::CopySign(Run * SlopeThresh, Rise);
						double DirBias = Rise <= 0 ? DownBias : UpBias;
						double UnweightedShift = (Rise - TargetRise) * DirBias;
						ErodeOffset += UnweightedShift * UniformEdgeWt;
					}
				});
				ErodeOffset = FMath::Lerp(ErodeOffset, SmoothOffset, SmoothingBlend); // Blend in smoothing
				double OffsetBlend = FMath::Min(Falloff * UsePower, 1.0);
				FVector3d NewPos = OrigPos + UpDir * ErodeOffset * OffsetBlend;
				NewPositionsOut[VerticesIdx] = NewPos;

				// Note: If we add a noise term, for ref, the landscape erode brush only applies noise after the final iteration, not inside the iteration loop
			});
		}
	}

	void ApplySculptStamp(
		TFunctionRef<FVector3d(const FVector3d& OrigPos)> GetMoveDirection,
		double BrushSpeedTuning, 
		const FMeshSculptFallofFunc& FalloffWrapper,
		const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut)
	{
		double UsePower = Stamp.Direction * Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;
		FVector3d MoveDirection = Stamp.LocalFrame.Z();

		bool bHaveAlpha = Stamp.HasAlpha();

		ParallelFor(Vertices.Num(), [Mesh, &Vertices, bHaveAlpha, &Stamp, UsePower, &NewPositionsOut, &FalloffWrapper, &GetMoveDirection](int32 k)
		{
			FVector3d OrigPos = Mesh->GetVertex(Vertices[k]);

			double Alpha = (bHaveAlpha) ? Stamp.StampAlphaFunc(Stamp, OrigPos) : 1.0;
			double Falloff = EvaluateFalloff(FalloffWrapper, Stamp, OrigPos) * Alpha;

			FVector3d MoveVector = GetMoveDirection(OrigPos) * UsePower;
			FVector3d NewPos = OrigPos + Falloff * MoveVector;
			NewPositionsOut[k] = NewPos;
		});
	}

	void ApplySmoothStamp(
		TFunctionRef<FVector3d(const FVector3d& OrigPos)> GetMoveDirection,
		bool bUseCotanCentroid,
		double BrushSpeedTuning,
		int32 NumIters,
		const FMeshSculptFallofFunc& FalloffWrapper,
		const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut)
	{
		FSmoothBrushOp::ApplyIterativeStamp(Mesh, Vertices, NewPositionsOut, NumIters,
		[&](int32 Iter, FVector3d OrigPos, int32 VID, TFunctionRef<FVector3d(int32)> GetVertexFn)->FVector3d
		{
			double Falloff = EvaluateFalloff(FalloffWrapper, Stamp, OrigPos);
			FVector3d SmoothedPos = OrigPos;
			if (bUseCotanCentroid)
			{
				SmoothedPos = UE::Geometry::FMeshWeights::CotanCentroidSafe(*Mesh, VID, GetVertexFn, 10.0);
			}
			else
			{
				SmoothedPos = UE::Geometry::FMeshWeights::UniformCentroid(*Mesh, VID, GetVertexFn);
			}
			FVector3d TargetLocation = UE::Geometry::Lerp(OrigPos, SmoothedPos, Falloff * Stamp.Power);
			return OrigPos + (TargetLocation - OrigPos).ProjectOnToNormal(GetMoveDirection(OrigPos));
		});
	}

	void ApplyFlattenStamp(
		TFunctionRef<FVector3d(const FVector3d& OrigPos)> GetDestinationPosition,
		TFunctionRef<FVector3d(const FVector3d& OrigPos)> GetMoveDirection,
		EPlaneBrushSideMode WhichSide,
		double BrushSpeedTuning,
		const FMeshSculptFallofFunc& FalloffWrapper,
		const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut)
	{
		// Like in FFlattenBrushOp, this gets used to figure out when to move vertices. Note
		//  that the rest of the function does differ from FFlattenBrushOp
		static const double PlaneSigns[3] = { 0, -1, 1 };
		double PlaneSign = PlaneSigns[(int32)WhichSide];

		const FVector3d& StampPos = Stamp.LocalFrame.Origin;
		double UsePower = Stamp.Power * Stamp.Radius * Stamp.DeltaTime * BrushSpeedTuning;

		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			FVector3d OrigPos = Mesh->GetVertex(Vertices[k]);
			FVector3d DestPos = GetDestinationPosition(OrigPos);
			// We'll end up normalizing this and multiplying by UsePower later, if needed
			FVector3d MoveVector = DestPos - OrigPos;

			double PlaneDot = MoveVector.Dot(GetMoveDirection(OrigPos));
			if (PlaneDot * PlaneSign < 0)
			{
				// We would be moving in the wrong direction, so keep original position.
				NewPositionsOut[k] = OrigPos;
				return;
			}


			double Falloff = EvaluateFalloff(FalloffWrapper, Stamp, OrigPos);
			// We Lerp DestPos with falloff, in addition to scaling the power by Falloff,
			// so that we get some additional smoothness where the surface approaches the
			// target plane.
			DestPos = UE::Geometry::Lerp(OrigPos, DestPos, Falloff);
			MoveVector = DestPos - OrigPos;
			double CurrentSquaredDistanceToDestination = MoveVector.SquaredLength();

			double DistanceToMoveVertex = UsePower * Falloff;
			
			if (CurrentSquaredDistanceToDestination <= DistanceToMoveVertex * DistanceToMoveVertex)
			{
				// Since we would move further, just move to destination and stop there
				NewPositionsOut[k] = DestPos;
				return;
			}

			// If we got to here, go ahead and move the vertex
			if (!MoveVector.Normalize())
			{
				// Shouldn't really happen but maybe could due to precision issues?
				// Indicates that we don't really need to move the vertex. Let's go ahead
				//  and use destination position.
				NewPositionsOut[k] = DestPos;
				return;
			}
			MoveVector *= DistanceToMoveVertex;
			NewPositionsOut[k] = OrigPos + MoveVector;
		});
	}

	FVector3d GetSphereMoveVector(const FVector3d& SphereCenter, const FVector3d& OrigPos,
		const FVector3d& FallbackVector)
	{
		FVector3d MoveVector = OrigPos - SphereCenter;
		if (!MoveVector.Normalize())
		{
			MoveVector = FallbackVector;
		}
		return MoveVector;
	}

	FVector3d ProjectOntoSphere(const FVector3d& SphereCenter, double SphereRadius, 
		const FVector3d& OrigPos, const FVector3d& FallbackMoveVector)
	{
		FVector3d MoveVector = GetSphereMoveVector(SphereCenter, OrigPos, FallbackMoveVector);
		return SphereCenter + SphereRadius * MoveVector;
	}
}//end namespace HeightBrushesLocals

UE::Geometry::FHeightBrushOpBase::FHeightBrushOpBase(UE::MeshPartition::UHeightSculptToolProperties* HeightSculptPropertiesIn)
	: HeightSculptProperties(HeightSculptPropertiesIn)
{
}

bool UE::Geometry::FHeightBrushOpBase::IsSphereBrush() const
{
	return HeightSculptProperties.IsValid() && HeightSculptProperties->ReferenceSurface == UE::MeshPartition::EHeightSculptReferenceSurface::Sphere;
}

void UE::Geometry::FHeightSculptBrushOp::ApplyStamp(const FDynamicMesh3* Mesh, 
	const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut)
{
	using namespace HeightBrushesLocals;

	if (IsSphereBrush())
	{
		const FVector3d& SphereCenter = SphereFrame.Origin;
		FVector3d FallbackMoveVector = SphereFrame.Z();

		static const double BrushSpeedTuning = 2.0;

		ApplySculptStamp(
			[&SphereCenter, &FallbackMoveVector](const FVector3d& OrigPos)
			{
				return GetSphereMoveVector(SphereCenter, OrigPos, FallbackMoveVector);
			},
			BrushSpeedTuning, GetFalloff(),
			Mesh, Stamp, Vertices, NewPositionsOut);
		return;
	}
	// else: plane brush
	
	FVector3d MoveDirection = Stamp.LocalFrame.Z();

	static const double BrushSpeedTuning = 2.0;

	ApplySculptStamp(
		// Only move along the plane Z
		[&MoveDirection](const FVector3d& OrigPos) { return MoveDirection; },
		BrushSpeedTuning, GetFalloff(),
		Mesh, Stamp, Vertices, NewPositionsOut);
}

void UE::Geometry::FHeightSmoothBrushOp::ApplyStamp(const FDynamicMesh3* Mesh, 
	const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut)
{
	using namespace HeightBrushesLocals;

	if (IsSphereBrush())
	{
		const FVector3d& SphereCenter = SphereFrame.Origin;
		FVector3d FallbackMoveVector = SphereFrame.Z();

		static const double BrushSpeedTuning = 12.0;

		ApplySmoothStamp(
			[&SphereCenter, &FallbackMoveVector](const FVector3d& OrigPos)
			{
				return GetSphereMoveVector(SphereCenter, OrigPos, FallbackMoveVector);
			},
			GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetPreserveUVFlow(),
			BrushSpeedTuning, GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetIterations(), GetFalloff(),
			Mesh, Stamp, Vertices, NewPositionsOut);
		return;
	}
	// else: plane brush

	FVector3d MoveDirection = Stamp.LocalFrame.Z();

	static const double BrushSpeedTuning = 12.0;

	ApplySmoothStamp(
		// Only move along the plane Z
		[&MoveDirection](const FVector3d& OrigPos) { return MoveDirection; },
		GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetPreserveUVFlow(),
		BrushSpeedTuning, GetPropertySetAs<UBaseSmoothBrushOpProps>()->GetIterations(), GetFalloff(),
		Mesh, Stamp, Vertices, NewPositionsOut);
}

UClass* UE::Geometry::FHeightSmoothBrushOp::GetPropertiesClass(bool bPrimaryBrush)
{
	return bPrimaryBrush ? UHeightSmoothBrushOpProps::StaticClass() : USecondaryHeightSmoothBrushOpProps::StaticClass();
}

FMeshSculptBrushOp::EReferencePlaneType UE::Geometry::FHeightFlattenBrushOp::GetReferencePlaneType() const
{
	if (const UMeshHeightSculptFlattenBrushOpProps* FlatOpProp = GetPropertySetAs<UMeshHeightSculptFlattenBrushOpProps>())
	{
		if (FlatOpProp->bFlattenToTarget)
		{
			return EReferencePlaneType::WorkPlane;
		}
	}
	return FHeightBrushOpBase::GetReferencePlaneType();
}

void UE::Geometry::FHeightFlattenBrushOp::BeginStroke(const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp, const TArray<int32>& InitialVertices)
{
	const UMeshHeightSculptFlattenBrushOpProps* FlatOpProp = GetPropertySetAs<UMeshHeightSculptFlattenBrushOpProps>();

	if (FlatOpProp && FlatOpProp->bFlattenToTarget)
	{
		StrokePlane = CurrentOptions.ConstantReferencePlane;
		StrokeHeight = FlatOpProp->SphereRadius;
		return;
	}

	StrokePlane = Stamp.LocalFrame;
	if (FlatOpProp && FlatOpProp->ZGridSnap > 0)
	{
		// Find the world z offset to snap to increments of ZGridSnap
		FVector3d WorldZ = Stamp.WorldFrame.GetAxis(2);
		double ZDist = WorldZ.Dot(Stamp.WorldFrame.Origin);
		double ZOff = UE::Geometry::SnapToIncrement<double>(ZDist, (double)FlatOpProp->ZGridSnap) - ZDist;
		// Transfer the world Z snap offset back to the local origin
		StrokePlane.Origin += Stamp.LocalFrame.FromFrameVector(Stamp.WorldFrame.ToFrameVector(WorldZ * ZOff));
	}

	FVector3d SphereCenter = SphereFrame.Origin;
	StrokeHeight = (Stamp.LocalFrame.Origin - SphereCenter).Length();
}

void UE::Geometry::FHeightFlattenBrushOp::ApplyStamp(const FDynamicMesh3* Mesh, 
	const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut)
{
	using namespace HeightBrushesLocals;
	
	const EPlaneBrushSideMode WhichSide = GetPropertySetAs<UMeshHeightSculptFlattenBrushOpProps>()->WhichSide;
	static const double BrushSpeedTuning = 2.0;

	if (IsSphereBrush())
	{
		const FVector3d& SphereCenter = SphereFrame.Origin;
		FVector3d FallbackMoveVector = SphereFrame.Z();

		ApplyFlattenStamp(
			[this, &SphereCenter, &FallbackMoveVector](const FVector3d& OrigPos)
			{
				return ProjectOntoSphere(SphereCenter, StrokeHeight, OrigPos, FallbackMoveVector);
			},
			[&SphereCenter, &FallbackMoveVector](const FVector3d& OrigPos)
			{
				return GetSphereMoveVector(SphereCenter, OrigPos, FallbackMoveVector);
			},
			WhichSide, BrushSpeedTuning, GetFalloff(),
			Mesh, Stamp, Vertices, NewPositionsOut);
		return;
	}
	// else: plane brush

	FVector3d MoveDirection = Stamp.LocalFrame.Z();

	ApplyFlattenStamp(
		[this](const FVector3d& OrigPos) { return StrokePlane.ToPlane(OrigPos, 2); },
		[&MoveDirection](const FVector3d& OrigPos) { return MoveDirection; },
		WhichSide, BrushSpeedTuning, GetFalloff(),
		Mesh, Stamp, Vertices, NewPositionsOut);
}

void UE::Geometry::FSlopeErodeBrushOp::ApplyStamp(const FDynamicMesh3* Mesh,
	const FSculptBrushStamp& Stamp, const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut)
{
	using namespace HeightBrushesLocals;

	if (IsSphereBrush())
	{
		const FVector3d& SphereCenter = SphereFrame.Origin;
		FVector3d FallbackMoveVector = SphereFrame.Z();

		ApplySlopeErodeStamp(
			GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->Iterations,
			GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->SlopeThreshold,
			GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->ErodeStrength,
			GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->SmoothingBlend,
			[&SphereCenter, &FallbackMoveVector](const FVector3d& OrigPos)
			{
				return GetSphereMoveVector(SphereCenter, OrigPos, FallbackMoveVector);
			},
			GetFalloff(),
			Mesh, Stamp, Vertices, NewPositionsOut);
		return;
	}
	// else: plane brush

	FVector3d MoveDirection = Stamp.LocalFrame.Z();

	ApplySlopeErodeStamp(
		GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->Iterations,
		GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->SlopeThreshold,
		GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->ErodeStrength,
		GetPropertySetAs<UMeshHeightSculptSlopeErodeBrushOpProps>()->SmoothingBlend,
		[&MoveDirection](const FVector3d& OrigPos) { return MoveDirection; },
		GetFalloff(),
		Mesh, Stamp, Vertices, NewPositionsOut);
}
