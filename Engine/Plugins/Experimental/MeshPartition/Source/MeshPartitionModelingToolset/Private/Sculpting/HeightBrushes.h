// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sculpting/MeshBrushOpBase.h"
#include "Sculpting/MeshSmoothingBrushOps.h"
#include "VectorTypes.h"

#include "HeightBrushes.generated.h"

namespace UE::MeshPartition
{
	class UHeightSculptToolProperties;
}

UCLASS()
class UMeshHeightSculptBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()

public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "10.", ClampMin = "0.0", ClampMax = "10.", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.85f;

	virtual float GetStrength() override { return Strength; }
	virtual float GetFalloff() override { return Falloff; }
	virtual void SetStrength(float NewStrength) { Strength = NewStrength; }
	virtual void SetFalloff(float NewFalloff) { Falloff = NewFalloff; }

	virtual bool SupportsStrengthPressure() override { return true; }
};

UCLASS()
class UMeshHeightSculptFlattenBrushOpProps : public UMeshHeightSculptBrushOpProps
{
	GENERATED_BODY()

public:
	/** Whether to flatten to the target plane/sphere (set by the gizmo). Otherwise, flatten based on the stroke start position. */
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta = (DisplayName = "To Target"))
	bool bFlattenToTarget = false;

	/** Control whether effect of brush should be limited to one side of the plane/sphere.  */
	UPROPERTY(EditAnywhere, Category = HeightBrush)
	EPlaneBrushSideMode WhichSide = EPlaneBrushSideMode::BothSides;

	/** Snap the per-stroke flattening plane to global height multiples of this value (if non-zero). */
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta=(ClampMin="0.0", EditCondition="!bFlattenToTarget"))
	float ZGridSnap = 0.f;

	/** Radius of sphere to which to set the height of vertices, if using sphere reference surface.  */
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta=(EditCondition="bFlattenToTarget"))
	double SphereRadius = 10000.0;
};

// Erosion (slope) brush properties
UCLASS()
class UMeshHeightSculptSlopeErodeBrushOpProps : public UMeshSculptBrushOpProps
{
	GENERATED_BODY()

public:

	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;
	
	// Desired maximum slope angle in degrees
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta=(ClampMin=0, ClampMax=89, Units="Degrees"))
	double SlopeThreshold = 30;

	// Number of times to apply sloping per brush application
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta=(ClampMin = 1, UIMax = 50))
	int32 Iterations = 20;

	// Downward strength of erode. Positive erode strength creates mountain-like ridges; negative erode strength will pull up hills.
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta = (ClampMin = -1, ClampMax = 1))
	double ErodeStrength = .5;

	// Amount of smoothing to blend into the brush; a value of 1 will apply only smoothing and no erosion/sloping
	UPROPERTY(EditAnywhere, Category = HeightBrush, meta = (ClampMin = 0, ClampMax = 1))
	double SmoothingBlend = 0.;


	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool SupportsStrengthPressure() override { return true; }

};

UCLASS(MinimalAPI)
class UHeightSmoothBrushOpProps : public UBaseSmoothBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0", ModelingQuickEdit, ModelingQuickSettings = 200))
	float Strength = 0.5;

	/** Amount of falloff to apply (0.0 - 1.0) */
	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	UPROPERTY(EditAnywhere, Category = SmoothBrush, meta = (ClampMin = 1, UIMax = 50))
	int32 Iterations = 1;

	/** If true, try to preserve the shape of the UV/3D mapping. This will limit Smoothing and Remeshing in some cases. */
	UPROPERTY(EditAnywhere, Category = SmoothBrush)
	bool bPreserveUVFlow = false;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool GetPreserveUVFlow() override { return bPreserveUVFlow; }
	virtual int32 GetIterations() { return Iterations; }
};



UCLASS(MinimalAPI)
class USecondaryHeightSmoothBrushOpProps : public UBaseSmoothBrushOpProps
{
	GENERATED_BODY()
public:
	/** Strength of the Brush */
	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush, meta = (DisplayName = "Strength", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Strength = 0.5;

	/** Amount of falloff to apply */
	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush, meta = (DisplayName = "Falloff", UIMin = "0.0", UIMax = "1.0", ClampMin = "0.0", ClampMax = "1.0"))
	float Falloff = 0.5;

	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush, meta = (ClampMin = 1, UIMax = 50))
	int32 Iterations = 1;

	/** If true, try to preserve the shape of the UV/3D mapping. This will limit Smoothing and Remeshing in some cases. */
	UPROPERTY(EditAnywhere, Category = ShiftSmoothBrush)
	bool bPreserveUVFlow = false;

	virtual float GetStrength() override { return Strength; }
	virtual void SetStrength(float NewStrength) override { Strength = FMathf::Clamp(NewStrength, 0.0f, 1.0f); }
	virtual float GetFalloff() override { return Falloff; }
	virtual bool GetPreserveUVFlow() override { return bPreserveUVFlow; }
	virtual int32 GetIterations() { return Iterations; }
};


namespace UE::Geometry
{
class FHeightBrushOpBase : public FMeshSculptBrushOp
{
public:
	
	FHeightBrushOpBase(UE::MeshPartition::UHeightSculptToolProperties* HeightSculptPropertiesIn);

	virtual EBrushRegionType GetBrushRegionType() const
	{
		return IsSphereBrush() ? EBrushRegionType::CylinderOnSphere : EBrushRegionType::InfiniteCylinder;
	}

	//~ Note: aside from the visual feedback, plane height brushes use the stamp alignment
	//~  to figure out the plane that they are working in
	virtual EStampAlignmentType GetStampAlignmentType() const override
	{
		return IsSphereBrush() ? EStampAlignmentType::ReferenceSphere : EStampAlignmentType::ReferencePlane;
	}

	virtual bool IgnoreZeroMovements() const
	{
		return false;
	}

	virtual bool UseLastStampFrameOnZeroMovement() const override
	{
		return true;
	}

	virtual bool SupportsVariableSpacing() const override
	{
		return true;
	}

	virtual EReferencePlaneType GetReferencePlaneType() const override
	{
		return IsSphereBrush() ? EReferencePlaneType::WorkPlane : EReferencePlaneType::None;
	}

	virtual void BeginStroke(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, const TArray<int32>& InitialVertices) override
	{
		SphereFrame = CurrentOptions.ConstantReferencePlane;
	}

protected:
	bool IsSphereBrush() const;

	// Used when using sphere reference surface
	FFrame3d SphereFrame;

private:
	// Shared property struct that can be accessed in addition to the brush-specific one
	TWeakObjectPtr<UE::MeshPartition::UHeightSculptToolProperties> HeightSculptProperties;
};

class FHeightSculptBrushOp : public FHeightBrushOpBase
{
public:
	FHeightSculptBrushOp(UE::MeshPartition::UHeightSculptToolProperties* HeightSculptPropertiesIn)
		: FHeightBrushOpBase(HeightSculptPropertiesIn)
	{}

	virtual bool SupportsStrokeType(EMeshSculptStrokeType StrokeType) const override
	{
		switch (StrokeType)
		{
		case EMeshSculptStrokeType::Airbrush:
		case EMeshSculptStrokeType::Dots:
		case EMeshSculptStrokeType::Spacing:
			return true;
		default:
			return false;
		}
	}

	virtual bool UsesAlpha() const override { return true; }

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override;

	static UClass* GetPropertiesClass() { return UMeshHeightSculptBrushOpProps::StaticClass(); }
};

class FHeightSmoothBrushOp : public FHeightBrushOpBase
{
public:
	FHeightSmoothBrushOp(UE::MeshPartition::UHeightSculptToolProperties* HeightSculptPropertiesIn)
		: FHeightBrushOpBase(HeightSculptPropertiesIn)
	{}

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override;

	static UClass* GetPropertiesClass(bool bPrimaryBrush);
};

class FHeightFlattenBrushOp : public FHeightBrushOpBase
{
public:
	FHeightFlattenBrushOp(UE::MeshPartition::UHeightSculptToolProperties* HeightSculptPropertiesIn)
		: FHeightBrushOpBase(HeightSculptPropertiesIn)
	{}

	// Reference plane is required for the To Target flattening mode
	virtual EReferencePlaneType GetReferencePlaneType() const override;

	virtual void BeginStroke(const FDynamicMesh3* Mesh, const FSculptBrushStamp& Stamp, 
		const TArray<int32>& InitialVertices) override;

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override;

	static UClass* GetPropertiesClass() { return UMeshHeightSculptFlattenBrushOpProps::StaticClass(); }

private:
	// Used when using plane reference surface
	FFrame3d StrokePlane;

	// Used when using sphere reference surface
	double StrokeHeight = 0.0;
};

class FSlopeErodeBrushOp : public FHeightBrushOpBase
{
public:
	FSlopeErodeBrushOp(UE::MeshPartition::UHeightSculptToolProperties* HeightSculptPropertiesIn)
		: FHeightBrushOpBase(HeightSculptPropertiesIn)
	{
	}

	virtual void ApplyStamp(const FDynamicMesh3* SrcMesh, const FSculptBrushStamp& Stamp,
		const TArray<int32>& Vertices, TArray<FVector3d>& NewPositionsOut) override;

	static UClass* GetPropertiesClass() { return UMeshHeightSculptSlopeErodeBrushOpProps::StaticClass(); }
};

}//end UE::Geometry
