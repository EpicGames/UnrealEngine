// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshAttributePaintTool.h"
#include "MeshPartitionChannel.h" // MeshPartition::FChannelName
#include "Modifiers/MeshPartitionProjectSculptLayersModifier.h" // FSculptLayerModifierWeightAttributeEntry

#include "MeshPartitionAttributePaintTool.generated.h"


namespace UE::MeshPartition
{
class UAttributePaintTool;

UCLASS(MinimalAPI)
class UAttributePaintToolAddChannelProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	void Initialize(MeshPartition::UAttributePaintTool* ParentTool);

	/** The target channel to paint. */
	UPROPERTY(EditAnywhere, Category = "Mesh Partition Channels", Meta = (GetOptions = "GetChannelOptions", ModelingQuickSettings=400, DisplayName = "Channel"))
	MeshPartition::FChannelName WeightChannelName;

private:
	UFUNCTION()
	TArray<FName> GetChannelOptions() const;

	TWeakObjectPtr<MeshPartition::UAttributePaintTool> ParentTool;
};

UCLASS(MinimalAPI)
class UAttributePaintTool : public UMeshAttributePaintTool
{
	GENERATED_BODY()
public:
	virtual void Setup() override;
	virtual void OnTick(float DeltaTime) override;

	// UDynamicMeshBrushTool
	virtual void OnShutdown(EToolShutdownType ShutdownType) override;

protected:
	virtual void ExternalUpdateValues(int32 AttribIndex, const TArray<int32>& VertexIndices, const TArray<float>& NewValues) override;
	virtual void ApplyStamp(const FBrushStampData& Stamp);
	
	virtual FString GetPropertyCacheIdentifier() const override;
	
	virtual void InitializeAttributes() override;

private:
	UPROPERTY()
	TObjectPtr<MeshPartition::UAttributePaintToolAddChannelProperties> AddChannelProperties = nullptr;

	// Used by MeshPartition::UAttributePaintToolAddChannelProperties
	void AddChannel(FName NewChannel);
	TArray<FName> GetChannelOptions();

	TValueWatcher<FName> WeightChannelNameWatcher;
	bool bFilteredAttributeSourceInitialized = false;

	friend class UAttributePaintToolAddChannelProperties;
};

UCLASS(MinimalAPI)
class UAttributePaintToolBuilder : public UMeshAttributePaintToolBuilder
{
	GENERATED_BODY()

public:
	virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};
} // namespace UE::MeshPartition