// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"

#include "MeshPartitionCommonProperties.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE::MeshPartition
{
class AMeshPartition;
class UMeshPartitionDefinition;

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UCreateProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
	
public:
	
	UPROPERTY(EditAnywhere, Category=Add, meta=(EditCondition="NewMegaMeshDefinition == nullptr", TransientToolProperty))
	TObjectPtr<AMeshPartition> ExistingMegaMesh = nullptr;
	
	UPROPERTY(EditAnywhere, Category=Add, meta=(EditCondition="ExistingMegaMesh == nullptr", TransientToolProperty))
	TObjectPtr<UMeshPartitionDefinition> NewMegaMeshDefinition = nullptr;
};

UCLASS(MinimalAPI)
class USplitProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
	
public:

	/**
	* Number of sections in X/Y/Z dimension.
	*/
	UPROPERTY(EditAnywhere, Category = Split,
		meta = (DisplayName = "Layout", ClampMin = 1, UIMax = 64, ClampMax = 1024))
	FIntVector SectionLayout = FIntVector(3, 3, 1);
};
} // namespace UE::MeshPartition
#undef UE_API
