// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Interface.h"

#include "MeshPartitionComponentBackedTarget.generated.h"

namespace UE::MeshPartition
{
UINTERFACE(MinimalAPI)
class UMeshPartitionComponentBackedTarget : public UInterface
{
	GENERATED_BODY()
};

class IMeshPartitionComponentBackedTarget 
{
	GENERATED_BODY()
public:
};
} // namespace UE::MeshPartition