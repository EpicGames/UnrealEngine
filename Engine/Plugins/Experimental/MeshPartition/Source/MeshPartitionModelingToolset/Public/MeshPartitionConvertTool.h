// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "MeshPartitionConvertTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

class UPreviewGeometry;

namespace UE::Geometry
{
	class FDynamicMesh3;
}

namespace UE::MeshPartition
{
class USplitProperties;
class UCreateProperties;
class AMeshPartition;

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UConvertToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& InSceneState) const override;

	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// -------------------------------------------------------------------------------------------------------------------------

/**
* Modeling Mode Tool to convert an arbitrary mesh into a MegaMesh actor. Provides utilities to optionally automatically split the mesh.
*/
UCLASS(MinimalAPI)
class UConvertTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()
public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnTick(float DeltaTime) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	
	UE_API virtual void Shutdown(EToolShutdownType InShutdownType) override;
protected:

	UE_API TArray<TUniquePtr<Geometry::FDynamicMesh3>> SplitMesh(const Geometry::FDynamicMesh3& InMesh) const;

	UE_API FIntVector GetClampedSectionLayout() const;
	UE_API void CreateOrUpdatePreviewGeometry();

	UPROPERTY()
	TObjectPtr<MeshPartition::USplitProperties> SplitProperties;

	UPROPERTY()
	TObjectPtr<MeshPartition::UCreateProperties> CreateProperties;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry;

	FBox SplitBounds;
	bool bPreviewGeometryNeedsUpdate = false;
};
} // namespace UE::MeshPartition

#undef UE_API
