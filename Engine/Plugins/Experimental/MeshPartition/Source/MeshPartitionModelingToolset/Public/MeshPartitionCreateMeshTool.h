// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/SingleClickTool.h"
#include "CoreMinimal.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"
#include "MeshPartitionRectangleGenerator.h" 
#include "PreviewMesh.h"
#include "Properties/MeshMaterialProperties.h"
#include "PropertySets/CreateMeshObjectTypeProperties.h"
#include "UObject/NoExportTypes.h"


#include "MeshPartitionCreateMeshTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
PREDECLARE_USE_GEOMETRY_CLASS(FSegmentTree3);
class UCombinedTransformGizmo;
class UDragAlignmentMechanic;

namespace UE::MeshPartition
{
class UCreateProperties;
class AMeshPartition;

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UCreateMeshToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()

public:

	enum class EMakeMeshShapeType : uint32
	{
		Rectangle,
	};

	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;

	EMakeMeshShapeType ShapeType{ EMakeMeshShapeType::Rectangle };
};

// -------------------------------------------------------------------------------------------------------------------------


/** Placement Target Types */
UENUM()
enum class ECreateMeshPlacementType : uint8
{
	GroundPlane = 0,
	OnScene = 1,
	AtOrigin = 2
};

UCLASS(MinimalAPI)
class UPlacementProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/** How the shape is placed in the scene. */
	UPROPERTY(EditAnywhere, Category = Positioning, meta = (DisplayName = "Target Position"))
	MeshPartition::ECreateMeshPlacementType TargetSurface = MeshPartition::ECreateMeshPlacementType::AtOrigin;

	/** Initial rotation of the shape around its up axis, before placement. After placement, use the gizmo to control rotation. */
	UPROPERTY(EditAnywhere, Category = Positioning, DisplayName = "Initial Rotation", meta = (UIMin = "0.0", UIMax = "360.0", EditCondition = "!bShowGizmoOptions", HideEditConditionToggle))
	float Rotation = 0.0;

	/** If true, aligns the shape along the normal of the surface it is placed on. */
	UPROPERTY(EditAnywhere, Category = Positioning, meta = (EditCondition = "TargetSurface == ECreateMeshPlacementType::OnScene"))
	bool bAlignToNormal = false;

	/** Show a gizmo to allow the mesh to be repositioned after the initial placement click. */
	UPROPERTY(EditAnywhere, Category = Positioning, meta = (EditCondition = "bShowGizmoOptions", EditConditionHides, HideEditConditionToggle))
	bool bShowGizmo = true;

	//~ Not user visible- used to hide the bShowGizmo option when not yet placed mesh.
	UPROPERTY(meta = (TransientToolProperty))
	bool bShowGizmoOptions = false;

	/** Should snap the boundary of the new mesh to the existing base mesh boundary.  Available only if this mesh is being added to an existing Mesh Partition. */
	UPROPERTY(EditAnywhere, Category = Snapping, meta = (EditCondition = "bEnableSnapOptions", HideEditConditionToggle))
	bool bSnapToExistingMegaMesh;

	/** Angle tolerance for edges to be considered aligned for snapping (in degrees), only enabled if this mesh is being added to an existing mesh. */
	UPROPERTY(EditAnywhere, Category = Snapping, meta = (UIMin = "0.0", UIMax = "90.0", EditCondition = "bEnableSnapOptions && bSnapToExistingMegaMesh", EditConditionHides,HideEditConditionToggle))
	float AlignmentThreshold = 5.0f;

	/** Distance threshold for edge snapping, only enabled if this mesh is being added to an existing mesh. */
	UPROPERTY(EditAnywhere, Category = Snapping, meta = (UIMin = "0.0", UIMax = "10000.0", ClampMax = "10000000.0", EditCondition = "bEnableSnapOptions && bSnapToExistingMegaMesh", EditConditionHides, HideEditConditionToggle))
	float DistanceThreshold = 100.0f;

	/** Should render the boundary spans that the tool will snap to, only enabled if this mesh is being added to an existing mesh.*/
	UPROPERTY(EditAnywhere, Category = Snapping, meta = (EditCondition = "bEnableSnapOptions && bSnapToExistingMegaMesh", EditConditionHides, HideEditConditionToggle))
	bool bDrawSnapTargets = false;

	/**
	* When generating boundary corners, how sharp the angle needs to be to warrant corner placement there.
	* Lower values require sharper corners, so are more tolerant of curved boundary edges. For instance, 180 will place corners at every
	* vertex along a boundary even if the boundary is perfectly straight, and 135 will place a vertex only once the boundary edge
	* bends 45 degrees off the straight path (i.e. 135 degrees to the previous edge).
	* Only enabled if this mesh is being added to an existing mesh.
	*/
	UPROPERTY(EditAnywhere, Category = Snapping, meta = (UIMin = 0.0f, UIMax = 180.0f, ClampMin = 0.0f, ClampMax = 180.0f, EditCondition = "bEnableSnapOptions && bSnapToExistingMegaMesh", HideEditConditionToggle))
	float CornerThresholdDegrees = 135.f;

	//~ Not user visible- used to enable the snap options
	UPROPERTY(meta = (TransientToolProperty))
	bool bEnableSnapOptions = false;

};


UENUM()
enum class ECreateRectangleSectionsGenerationMode : uint8
{
	/** Automatically generate a grid of sections based on the maximum number of triangles allowed in a section. */
	Automatic,

	/** Generate a grid of sections based on explicit user input for resolution in X and Y. */
	Explicit,

	// Insert new SectionGeneration types
	LastSectionGenerationMode UMETA(Hidden)
};

UCLASS(MinimalAPI)
class UCreateRectangleToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	
	/** World space size of the generated mesh. */
	UPROPERTY(EditAnywhere, Category=Mesh, meta=(DisplayName="Size"))
	FVector2D MeshSize = FVector2D(100000., 100000.);


	/** Number of quads in X/Y dimension for the generated mesh, only editable for automatic section generation. */
	UPROPERTY(EditAnywhere, Category=Mesh, meta=(EditCondition = "SectionsGeneration == ECreateRectangleSectionsGenerationMode::Automatic", DisplayName="Resolution"))
	FInt32Point MeshResolution = FInt32Point(1000, 1000);


	/**
	* Defines how sections are generated.
	*/
	UPROPERTY(EditAnywhere, Category=Sections, meta=(DisplayName="Generation"))
	MeshPartition::ECreateRectangleSectionsGenerationMode SectionsGeneration = MeshPartition::ECreateRectangleSectionsGenerationMode::Automatic;


	/**
	* Maximum number of triangles allowed per section. This is used to control the partitioning of the imported mesh into manageable sections.
	* These sections can later be combined and split out manually as desired with the Split and Merge tools.
	*
	* When Sections Generation is Explicit, blocks creation when above limit.
	*/
	UPROPERTY(EditAnywhere, Category=Sections, meta=(DisplayName="Max Triangles", ClampMin=2, UIMax=1000000))
	int32 MaxTrianglesPerSection = 500 * 500 * 2;


	/**
	* Number of sections in X/Y dimension.
	*
	* This is only available if the Sections Generation is set to Explicit.
	*/
	UPROPERTY(EditAnywhere, Category=Sections,
		meta=(EditCondition = "SectionsGeneration == ECreateRectangleSectionsGenerationMode::Explicit", EditConditionHides, DisplayName = "Layout"))
	FInt32Point SectionLayout = FInt32Point(4, 4);
	
	/**
	* Number of quads in X/Y dimension per section.
	*
	* This is only available if the Sections Generation is set to Explicit.
	*/
	UPROPERTY(EditAnywhere, Category=Sections,
		meta=(EditCondition="SectionsGeneration == ECreateRectangleSectionsGenerationMode::Explicit", EditConditionHides, DisplayName="Resolution"))
	FInt32Point SectionsResolution = FInt32Point(250, 250);
	

	/**
	* Save and unload sections as they are created.
	*
	* This is only available when World Partition is enabled.
	*/
	UPROPERTY(EditAnywhere, Category=Sections)
	bool bSaveAndUnload = false;

	
	
private:
	friend class UCreateRectangleMeshTool;
	friend class UCreateMeshTool;

	// Resolution in terms of sections, but not all sections will necessarily contain the same number of quads 
	FInt32Point ComputeMeshResolutionInSections() const;

	// compute mesh resolution in terms of quads
	FInt32Point ComputeMeshResolutionInQuads() const;

	UPROPERTY(meta=(TransientToolProperty))
	FInt32Point CachedMeshResolution = FInt32Point(-1, -1);;
};

UCLASS(MinimalAPI)
class ULocationVolumesProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
/**
	* Create location volumes to make it easy to load and unload parts of the mesh.
	* This is especially useful when generating large meshes in combination with save and unload.
	*
	* This is only available when World Partition is enabled.
	*/
	UPROPERTY(EditAnywhere, Category=LocationVolumes, meta=(DisplayName="Create Volumes"))
	bool bCreateLocationVolumes = false;

	/**
	* Number of location volumes in X/Y dimension. 
	* 
	* This is only available when World Partition is enabled.
	*/
	UPROPERTY(EditAnywhere, Category=LocationVolumes, meta=(DisplayName="Volume Resolution"))
	FInt32Point LocationVolumesResolution = FInt32Point(2, 2);

		
private:
	friend class UCreateRectangleMeshTool;
	friend class UCreateMeshTool;

	FInt32Point GetLocationVolumesResolution() const;
};
// -------------------------------------------------------------------------------------------------------------------------

/**
* Basic functionality used creating and placing a MeshPartition.  Derive from this class and override functions such as GenerateAsset(),
* for example see MeshPartition::UCreateRectangleMeshTool
*/
UCLASS(MinimalAPI)
class UCreateMeshTool : public USingleClickTool, public IHoverBehaviorTarget, public IInteractiveToolCameraFocusAPI, public IInteractiveToolShutdownQueryAPI
{
	GENERATED_BODY()
public:
	UE_API UCreateMeshTool(const FObjectInitializer&);

	UE_API virtual void SetWorld(UWorld* World);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual bool CanAccept() const override;

	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// USingleClickTool
	UE_API virtual void OnClicked(const FInputDeviceRay& ClickPos) override;
	UE_API virtual FInputRayHit IsHitByClick(const FInputDeviceRay& ClickPos) override;

	// IHoverBehaviorTarget interface
	UE_API virtual FInputRayHit BeginHoverSequenceHitTest(const FInputDeviceRay& PressPos) override;
	UE_API virtual void OnBeginHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual bool OnUpdateHover(const FInputDeviceRay& DevicePos) override;
	UE_API virtual void OnEndHover() override;

	// IInteractiveToolCameraFocusAPI implementation
	UE_API virtual bool SupportsWorldSpaceFocusBox() override;
	UE_API virtual FBox GetWorldSpaceFocusBox() override;
	UE_API virtual bool SupportsWorldSpaceFocusPoint() override;
	UE_API virtual bool GetWorldSpaceFocusPoint(const FRay& WorldRay, FVector& PointOut) override;

	// IInteractiveToolShutdownQueryAPI implementation
	virtual EToolShutdownType GetPreferredShutdownType(EToolShutdownReason, EToolShutdownType) const override { return EToolShutdownType::Cancel; }

protected:

	enum class EState
	{
		PlacingPrimitive,
		AdjustingSettings
	};

	EState CurrentState = EState::PlacingPrimitive;
	UE_API void SetState(EState NewState);

	virtual void GeneratePreviewMesh(FDynamicMesh3* OutMesh) const {}
	
	virtual void UpdatePreview();

	// Refresh the host's display w/ result of BuildDisplayMessage()
	UE_API void UpdateDisplayMessage() const;

	// Override to set tool-specific warning messages
	virtual void BuildDisplayMessage(FText& OutMessage) const
	{}


	virtual void GenerateAsset() {};

	UPROPERTY()
	TObjectPtr<MeshPartition::UCreateRectangleToolProperties> CreateRectangleProperties;

	UPROPERTY()
	TObjectPtr<MeshPartition::UPlacementProperties> PlacementProperties;
	
	UPROPERTY()
	TObjectPtr<MeshPartition::UCreateProperties> MegaMeshCreateProperties;

	UPROPERTY()
	TObjectPtr<UPreviewMesh> PreviewMesh = nullptr;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> Gizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;

	UPROPERTY()
	FString AssetName = TEXT("GeneratedAsset");

	UPROPERTY()
	TObjectPtr<UWorld> World;

	UE_API void UpdatePreviewPosition(const FInputDeviceRay& ClickPos);
	Geometry::FFrame3d ShapeFrame;

	

	UE_API void UpdateTargetSurface();

	// @return true if the primitive needs to be centered in the XY plane when placed.
	virtual bool ShouldCenterXY() const
	{
		// Most primitives are already XY centered, and re-centering them only introduces issues at very low samplings where the bounds center is offset from the intended center.
		return false;
	}

	// Used to make the initial placement of the mesh undoable
	class FStateChange : public FToolCommandChange
	{
	public:
		FStateChange(const FTransform& MeshTransformIn)
			: MeshTransform(MeshTransformIn)
		{
		}

		UE_API virtual void Apply(UObject* Object) override;
		UE_API virtual void Revert(UObject* Object) override;
		virtual FString ToString() const override
		{
			return TEXT("MeshPartition::UCreateMeshTool::FStateChange");
		}

	protected:
		FTransform MeshTransform;
	};

	// helper that maybe of use in segmenting an existing mesh in a derived class
	UE_API static TArray<TUniquePtr<FDynamicMesh3>> SplitMesh(const FDynamicMesh3& InMesh, const FVector3d SectionsPerSide);

	void InitSnapTarget();
	bool SnapPreviewMeshToExisting(const FTransform& ShapeTransform, FVector3d& OutSnapOffset) const;

	// Boundary spans of the preview mesh (stored in preview mesh space)
	TArray<Geometry::FSegment3d> PreviewBoundarySegments;

	// Boundary spans of existing mesh providers
	TArray<Geometry::FSegment3d> SnapTargetSegments;
	TUniquePtr<FSegmentTree3> SnapTargetSegmentSpatial;

};


UCLASS(MinimalAPI)
class UCreateRectangleMeshTool : public MeshPartition::UCreateMeshTool
{
	GENERATED_BODY()
public:

	UE_API explicit UCreateRectangleMeshTool(const FObjectInitializer& ObjectInitializer);
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;


	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;
	UE_API virtual void GenerateAsset() override;
	UE_API virtual bool CanAccept() const override;

protected:

	using FSectionInfo = MeshPartition::FRectangleGeneratorUtils::FSectionInfo;
	virtual void UpdatePreview() override;

	UE_API virtual void GeneratePreviewMesh(FDynamicMesh3* OutMesh) const override;

	// Validate that the current configuration is buildable / within triangle limits
	UE_API bool IsConfigurationValid(FText* OutErrorMessage = nullptr) const;

	void RenderSections(IToolsContextRenderAPI* RenderAPI) const;

	UE_API virtual void BuildDisplayMessage(FText& OutMessage) const override;
	FMegaMeshRectangleGeneratorParams GetGeneratorParams() const ;

	TArray<FSectionInfo> SectionInfos;

};
} // namespace UE::MeshPartition


#undef UE_API
