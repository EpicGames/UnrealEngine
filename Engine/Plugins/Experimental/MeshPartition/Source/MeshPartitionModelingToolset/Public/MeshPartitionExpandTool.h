// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "MeshPartitionExpandTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::Geometry
{
class FDynamicMesh3;
class FMeshBoundaryLoops;
template <class T> class TMeshAABBTree3;
using FDynamicMeshAABBTree3 = TMeshAABBTree3<FDynamicMesh3>;
}
class UBoundarySelectionMechanic;
class IToolsContextRenderAPI;
class UPreviewMesh;

namespace UE::MeshPartition
{
class AMeshPartition;

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UExpandToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// -------------------------------------------------------------------------------------------------------------------------

UENUM()
enum class EExpandType
{
	/** Extrude one or more selected topology edges simultaneously */
	Extrude,

	/** Mirror the existing mesh across selected edge or corner */
	Mirror
};

UENUM()
enum class EExpandExtrudeEdgeDirectionMode
{
	/** Each vertex gets its own local frame to extrude along. */
	LocalExtrudeFrames,

	/** All vertices are extruded in the same direction */
	SingleDirection,
};

UCLASS(MinimalAPI)
class UExpandToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	/**
	* When generating corners, how sharp the angle needs to be to warrant corner placement there. 
	* Lower values require sharper corners, so are more tolerant of curved boundary edges. For instance, 180 will place corners at every 
	* vertex along a boundary even if the boundary is perfectly straight, and 135 will place a vertex only once the boundary edge 
	* bends 45 degrees off the straight path (i.e. 135 degrees to the previous edge).
	*/
	UPROPERTY(EditAnywhere, Category = Settings, meta = (UIMin = 0.0f, UIMax = 180.0f, ClampMin = 0.0f, ClampMax = 180.0f))
	float CornerThresholdDegrees = 135.f;

	UPROPERTY(EditAnywhere, Category = Settings, meta = (UIMin = 0, UIMax = 10, ClampMin = 0))
	int32 MinSpanSize = 1;

	/** Which algorithm to use to expand the current selection */
	UPROPERTY(EditAnywhere, Category = Settings)
	MeshPartition::EExpandType ExpandType = MeshPartition::EExpandType::Extrude;

	/** Number of steps to extend outward */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (UIMin = 0, UIMax = 1000, ClampMin = 0, ClampMax = 1000000, EditCondition = "ExpandType == EExpandType::Extrude"))
	int32 NumSteps = 10;

	/** Size of each outward-extending step */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (UIMin = 0.0f, UIMax = 10000.0f, ClampMin = 0.0f, ClampMax = 1000000.0f, EditCondition = "ExpandType == EExpandType::Extrude"))
	float StepSize = 100.0f;

	/** How direction to move the vertices is determined */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (EditCondition = "ExpandType == EExpandType::Extrude"))
	MeshPartition::EExpandExtrudeEdgeDirectionMode ExtrudeDirection = MeshPartition::EExpandExtrudeEdgeDirectionMode::LocalExtrudeFrames;

	/** Adjust individual extrude directions in an effort to make extruded edges parallel to their original edges */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (EditCondition = "ExpandType == EExpandType::Extrude && ExtrudeDirection == EExpandExtrudeEdgeDirectionMode::LocalExtrudeFrames"))
	bool bAdjustToExtrudeEvenly = true;

	/** When generating extrude frames, whether to use unselected neighbors for setting the frame. */
	UPROPERTY(EditAnywhere, Category = Extrude, meta = (EditCondition = "ExpandType == EExpandType::Extrude && ExtrudeDirection == EExpandExtrudeEdgeDirectionMode::LocalExtrudeFrames"))
	bool bUseUnselectedForFrames = false;

	/** Whether to mirror across topological corners as well as edges */
	UPROPERTY(EditAnywhere, Category = Mirror, meta = (EditCondition = "ExpandType == EExpandType::Mirror"))
	bool bShouldMirrorCorners = true;

	/** Render potenetial new sections in different colors */
	UPROPERTY(EditAnywhere, Category = Visualization)
	bool bRenderNewSections = false;

};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UExpandTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolExclusiveToolAPI
{
	GENERATED_BODY()

private:

	// UMultiSelectionMeshEditingTool overrides
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType InShutdownType) override;
	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	UPROPERTY()
	TObjectPtr<MeshPartition::UExpandToolProperties> ExpandProperties;

	// Note we use a WeakObjectPtr (rather than UPROPERTY) here b/c this actor belongs to the world, not the tool
	TWeakObjectPtr<AMeshPartition> MegaMeshActor;

	UPROPERTY()
	TArray<TObjectPtr<UBoundarySelectionMechanic>> SelectionMechanics;

	TArray<TUniquePtr<Geometry::FDynamicMesh3>> OriginalMeshes;

	TArray<TUniquePtr<Geometry::FMeshBoundaryLoops>> MeshBoundaryLoops;

	TArray<TUniquePtr<Geometry::FDynamicMeshAABBTree3>> MeshSpatials;

	UPROPERTY()
	TArray<TObjectPtr<UPreviewMesh>> PreviewMeshes;

	// Helpers

	/** For a given span, compute the average "perpendicular to span edge" vector */
	UE_API FVector3d AverageOutDirection(int32 TargetIndex, int32 SpanIndex) const;

	/** For all selected input meshes, find the average largest bounding box dimension */
	UE_API double InputMeshWidth() const;

	/** For all selected input meshes, find the average boundary edge length */
	UE_API double AverageInputMeshBoundaryEdgeLength() const;

	// Respond to UI events
	UE_API void OnSelectionModified();
	UE_API void RecomputeBoundaryTopology();
	UE_API void GeneratePreviewMeshes();

	// Generate new geometry
	UE_API void Extrude();
	UE_API void Mirror();

	UE_API void CreateNewBaseModifiers();
};
} // namespace UE::MeshPartition

#undef UE_API
