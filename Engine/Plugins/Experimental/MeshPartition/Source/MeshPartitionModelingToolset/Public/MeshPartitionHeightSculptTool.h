// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MeshVertexSculptTool.h"
#include "InteractiveToolQueryInterfaces.h"

#include "MeshPartitionHeightSculptTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
UENUM()
enum class EHeightSculptReferenceSurface : uint8
{
	Plane,
	Sphere
};

UCLASS(MinimalAPI)
class UHeightSculptToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** 
	 * When true, only vertices that are connected to the mouse hit point through
	 *  vertices inside the brush are moved. For example, if you brush vertices
	 *  of a hill that has a cave close to the surface, only the vertices of the
	 *  outside surface will be moved even if the brush happens to overlap some
	 *  hidden cave vertices. However this prevents brushing off of the mesh, because
	 *  the tool requires a hit triangle.
	 */
	UPROPERTY(EditAnywhere, Category = HeightSculptOptions)
	bool bRequireConnectivity = true;

	/** 
	 * When true, the reference plane/sphere transform is initialized using the target
	 *  object's transform 
	 */
	UPROPERTY(EditAnywhere, Category = HeightSculptOptions, AdvancedDisplay, Meta=(DisplayName = "Initialize From Target"))
	bool bInitializeReferenceTransformFromTarget = true;

	/** Specify the type of height sculpt for the tool */
	UPROPERTY(EditAnywhere, Category = HeightSculptOptions)
	EHeightSculptReferenceSurface ReferenceSurface = EHeightSculptReferenceSurface::Plane;
};

UCLASS(MinimalAPI)
class UHeightSculptToolSettingsProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** When true, the standard editor transform gizmo is shown when selecting other actors,
	 *  allowing you to reposition reference objects without leaving the tool */
	UPROPERTY(EditAnywhere, Category = ToolSettings, Meta = (ModelingQuickSettings))
	bool bAllowEditorGizmo = true;
};

UCLASS(MinimalAPI)
class UHeightSculptToolBuilder : public UMeshVertexSculptToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
};

UCLASS(MinimalAPI)
class UHeightSculptTool : public UMeshVertexSculptTool, public IInteractiveToolEditorGizmoAPI
{
	GENERATED_BODY()

public:
	// IInteractiveToolEditorGizmoAPI - allow standard editor gizmo while this tool is active,
	// so users can reposition reference objects without leaving the tool
	virtual bool GetAllowStandardEditorGizmos() override;

	// BrushIDs that can be used by SetActiveBrushType
	enum class EBrushType
	{
		HeightSculpt = (int32)EMeshVertexSculptBrushType::LastValue + 1,
		HeightSmooth,
		HeightFlatten,
		SlopeErode
	};

protected:
	// UMeshVertexSculptTool
	UE_API virtual void RegisterBrushes() override;
	UE_API virtual FString GetPropertyCacheIdentifier() const override;
	UE_API virtual bool RequireConnectivityToHitPointInStamp() const override;
	UE_API virtual void UpdateBrushType(int32 BrushID) override;

	// UMeshSculptToolBase
	UE_API virtual bool ShowWorkPlane() const override;

	// UInteractiveTool
	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType ShutdownType) override;

private:
	UPROPERTY()
	TObjectPtr<MeshPartition::UHeightSculptToolProperties> HeightSculptProperties = nullptr;

	UPROPERTY()
	TObjectPtr<MeshPartition::UHeightSculptToolSettingsProperties> ToolSettingsProperties = nullptr;

	// True when the active brush is one that we added in the subclass
	bool bHaveHeightBrushActive = false;
};
} // namespace UE::MeshPartition

#undef UE_API
