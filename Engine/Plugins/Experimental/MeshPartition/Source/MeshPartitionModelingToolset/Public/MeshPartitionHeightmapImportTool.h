// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"
#include "MeshPartitionHeightmapImporter.h"

#include "MeshPartitionHeightmapImportTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI)
class UHeightmapImportToolBuilder : public UInteractiveToolBuilder
{
	GENERATED_BODY()
public:
	virtual bool CanBuildTool(const FToolBuilderState& InSceneState) const override { return true; }

	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& InSceneState) const override;
};

UENUM()
enum class EHeightmapImportSectionsGenerationMode : uint8
{
	/** Automatically generate a grid of sections based on the maximum number of triangles allowed in a section. */
	Automatic,

	/** Generate a grid of sections based on explicit user input for resolution in X and Y. */
	Explicit
};

UCLASS(MinimalAPI)
class UHeightmapImportPropertySet : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	/** Filename of  the heightmap to use for import. */
	UPROPERTY(EditAnywhere, Category = Heightmap, meta=(DisplayName="Filename"))
	FFilePath HeightmapFile;

	/** Number of vertices in X/Y dimension for the generated mesh. */
	UPROPERTY(EditAnywhere, Category=Mesh, meta=(DisplayName="Resolution"))
	FInt32Point MeshResolution = FInt32Point(1024, 1024);

	/** World space size of the generated mesh. */
	UPROPERTY(EditAnywhere, Category=Mesh, meta=(DisplayName="Size"))
	FVector MeshSize = FVector(1024.0, 1024.0, 250.0);

	/**
	* Defines how sections are generated.
	*/
	UPROPERTY(EditAnywhere, Category=Sections, meta=(DisplayName="Generation"))
	MeshPartition::EHeightmapImportSectionsGenerationMode SectionsGeneration = MeshPartition::EHeightmapImportSectionsGenerationMode::Automatic;

	/**
	* Maximum number of triangles allowed per section. This is used to control the automatic partitioning of the imported mesh into manageable sections.
	* These sections can later be combined and split out manually as desired with the Split and Merge tools.
	*
	* This is only available if the Sections Generation is set to Automatic.
	*/
	UPROPERTY(EditAnywhere, Category=Sections,
		meta=(EditCondition="SectionsGeneration == EHeightmapImportSectionsGenerationMode::Automatic", EditConditionHides, DisplayName="Max Triangles"))
	int32 MaxTrianglesPerSection = 512 * 512 * 2;

	/**
	* Number of sections in X/Y dimension.
	*
	* This is only available if the Sections Generation is set to Explicit.
	*/
	UPROPERTY(EditAnywhere, Category=Sections,
		meta=(EditCondition="SectionsGeneration == EHeightmapImportSectionsGenerationMode::Explicit", EditConditionHides, DisplayName="Resolution"))
	FInt32Point SectionsResolution = FInt32Point(4, 4);

	/**
	* Save and unload sections as they are created.
	*
	* This is only available when World Partition is enabled.
	*/
	UPROPERTY(EditAnywhere, Category=Sections, meta=(EditCondition="bWorldPartitionIsAvailable", HideEditConditionToggle))
	bool bSaveAndUnload = false;

	/**
	* Create location volumes to make it easy to load and unload parts of the mesh imported from the heightmap.
	* This is especially useful when generating large meshes in combination with save and unload.
	*
	* This is only available when World Partition is enabled.
	*/
	UPROPERTY(EditAnywhere, Category=LocationVolumes, meta=(EditCondition="bWorldPartitionIsAvailable", HideEditConditionToggle, DisplayName="Create Volumes"))
	bool bCreateLocationVolumes = false;

	/**
	* Number of location volumes in X/Y dimension.
	*/
	UPROPERTY(EditAnywhere, Category=LocationVolumes, meta=(EditCondition="bWorldPartitionIsAvailable && bCreateLocationVolumes", DisplayName="Resolution"))
	FInt32Point LocationVolumesResolution = FInt32Point(2, 2);

private:
	friend class UHeightmapImportTool;

	FInt32Point GetSectionsResolution() const;
	FInt32Point GetLocationVolumesResolution() const;

	UPROPERTY(meta=(TransientToolProperty))
	bool bWorldPartitionIsAvailable;
};


/**
* 
*/
UCLASS(MinimalAPI)
class UHeightmapImportTool : public UInteractiveTool, public IInteractiveToolShutdownQueryAPI
{
	GENERATED_BODY()
public:
	UE_API virtual void SetWorld(UWorld* InWorld);

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType InShutdownType) override;

	UE_API virtual void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasAccept() const override { return true; }
	virtual bool HasCancel() const override { return true; }
	UE_API virtual bool CanAccept() const override;
	UE_API virtual void OnPropertyModified(UObject* PropertySet, FProperty* Property) override;

	// IInteractiveToolShutdownQueryAPI implementation
	virtual EToolShutdownType GetPreferredShutdownType(EToolShutdownReason, EToolShutdownType) const override { return EToolShutdownType::Cancel; }

private:
	UPROPERTY()
	TObjectPtr<MeshPartition::UHeightmapImportPropertySet> Properties;

	UPROPERTY()
	TObjectPtr<UWorld> World;

	bool IsConfigurationValid(FText* OutErrorMessage = nullptr) const;
	void UpdateDisplayMessage() const;
	FHeightmapImportParams GetParams() const;
};
} // namespace UE::MeshPartition

#undef UE_API
