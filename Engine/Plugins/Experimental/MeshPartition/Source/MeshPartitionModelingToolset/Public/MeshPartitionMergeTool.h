// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"

#include "MeshPartitionMergeTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI)
class UMergeToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UMergeToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category=Merge)
	bool bWeldEdges = true;
};

// -------------------------------------------------------------------------------------------------------------------------

/**
* Modeling Mode Tool to merge multiple MegaMesh Sections to a single section.
*/
UCLASS(MinimalAPI)
class UMergeTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolExclusiveToolAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType InShutdownType) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
protected:
	UE_API void Merge();

	UPROPERTY()
	TObjectPtr<MeshPartition::UMergeToolProperties> MergeProperties;
	
};
} // namespace UE::MeshPartition

#undef UE_API
