// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "ModelingModeToolExtensions.h"

class ILevelEditor;

DECLARE_LOG_CATEGORY_EXTERN(LogMegaMeshModelingToolset, Log, All);

namespace UE::MeshPartition
{
class FMegaMeshModelingToolsetModule : public IModuleInterface, public IModelingModeToolExtension
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** IModelingModeToolExtension implementation */
	virtual FText GetExtensionName() override;
	virtual FText GetToolSectionName() override;
	virtual void GetExtensionTools(const FExtensionToolQueryInfo& QueryInfo, TArray<FExtensionToolDescription>& ToolsOut) override;
	virtual bool GetExtensionExtendedInfo(FModelingModeExtensionExtendedInfo& InfoOut) override;
	virtual bool GetExtensionToolTargets(TArray<TSubclassOf<UToolTargetFactory>>& ToolTargetFactoriesOut) override;
};
} // namespace UE::MeshPartition