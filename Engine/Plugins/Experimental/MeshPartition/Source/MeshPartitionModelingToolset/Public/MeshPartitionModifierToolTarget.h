// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
//#include "MeshPartitionComponentBackedTarget.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "ToolTargets/SceneComponentToolTarget.h"
#include "UObject/Object.h"
#include "MeshPartitionChannel.h"
#include "MeshPartitionChannelCollection.h"
#include "OrientedBoxTypes.h"

#include "MeshPartitionModifierToolTarget.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

class UTexture;
class UMaterialInstanceDynamic;

namespace UE::Geometry
{
class FDynamicMesh3;
}

namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;
class UMeshProviderModifier;
class UModifierComponent;
class APreviewSection;
class FMeshData;

/**
* UToolTarget specialization that supports viewing megamesh modifiers by extracting
* the mesh as it exists after the modifier is applied
*/
UCLASS(Transient, MinimalAPI)
class UModifierToolTarget : 
	public USceneComponentToolTarget,
	public IDynamicMeshProvider,
	public IMaterialProvider,
	public IPhysicsDataSource
{
	GENERATED_BODY()
	
public:
	UE_API void Initialize(MeshPartition::UModifierComponent* ModifierComponent);
	
	// IDynamicMeshProvider implementation
	UE_API virtual Geometry::FDynamicMesh3 GetDynamicMesh() override;

	// IMaterialProvider implementation
	UE_API int32 GetNumMaterials() const override;
	UE_API UMaterialInterface* GetMaterial(int32 InMaterialIndex) const override;
	UE_API void GetMaterialSet(FComponentMaterialSet& OutMaterialSet, bool bInPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& InMaterialSet, bool bInApplyToAsset) override;	

	// IPhysicsDataSource implementation
	UE_API virtual UBodySetup* GetBodySetup() const override;
	UE_API virtual IInterface_CollisionDataProvider* GetComplexCollisionProvider() const override;

	// UToolTarget
	UE_API virtual bool IsValid() const override;

	// USceneComponentToolTarget
	UE_API virtual void SetOwnerVisibility(bool bInVisible) const override;
	
	void ConfigurePreviewForRendering(UPrimitiveComponent* PrimitiveComponent) const;

	void UpdateRenderTextureForPreview(const Geometry::FDynamicMesh3& PreviewMesh);
protected:
	UE_API void BuildModifiedMeshUpToTarget(Geometry::FDynamicMesh3& OutResultMesh, bool bIncludeTargetModifier = false);

	UE_API virtual TArray<Geometry::FOrientedBox3d> GetBounds() const;
	UE_API virtual TArray<MeshPartition::UModifierComponent*> GetModifiersToProcess() const;
protected:
	UPROPERTY()
	TObjectPtr<MeshPartition::UModifierComponent> TargetModifier = nullptr;
	UPROPERTY()
	mutable TArray<TWeakObjectPtr<MeshPartition::APreviewSection>> PreviewSections;

	TArray<uint8> ChannelTable;
	FVector2f ChannelTexcoordDesc;

	UPROPERTY()
	TObjectPtr<UTexture> ChannelTexture;
	UPROPERTY()
	TObjectPtr<UMaterialInstanceDynamic> MID;

	template <typename KeyType, typename ValueType>
	using TWeakObjectPtrKeyMap = TMap<KeyType, ValueType, FDefaultSetAllocator, TWeakObjectPtrMapKeyFuncs<KeyType, ValueType>>;

	/** Cached UV topology for fast texture updates during painting */
	TOptional<MeshPartition::FChannelTextureRenderer::FChannelRenderUVMeshTopology> CachedUVTopology;

	mutable TWeakObjectPtrKeyMap<TWeakObjectPtr<MeshPartition::APreviewSection>, TWeakObjectPtr<AActor>> PreviewActors;

private:
	// When using SetOwnerVisibility, we hide all base sections that were entangled with our preview sections. This
	//  keeps track of them so that we can unhide them.
	mutable TArray<TWeakObjectPtr<MeshPartition::UMeshProviderModifier>> HiddenBaseSections;
};

/**
* UToolTarget specialization that supports editing megamesh modifiers by extracting
* the mesh as it exists before or after the modifier is applied
*/
UCLASS(Transient, MinimalAPI)
class UEditableModifierToolTarget : 
	public MeshPartition::UModifierToolTarget,
	public IDynamicMeshCommitter
{
	GENERATED_BODY()
	
public:
	// IDynamicMeshProvider implementation
	UE_API virtual Geometry::FDynamicMesh3 GetDynamicMesh() override;
	
	// IDynamicMeshCommitter implementation
	UE_API virtual void CommitDynamicMesh(const Geometry::FDynamicMesh3& InMesh, const FDynamicMeshCommitInfo& InCommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

protected:
	UE_API virtual TArray<Geometry::FOrientedBox3d> GetBounds() const override;
	UE_API virtual TArray<MeshPartition::UModifierComponent*> GetModifiersToProcess() const override;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(Transient, MinimalAPI)
class UModifierToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()
public:

	UE_API virtual bool CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) override;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(Transient, MinimalAPI)
class UEditableModifierToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()
public:

	UE_API virtual bool CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) override;
};
} // namespace UE::MeshPartition

#undef UE_API
