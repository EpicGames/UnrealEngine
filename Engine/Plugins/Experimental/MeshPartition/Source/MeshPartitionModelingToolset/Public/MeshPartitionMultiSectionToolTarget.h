// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ToolTargets/ToolTarget.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/PrimitiveComponentBackedTarget.h"

#include "MeshPartitionMultiSectionToolTarget.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
class UMeshProviderModifier;

/**
* Tool target that combines multiple base sections to allow them to be edited together, and splits
*  them back out on commit. Also supports single base section editing.
*/
UCLASS(Transient, MinimalAPI)
class UMultiSectionToolTarget : public UToolTarget
	, public IDynamicMeshProvider
	, public IDynamicMeshCommitter

	//~ This target isn't backed by a single primitive component, and it doesn't actually provide materials.
	//~  Unfortunately, numerous tools that we want runnable on this target have these interfaces as requirements,
	//~  so we provide implementations for them...
	, public IPrimitiveComponentBackedTarget
	, public IMaterialProvider

	//~ Note: if we implement IMegaMeshComponentBackedTarget, we may need to adjust tools that currently
	//~  expect that interface. Namely, the merge tool uses it to get multiple single-section targets,
	//~  and it would either need to be modified to ask for that target specifically, or adjusted to use
	//~  the multi-section target.
{
	GENERATED_BODY()
	
public:
	void Initialize(const TArray<TObjectPtr<MeshPartition::UMeshProviderModifier>> BaseSections);

	// UToolTarget
	UE_API virtual bool IsValid() const override;

	// IDynamicMeshProvider implementation
	UE_API virtual Geometry::FDynamicMesh3 GetDynamicMesh() override;

	// IDynamicMeshCommitter implementation
	UE_API virtual void CommitDynamicMesh(const Geometry::FDynamicMesh3& InMesh, const FDynamicMeshCommitInfo& InCommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

	//// IMaterialProvider implementation
	UE_API virtual int32 GetNumMaterials() const override;
	UE_API virtual UMaterialInterface* GetMaterial(int32 InMaterialIndex) const override;
	UE_API virtual void GetMaterialSet(FComponentMaterialSet& OutMaterialSet, bool bInPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& InMaterialSet, bool bInApplyToAsset) override;	

	// IPrimitiveComponentBackedTarget implementation
	UE_API virtual UPrimitiveComponent* GetOwnerComponent() const override;
	UE_API virtual USceneComponent* GetOwnerSceneComponent() const override;
	UE_API virtual AActor* GetOwnerActor() const override;
	UE_API virtual void SetOwnerVisibility(bool bVisible) const override;
	UE_API virtual FTransform GetWorldTransform() const override;
	UE_API virtual bool HitTestComponent(const FRay& WorldRay, FHitResult& OutHit) const override;

	const TArray<TWeakObjectPtr<MeshPartition::UMeshProviderModifier>>& GetBaseSections() const { return BaseSections; }

private:
	Geometry::FDynamicMesh3 BuildMergedMesh();
	void SplitMeshesToSections(const Geometry::FDynamicMesh3& InMesh);

	FTransform TargetTransform = FTransform::Identity;

	UPROPERTY()
	TArray<TWeakObjectPtr<MeshPartition::UMeshProviderModifier>> BaseSections;
	// 1:1 with BaseSections. Used to figure out which channels a section originally had.
	TArray<TArray<FName>> PerSectionChannels;
};

UCLASS(Transient, MinimalAPI)
class UMultiSectionToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()
public:
	virtual bool CanBuildTarget(UObject* SourceObject, const FToolTargetTypeRequirements& TargetTypeInfo) const override;
	virtual UToolTarget* BuildTarget(UObject * SourceObject, const FToolTargetTypeRequirements & TargetTypeInfo) override;

	virtual int32 CanBuildTargets(const TArray<UObject*>&InputObjects, const FToolTargetTypeRequirements & TargetTypeInfo, TArray<bool>&WouldBeUsedOut) override;
	virtual TArray<UToolTarget*> BuildTargets(const TArray<UObject*>&InputObjects, const FToolTargetTypeRequirements & TargetTypeInfo, TArray<bool>&WasUsedOut) override;
	virtual UToolTarget* BuildFirstTarget(const TArray<UObject*>&InputObjects, const FToolTargetTypeRequirements & TargetTypeInfo, TArray<bool>&WasUsedOut) override;
};
} // UE::MeshPartition

#undef UE_API
