// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"

#include "MeshPartitionPlaceModifierTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

class UCombinedTransformGizmo;
class UDragAlignmentMechanic;

namespace UE::MeshPartition
{
UENUM()
enum class EModifierClassType : uint8
{
	InstancedPatch,
	Project,
	Patch,
	Remesh,
	Spline,
	TexturePatch,
	MeshLayer,
	Noise,
	Boolean,
	Lattice,
	SplineRemesh,
	WeightUtility,

	// Insert new modifier types above this line
	LastModifier UMETA(Hidden)
};

class UPlaceModifierTool;
class AMeshPartition;
class UModifierComponent;

UCLASS(MinimalAPI)
class UPlaceModifierToolBuilder : public USingleSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual USingleSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;

	int32 DefaultModifierTypeID = -1;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UPlaceModifierToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, Category = "Add Modifier")
	MeshPartition::EModifierClassType ModifierType = MeshPartition::EModifierClassType::Patch;

};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UPlaceModifierTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()

public:

	UE_API UPlaceModifierTool();

	UE_API virtual void Setup() override;
	UE_API virtual void Shutdown(EToolShutdownType InShutdownType) override;

	UE_API void Render(IToolsContextRenderAPI* RenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	virtual bool CanAccept() const override { return true; }

	UE_API void SetDefaultModifierType(int32 InModifierTypeID);

protected:

	UE_API TObjectPtr<UClass> GetModifierToCreateClass();
	UE_API void SetupNewModifier(AActor*, MeshPartition::UModifierComponent*);
	UE_API void AssignToActiveLayer(MeshPartition::UModifierComponent*);

	TWeakObjectPtr<AMeshPartition> TargetMegaMesh;

	UPROPERTY()
	TObjectPtr<MeshPartition::UPlaceModifierToolProperties> Settings;

	UPROPERTY()
	TObjectPtr<UCombinedTransformGizmo> Gizmo = nullptr;

	UPROPERTY()
	TObjectPtr<UDragAlignmentMechanic> DragAlignmentMechanic = nullptr;
private:
	int32 DefaultModifierTypeID = -1;

	TArray<MeshPartition::EModifierClassType> ModifierTypes;
};
} // namespace UE::MeshPartition

#undef UE_API
