// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MultiSelectionMeshEditingTool.h"
#include "InteractiveToolBuilder.h"
#include "InteractiveToolQueryInterfaces.h"

#include "Drawing/PreviewGeometryActor.h"

#include "MeshPartitionResectionTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI)
class UMeshPartitionResectionToolBuilder : public UMultiSelectionMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual UMultiSelectionMeshEditingTool* CreateNewTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class UMeshPartitionResectionToolProperties : public UInteractiveToolPropertySet
{
	GENERATED_BODY()
public:
	UPROPERTY(EditAnywhere, Category=Merge)
	bool bWeldEdges = true;

	/**
	* Number of sections in X/Y/Z dimension.
	*/
	UPROPERTY(EditAnywhere, Category = Sections,
		meta = (DisplayName = "Layout", ClampMin=1, UIMax=64, ClampMax=1024))
	FIntVector SectionLayout = FIntVector(4, 4, 1);
};


// -------------------------------------------------------------------------------------------------------------------------

/**
* Modeling Mode Tool to merge multiple MegaMesh Sections to a single section.
*/
UCLASS(MinimalAPI)
class UMeshPartitionResectionTool : public UMultiSelectionMeshEditingTool, public IInteractiveToolExclusiveToolAPI, public IInteractiveToolShutdownQueryAPI
{
	GENERATED_BODY()

public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType InShutdownType) override;




private:

	// IInteractiveToolShutdownQueryAPI implementation
	virtual EToolShutdownType GetPreferredShutdownType(EToolShutdownReason, EToolShutdownType) const override { return EToolShutdownType::Cancel; }

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }
	UE_API virtual void OnTick(float DeltaTime) override;

	// internal helpers

	void Resection();
	void CreateOrUpdatePreviewGeometry();


	FIntVector GetClampedSectionLayout() const
	{
		return Properties->SectionLayout.ComponentMax(FIntVector(1));
	}

	UPROPERTY()
	TObjectPtr<UMeshPartitionResectionToolProperties> Properties;

	UPROPERTY()
	TObjectPtr<UPreviewGeometry> PreviewGeometry;

	FBox ResectionBounds;
	bool bPreviewGeometryNeedsUpdate = false;

	

};
} // namespace UE::MeshPartition

#undef UE_API
