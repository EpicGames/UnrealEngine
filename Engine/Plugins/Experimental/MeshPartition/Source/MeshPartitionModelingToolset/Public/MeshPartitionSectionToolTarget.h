// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "MeshPartitionComponentBackedTarget.h"
#include "ToolTargets/DynamicMeshComponentToolTarget.h"

#include "MeshPartitionSectionToolTarget.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
/**
* Target that has the exact same support as a dynamic mesh component, by operating on the underlying
*  base section dynamic mesh component.
*/
UCLASS(Transient, MinimalAPI)
class USectionToolTarget : 
	public UDynamicMeshComponentToolTarget,
	public IMeshPartitionComponentBackedTarget
{
	GENERATED_BODY()

public:
	
	// IPrimitiveComponentBackedTarget
	UE_API virtual void SetOwnerVisibility(bool bVisible) const override;

protected:
	friend class USectionToolTargetFactory;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(Transient, MinimalAPI)
class USectionToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()

public:

	UE_API virtual bool CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InTargetTypeInfo) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InTargetTypeInfo) override;
};
} // UE::MeshPartition

#undef UE_API
