// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BaseTools/MeshSurfacePointMeshEditingTool.h"
#include "DynamicMeshBrushTool.h"
#include "MeshSelectionTool.h"

#include "MeshPartitionSplitTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
UCLASS(MinimalAPI)
class USplitToolBuilder : public UMeshSurfacePointMeshEditingToolBuilder
{
	GENERATED_BODY()

public:
	UE_API virtual UMeshSurfacePointTool* CreateNewTool(const FToolBuilderState& InSceneState) const override;

	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(MinimalAPI)
class USplitTool : public UMeshSelectionTool
{
	GENERATED_BODY()

public:

	UE_API USplitTool();

	UE_API virtual void Setup() override;

	UE_API virtual void OnShutdown(EToolShutdownType InShutdownType) override;

	virtual void ApplyAction(EMeshSelectionToolActions InActionType) override { /* do nothing */ }
protected:
	UE_API void SeparateSelectedTriangles();
};
} // UE::MeshPartition

#undef UE_API
