// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InteractiveToolChange.h"
#include "BaseTools/SingleSelectionMeshEditingTool.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"

#include "MeshPartitionStitchTool.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

PREDECLARE_GEOMETRY(struct FGroupTopologySelection);
PREDECLARE_GEOMETRY(class FDynamicMesh3);
PREDECLARE_GEOMETRY(class FGroupTopology);

class UMeshOpPreviewWithBackgroundCompute;
class UPolygonSelectionMechanic;


namespace UE::MeshPartition
{

class UStitchTool;
/**
 * ToolBuilder
 */
UCLASS(MinimalAPI)
class UStitchToolBuilder : public UInteractiveToolWithToolTargetsBuilder
{
	GENERATED_BODY()
public:
	UE_API virtual UInteractiveTool* BuildTool(const FToolBuilderState& SceneState) const override;
	UE_API virtual bool CanBuildTool(const FToolBuilderState& SceneState) const override;

	/** Returns the tool target type requirements for the target MegaMesh for this tool. */
	UE_API virtual const FToolTargetTypeRequirements& GetTargetRequirements() const override;
	/** Returns the tool target type requirements for meshes that are stitchable into the target MegaMesh. */
	UE_API const FToolTargetTypeRequirements& GetStitchableTargetRequirements() const;
};

UENUM()
enum class EStitchToolActions
{
	NoAction,
	WeldEdges,
	FillHole
};

UCLASS(MinimalAPI)
class UStitchToolActions : public UInteractiveToolPropertySet
{
	GENERATED_BODY()

public:
	TWeakObjectPtr<MeshPartition::UStitchTool> ParentTool;

	void Initialize(MeshPartition::UStitchTool* ParentToolIn) { ParentTool = ParentToolIn; }

	UE_API void PostAction(MeshPartition::EStitchToolActions InAction);

	UFUNCTION(CallInEditor)
	UE_API void Weld();

	UFUNCTION(CallInEditor)
	UE_API void FillHole();
};

class FStitchToolMeshChange : public FToolCommandChange
{
public:
	FStitchToolMeshChange(TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChangeIn)
		: MeshChange(MoveTemp(MeshChangeIn))
	{};

	UE_API virtual void Apply(UObject* InObject) override;
	UE_API virtual void Revert(UObject* InObject) override;
	virtual FString ToString() const override { return  TEXT("MeshPartition::FStitchToolMeshChange"); }
protected:
	TUniquePtr<UE::Geometry::FDynamicMeshChange> MeshChange;
};

UCLASS(MinimalAPI)
class UStitchTool : public USingleSelectionMeshEditingTool
{
	GENERATED_BODY()
public:
	UE_API virtual void Setup() override;
	UE_API virtual void OnShutdown(EToolShutdownType InShutdownType) override;
	UE_API virtual void OnTick(float InDeltaTime) override;
	UE_API virtual void Render(IToolsContextRenderAPI* InRenderAPI) override;

	virtual bool HasCancel() const override { return true; }
	virtual bool HasAccept() const override { return true; }

	UE_API void RequestAction(MeshPartition::EStitchToolActions InAction);
	
	UE_API UE::Geometry::FDynamicMeshAABBTree3& GetSpatial();

	UE_API void OnSelectionModifiedEvent();

	UE_API void UpdateFromCurrentMesh(bool bUpdateTopology);

	UE_API void SetTargetToStitch(UToolTarget* InTargetToStitch);

protected:
	UE_API void ApplyWeld();
	UE_API void ApplyFillHole();

	UE_API bool BeginMeshFaceEditChange();
	UE_API bool BeginMeshEdgeEditChange();
	UE_API void EmitCurrentMeshChangeAndUpdate(const FText& InTransactionLabel, TUniquePtr<UE::Geometry::FDynamicMeshChange> InMeshChange);

	UPROPERTY()
	TObjectPtr<UMeshOpPreviewWithBackgroundCompute> Preview = nullptr;

	UPROPERTY()
	TObjectPtr<UToolTarget> TargetToStitch = nullptr;

	UPROPERTY()
	TObjectPtr<UPolygonSelectionMechanic> SelectionMechanic = nullptr;
	
	TSharedPtr<UE::Geometry::FGroupTopology> Topology;
	TSharedPtr<UE::Geometry::FDynamicMesh3> CurrentMesh;
	TSharedPtr<UE::Geometry::FDynamicMeshAABBTree3> MeshSpatial;
	
	UPROPERTY()
	TObjectPtr<MeshPartition::UStitchToolActions> EditActions = nullptr;

	FTransform WorldTransform;

	UE::Geometry::FFrame3d ActiveSelectionFrameLocal;
	UE::Geometry::FFrame3d ActiveSelectionFrameWorld;
	TArray<int32> ActiveTriangleSelection;
	UE::Geometry::FAxisAlignedBox3d ActiveSelectionBounds;
	struct FSelectedEdge
	{
		int32 EdgeTopoID;
		TArray<int32> EdgeIDs;
	};
	TArray<FSelectedEdge> ActiveEdgeSelection;

	MeshPartition::EStitchToolActions PendingAction = MeshPartition::EStitchToolActions::NoAction;

	bool bSpatialDirty = true;
	bool bSelectionStateDirty = true;

	friend class FStitchToolMeshChange;
};

} // namespace UE::MeshPartition

#undef UE_API
