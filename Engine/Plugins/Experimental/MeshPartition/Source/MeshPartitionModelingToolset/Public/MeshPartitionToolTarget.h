// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MeshPartitionComponentBackedTarget.h"
#include "TargetInterfaces/DynamicMeshCommitter.h"
#include "TargetInterfaces/DynamicMeshProvider.h"
#include "TargetInterfaces/MaterialProvider.h"
#include "TargetInterfaces/MeshDescriptionProvider.h"
#include "TargetInterfaces/PhysicsDataSource.h"
#include "ToolTargets/PrimitiveComponentToolTarget.h"
#include "UObject/Object.h"
#include "MeshPartitionToolTarget.generated.h"

#define UE_API MESHPARTITIONMODELINGTOOLSET_API

namespace UE::MeshPartition
{
class UMeshPartitionEditorComponent;
class UMeshProviderModifier;

/**
* UToolTarget specialization that supports a entire AMeshPartition actor and its corresponding UMeshPartitionEditorComponent.
* A specialized per-section target is available in UMegaMeshSectionTooLTarget.
*/
UCLASS(Transient, MinimalAPI)
class UMeshPartitionToolTarget :
	public UPrimitiveComponentToolTarget,
	public IDynamicMeshProvider,
	public IDynamicMeshCommitter,
	public IMaterialProvider,
	public IPhysicsDataSource
{
	GENERATED_BODY()
	
public:
	UE_API void Initialize(UMeshPartitionEditorComponent* InEditorComponent);
	
	// IDynamicMeshProvider implementation
	UE_API virtual Geometry::FDynamicMesh3 GetDynamicMesh() override;

	// IDynamicMeshCommitter implementation
	UE_API virtual void CommitDynamicMesh(const Geometry::FDynamicMesh3& InMesh, const FDynamicMeshCommitInfo& InCommitInfo) override;
	using IDynamicMeshCommitter::CommitDynamicMesh; // unhide the other overload

	// IMaterialProvider implementation
	UE_API int32 GetNumMaterials() const override;
	UE_API UMaterialInterface* GetMaterial(int32 InMaterialIndex) const override;
	UE_API void GetMaterialSet(FComponentMaterialSet& OutMaterialSet, bool bInPreferAssetMaterials) const override;
	UE_API virtual bool CommitMaterialSetUpdate(const FComponentMaterialSet& InMaterialSet, bool bInApplyToAsset) override;	

	// IPhysicsDataSource implementation
	UE_API virtual UBodySetup* GetBodySetup() const override;
	UE_API virtual IInterface_CollisionDataProvider* GetComplexCollisionProvider() const override;

	// UPrimitiveComponentToolTarget
	UE_API virtual void SetOwnerVisibility(bool bInVisible) const override;
	UE_API virtual FTransform GetWorldTransform() const override;
	
protected:
	UE_API Geometry::FDynamicMesh3 BuildMergedMesh();
	UE_API void SplitMeshesToSections(const Geometry::FDynamicMesh3& InMesh);

protected:
	TArray<MeshPartition::UMeshProviderModifier*> MeshModifiers;
	// Cached world frame for the merged mesh
	FTransform TargetTransform;
	bool bIsMeshModifiersInitialized = false;
};

// -------------------------------------------------------------------------------------------------------------------------

UCLASS(Transient, MinimalAPI)
class UMeshPartitionToolTargetFactory : public UToolTargetFactory
{
	GENERATED_BODY()
public:

	UE_API virtual bool CanBuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) const override;

	UE_API virtual UToolTarget* BuildTarget(UObject* InSourceObject, const FToolTargetTypeRequirements& InRequirements) override;
};
} // namespace UE::MeshPartition

#undef UE_API
